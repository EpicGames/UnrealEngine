{
    "FileVersion" : 3,
    "Version" : 1,
    "VersionName" : "0.1",
    "FriendlyName" : "Chaos Flesh",
    "Description" : "Chaos Flesh Simulation",
    "Category": "Physics",
    "CreatedBy" : "Epic Games, Inc.",
    "CreatedByURL" : "http://epicgames.com",
    "DocsURL" : "",
    "MarketplaceURL" : "",
    "SupportURL" : "",
    "EnabledByDefault" : false,
    "CanContainContent" : true,
	"IsExperimentalVersion": true,
    "Installed" : false,
	"Modules": [
		{
			"Name": "ChaosFlesh",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ChaosFleshEngine",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ChaosFleshEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ChaosFleshNodes",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "ChaosFleshDeprecatedNodes",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "ProceduralMeshComponent",
			"Enabled": true
		},
		{
			"Name": "Dataflow",
			"Enabled": true
		},
		{
			"Name": "ChaosCaching",
			"Enabled": true
		},
        {
            "Name": "ChaosCachingUSD",
			"Enabled": true,
			"SupportedTargetPlatforms": [
				"Win64"
			]
        },
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		}, 
		{
			"Name": "ComputeFramework",
			"Enabled": true
		}, 
		{
			"Name": "DeformerGraph",
			"Enabled": true
		},
        {
            "Name": "USDImporter",
			"Enabled": true,
			"SupportedTargetPlatforms": [
				"Win64"
			]
        },
		{
			"Name": "TetMeshing",
			"Enabled": true
		}
	]
}
