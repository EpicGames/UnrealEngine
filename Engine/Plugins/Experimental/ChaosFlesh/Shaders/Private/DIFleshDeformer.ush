// Copyright Epic Games, Inc.  All Rights Reserved.

uint {DataInterfaceName}_NumVertices;

uint {DataInterfaceName}_NumTetVertexBuffer;
uint {DataInterfaceName}_NumTetRestVertexBuffer;

uint {DataInterfaceName}_NumVertsToParentsBuffer;
int {DataInterfaceName}_VertsToParentsBufferOffset;
uint {DataInterfaceName}_VertsToParentsBufferStride;

uint {DataInterfaceName}_NumParentsBuffer;
int {DataInterfaceName}_ParentsBufferOffset;
uint {DataInterfaceName}_ParentsBufferStride;	// 1 = 8 bit, 2 = 16 bit, 4 = 32 bit

uint {DataInterfaceName}_NumWeightsBuffer;
uint {DataInterfaceName}_NumOffsetBuffer;
uint {DataInterfaceName}_NumMaskBuffer;

Buffer<float> {DataInterfaceName}_TetRestVertexBuffer;
Buffer<float> {DataInterfaceName}_TetVertexBuffer;

Buffer<uint> {DataInterfaceName}_VertsToParentsBuffer;
Buffer<uint> {DataInterfaceName}_ParentsBuffer;
Buffer<half> {DataInterfaceName}_WeightsBuffer;
Buffer<half> {DataInterfaceName}_OffsetBuffer;
Buffer<half> {DataInterfaceName}_MaskBuffer;

uint ReadNumVertices_{DataInterfaceName}(uint Index)
{
	return {DataInterfaceName}_NumVertices;
}

#define ENABLE_DEFORMER_FLESH 1 // for syntax highlighting

//
// Tet mesh points, rest and dynamic 
//

float3 ReadRestTetVertex_{DataInterfaceName}(int Index)
{
#if ENABLE_DEFORMER_FLESH
	if(Index >= 0 && Index*3+2 < {DataInterfaceName}_NumTetRestVertexBuffer)
	{
		return float3(
			{DataInterfaceName}_TetRestVertexBuffer[Index * 3], 
			{DataInterfaceName}_TetRestVertexBuffer[Index * 3 + 1], 
			{DataInterfaceName}_TetRestVertexBuffer[Index * 3 + 2]);
	}
#endif
	return float3(0,0,0);
}

float3 ReadTetVertex_{DataInterfaceName}(int Index)
{
#if ENABLE_DEFORMER_FLESH
	if(Index >= 0 && Index*3+2 < {DataInterfaceName}_NumTetVertexBuffer)
	{
		return float3(
			{DataInterfaceName}_TetVertexBuffer[Index * 3], 
			{DataInterfaceName}_TetVertexBuffer[Index * 3 + 1], 
			{DataInterfaceName}_TetVertexBuffer[Index * 3 + 2]);
	}
#endif
	return float3(0,0,0);
}

//
// Render mesh to tet bindings
//

// Stride 1 = 8 bit, 2 = 16 bit, 4 = 32 bit - based off of sizeof(<index type>)
uint UnpackUInt(const Buffer<uint> Array, const uint Index, const uint Stride, const uint NumValues)
{
	// NumValues reflects the number of packed values in Array
	if(Index >= NumValues)
		return 0;
	
	uint ValsPerEntry = 4 / Stride;
	uint MajorIndex = floor(Index / ValsPerEntry);

	uint Val = Array[MajorIndex];

	uint MinorIndex = Index % ValsPerEntry;
	return BitFieldExtractU32(Val, Stride*8, Stride*8*MinorIndex);
}

int UnpackInt_32bit(const Buffer<uint> Array, const uint Index, const uint ArraySize)
{
	return Index >= ArraySize ? -1 : int(Array[Index]);
}

int UnpackInt_16bit(const Buffer<uint> Array, const uint Index, const uint ArraySize)
{
	// Array: [..., <A,B>, ...], where A-B are 16 bit ints, and Index refers to one of A or B.
	uint BaseIndex = floor(Index / 2); // 32 bit index
	uint SubIndex = Index % 2;  // [0, 1], corrsponding to A or B
	return BaseIndex >= ArraySize ? -1 : int((Array[BaseIndex] >> (SubIndex * 16)) & 0xFFFF);
}

int UnpackInt_8bit(const Buffer<uint> Array, const uint Index, const uint ArraySize)
{
	// Array: [..., <A,B,C,D>, ...], where A-D are 8 bit ints, and Index refers to one of A-D.
	uint BaseIndex = floor(Index / 4); // 32 bit index
	uint SubIndex = Index % 4;  // [0, 3], corresponding to A, B, C, or D
	return BaseIndex >= ArraySize ? -1 : int((Array[BaseIndex] >> (SubIndex * 8)) & 0xFF);
}

int4 ReadParents32_{DataInterfaceName}(uint BufferIndex)
{
#if ENABLE_DEFORMER_FLESH
	return int4(
		UnpackInt_32bit({DataInterfaceName}_ParentsBuffer, BufferIndex    , {DataInterfaceName}_NumParentsBuffer) - {DataInterfaceName}_ParentsBufferOffset, 
		UnpackInt_32bit({DataInterfaceName}_ParentsBuffer, BufferIndex + 1, {DataInterfaceName}_NumParentsBuffer) - {DataInterfaceName}_ParentsBufferOffset, 
		UnpackInt_32bit({DataInterfaceName}_ParentsBuffer, BufferIndex + 2, {DataInterfaceName}_NumParentsBuffer) - {DataInterfaceName}_ParentsBufferOffset, 
		UnpackInt_32bit({DataInterfaceName}_ParentsBuffer, BufferIndex + 3, {DataInterfaceName}_NumParentsBuffer) - {DataInterfaceName}_ParentsBufferOffset);
#endif
	return int4(-1,-1,-1,-1);
}

int4 ReadParents16_{DataInterfaceName}(uint BufferIndex)
{
#if ENABLE_DEFORMER_FLESH
	return int4(
		UnpackInt_16bit({DataInterfaceName}_ParentsBuffer, BufferIndex    , {DataInterfaceName}_NumParentsBuffer) - {DataInterfaceName}_ParentsBufferOffset, 
		UnpackInt_16bit({DataInterfaceName}_ParentsBuffer, BufferIndex + 1, {DataInterfaceName}_NumParentsBuffer) - {DataInterfaceName}_ParentsBufferOffset, 
		UnpackInt_16bit({DataInterfaceName}_ParentsBuffer, BufferIndex + 2, {DataInterfaceName}_NumParentsBuffer) - {DataInterfaceName}_ParentsBufferOffset, 
		UnpackInt_16bit({DataInterfaceName}_ParentsBuffer, BufferIndex + 3, {DataInterfaceName}_NumParentsBuffer) - {DataInterfaceName}_ParentsBufferOffset);
#endif
	return int4(-1,-1,-1,-1);
}

int4 ReadParents8_{DataInterfaceName}(uint BufferIndex)
{
#if ENABLE_DEFORMER_FLESH
	return int4(
		UnpackInt_8bit({DataInterfaceName}_ParentsBuffer, BufferIndex    , {DataInterfaceName}_NumParentsBuffer) - {DataInterfaceName}_ParentsBufferOffset, 
		UnpackInt_8bit({DataInterfaceName}_ParentsBuffer, BufferIndex + 1, {DataInterfaceName}_NumParentsBuffer) - {DataInterfaceName}_ParentsBufferOffset, 
		UnpackInt_8bit({DataInterfaceName}_ParentsBuffer, BufferIndex + 2, {DataInterfaceName}_NumParentsBuffer) - {DataInterfaceName}_ParentsBufferOffset, 
		UnpackInt_8bit({DataInterfaceName}_ParentsBuffer, BufferIndex + 3, {DataInterfaceName}_NumParentsBuffer) - {DataInterfaceName}_ParentsBufferOffset);
#endif
	return int4(-1,-1,-1,-1);
}

int4 ReadParents_{DataInterfaceName}(uint VertexIndex)
{
#if ENABLE_DEFORMER_FLESH
	uint BufferIndex = VertexIndex;

	// Many render vertices will likely share the same set of parents, so the set of parents is sparsely
	// represented.  The VertsToParentsBuffer contains the mapping from render vertices to parents.
	int ParentsIndex = 
		UnpackUInt(
			{DataInterfaceName}_VertsToParentsBuffer, 
			BufferIndex, 
			{DataInterfaceName}_VertsToParentsBufferStride, 
			{DataInterfaceName}_NumVertsToParentsBuffer);
	ParentsIndex -= {DataInterfaceName}_VertsToParentsBufferOffset;
	if(ParentsIndex >= 0)
	{
		// That gives us the index of the int4, which we have laid out in a flat array, so mult by 4.
		// The parents buffer is potentially packed 8, 16, or 32 bit numbers, but we'll deal with
		// that later.
		ParentsIndex *= 4;

		// Do lookups into the parents buffer, unpacking as necessary depending on the storage format.	
		if({DataInterfaceName}_ParentsBufferStride == 4)
			return ReadParents32_{DataInterfaceName}(ParentsIndex);
		else if({DataInterfaceName}_ParentsBufferStride == 2)
			return ReadParents16_{DataInterfaceName}(ParentsIndex);
		else // if 1
			return ReadParents8_{DataInterfaceName}(ParentsIndex);
	}
#endif
	return int4(-1,-1,-1,-1);
}

half4 ReadWeights_{DataInterfaceName}(uint VertexIndex)
{
#if ENABLE_DEFORMER_FLESH
	if(VertexIndex < {DataInterfaceName}_NumWeightsBuffer)
	{
		uint BufferIndex = VertexIndex;
		return half4(
			{DataInterfaceName}_WeightsBuffer[BufferIndex * 4], 
			{DataInterfaceName}_WeightsBuffer[BufferIndex * 4 + 1], 
			{DataInterfaceName}_WeightsBuffer[BufferIndex * 4 + 2],
			{DataInterfaceName}_WeightsBuffer[BufferIndex * 4 + 3]);
	}
#endif
	return half4(0,0,0,0);
}

half3 ReadOffset_{DataInterfaceName}(uint VertexIndex)
{
#if ENABLE_DEFORMER_FLESH
	if(VertexIndex < {DataInterfaceName}_NumOffsetBuffer)
	{
		uint BufferIndex = VertexIndex;
		return half3(
			{DataInterfaceName}_OffsetBuffer[BufferIndex * 3], 
			{DataInterfaceName}_OffsetBuffer[BufferIndex * 3 + 1], 
			{DataInterfaceName}_OffsetBuffer[BufferIndex * 3 + 2]);
	}
#endif
	return half3(0,0,0);
}

half ReadMask_{DataInterfaceName}(uint VertexIndex)
{
#if ENABLE_DEFORMER_FLESH
	if(VertexIndex < {DataInterfaceName}_NumMaskBuffer)
	{
		uint BufferIndex = VertexIndex;
		return {DataInterfaceName}_MaskBuffer[BufferIndex];
	}
#endif
	return half(0);
}

//
// Triangle orientation
//

// [ Duff et al. 2017, "Building an Orthonormal Basis, Revisited" ]
// Discontinuity at TangentZ.z == 0
float3x3 GetTangentBasis( float3 TangentZ )
{
    const float Sign = TangentZ.z >= 0 ? 1 : -1;
	const float a = -rcp( Sign + TangentZ.z );
    const float b = TangentZ.x * TangentZ.y * a;

    float3 TangentX = { 1 + Sign * a * Pow2( TangentZ.x ), Sign * b, -Sign * TangentZ.x };
    float3 TangentY = { b,  Sign + a * Pow2( TangentZ.y ), -TangentZ.y };

    return float3x3( TangentX, TangentY, TangentZ );
}

float3 SafeNormalize(float3 Vec)
{
	float L = length(Vec);
	if(L > 1.0e-6)
	{
		return Vec / L;
	}
	return float3(1,0,0);
}

float3x3 GetOrthogonalBasisVectors(float3 PtA, float3 PtB, float3 PtC)
{
	float3 EdgeBA = PtB - PtA;
	float3 EdgeCA = PtC - PtA;
	float3 OrthoNorm = SafeNormalize(cross(EdgeBA, EdgeCA));
	return GetTangentBasis(OrthoNorm);
/*
	float3 EdgeBA = SafeNormalize(PtB - PtA);
	float3 EdgeCA = SafeNormalize(PtC - PtA);
	float3 OrthoNorm = cross(EdgeBA, EdgeCA);
	float3 OrthoCA = cross(EdgeBA, OrthoNorm);
	return float3x3(
		EdgeBA[0], OrthoCA[0], OrthoNorm[0],
		EdgeBA[1], OrthoCA[1], OrthoNorm[1],
		EdgeBA[2], OrthoCA[2], OrthoNorm[2]); // Col major
*/
}


float3x3 GetRestOrthogonalBasisVectors(int4 Tet) // assumes Tet[3] == -1
{
	float3 PtA = ReadRestTetVertex_{DataInterfaceName}(Tet[0]);
	float3 PtB = ReadRestTetVertex_{DataInterfaceName}(Tet[1]);
	float3 PtC = ReadRestTetVertex_{DataInterfaceName}(Tet[2]);
	return GetOrthogonalBasisVectors(PtA, PtB, PtC);
}

float3x3 GetDynamicOrthogonalBasisVectors(int4 Tet) // assumes Tet[3] == -1
{
	float3 PtA = ReadTetVertex_{DataInterfaceName}(Tet[0]);
	float3 PtB = ReadTetVertex_{DataInterfaceName}(Tet[1]);
	float3 PtC = ReadTetVertex_{DataInterfaceName}(Tet[2]);
	return GetOrthogonalBasisVectors(PtA, PtB, PtC);
}

float3 GetRotatedOffsetVector(uint VertexIndex, int4 Tet)
{
	float3 Offset = { 0, 0, 0 };
#if ENABLE_DEFORMER_FLESH
	Offset = ReadOffset_{DataInterfaceName}(VertexIndex);
	if(Offset[0] != 0 && Offset[1] != 0 && Offset[2] != 0)
	{
		float3x3 RestRotInv = transpose(GetRestOrthogonalBasisVectors(Tet)); // Ortho matrix, so T == I
		float3x3 CurrRot = GetDynamicOrthogonalBasisVectors(Tet);
		Offset = mul(Offset, mul(RestRotInv, CurrRot));
	}
#endif
	return Offset;
}

//
// Helpers
//

float3 GetEmbeddedPos_{DataInterfaceName}(uint VertexIndex)
{
	float3 Pos = { 0, 0, 0 };
#if ENABLE_DEFORMER_FLESH
	int4 Parents = ReadParents_{DataInterfaceName}(VertexIndex);
	half4 Weights = ReadWeights_{DataInterfaceName}(VertexIndex);
	UNROLL for(uint i=0; i < 4; i++)
	{
		// Returns zero vec if out of range.
		Pos += ReadTetVertex_{DataInterfaceName}(Parents[i]) * Weights[i];
	}
	Pos += GetRotatedOffsetVector(VertexIndex, Parents);
#endif
	return Pos;
}
