// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	
	public class ChaosFlesh : ModuleRules
	{
		public ChaosFlesh(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Chaos",
					"InputCore",
				}
			);

			PrivateDefinitions.Add("CHAOS_INCLUDE_LEVEL_1=1");
		}
	}

}
