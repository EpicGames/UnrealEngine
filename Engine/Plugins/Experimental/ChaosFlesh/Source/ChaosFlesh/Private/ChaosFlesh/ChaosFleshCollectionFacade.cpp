// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFlesh/ChaosFleshCollectionFacade.h"

#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/TransformCollection.h"
#include "GeometryCollection/GeometryCollection.h"
#include "ChaosFlesh/TetrahedralCollection.h"

#define LOCTEXT_NAMESPACE "FFleshCollectionFacade"

namespace Chaos 
{

	FFleshCollectionFacade::FFleshCollectionFacade(FFleshCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(&InCollection)
		, BoneName(InCollection, "BoneName", FTransformCollection::TransformGroup)
		, Transform(InCollection, FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup)
		, TransformToGeometryIndex(InCollection, "TransformToGeometryIndex", FTransformCollection::TransformGroup)
		, Parent(InCollection, FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup)
		, Child(InCollection, FTransformCollection::ChildrenAttribute, FTransformCollection::TransformGroup)
		, BoneMap(InCollection, FName("BoneMap"), FName("Vertices"))
		, Vertex(InCollection, FName("Vertex"), FName("Vertices"))
		, Indices(InCollection, FName("Indices"), FName("Faces"))
		, Tetrahedron(InCollection, FTetrahedralCollection::TetrahedronAttribute, FTetrahedralCollection::TetrahedralGroup)
		, GeometryToTransformIndex(InCollection, "TransformIndex", FGeometryCollection::GeometryGroup)
		, VertexStart(InCollection, "VertexStart", FGeometryCollection::GeometryGroup)
		, VertexCount(InCollection, "VertexCount", FGeometryCollection::GeometryGroup)
		, FaceStart(InCollection, "FaceStart", FGeometryCollection::GeometryGroup)
		, FaceCount(InCollection, "FaceCount", FGeometryCollection::GeometryGroup)
	{}

	FFleshCollectionFacade::FFleshCollectionFacade(const FFleshCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(nullptr)
		, BoneName(InCollection, "BoneName", FTransformCollection::TransformGroup)
		, Transform(InCollection, FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup)
		, TransformToGeometryIndex(InCollection, "TransformToGeometryIndex", FTransformCollection::TransformGroup)
		, Parent(InCollection, FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup)
		, Child(InCollection, FTransformCollection::ChildrenAttribute, FTransformCollection::TransformGroup)
		, BoneMap(InCollection, FName("BoneMap"), FName("Vertices"))
		, Vertex(InCollection, FName("Vertex"), FName("Vertices"))
		, Indices(InCollection, FName("Indices"), FName("Faces"))
		, Tetrahedron(InCollection, FTetrahedralCollection::TetrahedronAttribute, FTetrahedralCollection::TetrahedralGroup)
		, GeometryToTransformIndex(InCollection, "TransformIndex", FGeometryCollection::GeometryGroup)
		, VertexStart(InCollection, "VertexStart", FGeometryCollection::GeometryGroup)
		, VertexCount(InCollection, "VertexCount", FGeometryCollection::GeometryGroup)
		, FaceStart(InCollection, "FaceStart", FGeometryCollection::GeometryGroup)
		, FaceCount(InCollection, "FaceCount", FGeometryCollection::GeometryGroup)
	{}

	bool FFleshCollectionFacade::IsValid() const {
		return BoneName.IsValid() && Transform.IsValid() && TransformToGeometryIndex.IsValid() && Parent.IsValid() && Child.IsValid() &&
			BoneMap.IsValid() && Vertex.IsValid() &&
			Indices.IsValid() && Tetrahedron.IsValid() &&
			GeometryToTransformIndex.IsValid() && VertexStart.IsValid() && VertexCount.IsValid() && FaceStart.IsValid() && FaceCount.IsValid();
	}

	bool FFleshCollectionFacade::IsTetrahedronValid() const {
		return Vertex.IsValid() && Tetrahedron.IsValid();
	}

	bool FFleshCollectionFacade::IsHierarchyValid() const {
		return BoneName.IsValid() && Transform.IsValid() && Parent.IsValid() && Child.IsValid();
	}

	bool FFleshCollectionFacade::IsGeometryValid() const {
		return TransformToGeometryIndex.IsValid() && GeometryToTransformIndex.IsValid()
			&& VertexStart.IsValid() && VertexCount.IsValid()
			&& FaceStart.IsValid() && FaceCount.IsValid();
	}

	int FFleshCollectionFacade::NumGeometry() const {
		if (VertexStart.IsValid())
		{
			return VertexStart.Num();
		}
		return 0;
	}

	int FFleshCollectionFacade::NumVertices() const {
		if (Vertex.IsValid())
		{
			return Vertex.Num();
		}
		return 0;
	}

	int FFleshCollectionFacade::NumFaces() const {
		if (Indices.IsValid())
		{
			return Indices.Num();
		}
		return 0;
	}

	int FFleshCollectionFacade::AppendGeometry(const FFleshCollection& NewGeomerty) 
	{
		int32 GeomIndex = NumGeometry();
		Collection->AppendGeometry(NewGeomerty);
		return GeomIndex;
	}

	void FFleshCollectionFacade::GlobalMatrices(TArray<FTransform>& OutComponentTransform)
	{
		if (Transform.IsValid() && Parent.IsValid())
		{
			GeometryCollectionAlgo::GlobalMatrices(Transform.Get(), Parent.Get(), OutComponentTransform);
		}
	}

	FTransform3f FFleshCollectionFacade::GlobalMatrix3f(int32 InIndex)
	{
		if (Transform.IsValid() && Parent.IsValid())
		{
			if (0 <= InIndex && InIndex < Transform.Num())
			{
				return GeometryCollectionAlgo::GlobalMatrix3f(Transform.Get(), Parent.Get(), InIndex);
			}
		}
		return FTransform3f::Identity;
	}


	int FFleshCollectionFacade::NumTransforms() const {
		if (Transform.IsValid())
		{
			return Transform.Num();
		}
		return 0;
	}

	void FFleshCollectionFacade::ComponentSpaceVertices(TArray<FVector3f>& OutComponentSpaceVertices) const
	{
		ComponentSpaceVertices(OutComponentSpaceVertices, 0, Vertex.Num());
	}

	void FFleshCollectionFacade::ComponentSpaceVertices(TArray<FVector3f>& OutComponentSpaceVertices, int32 Start, int32 Count) const
	{
		if (IsHierarchyValid() && Vertex.IsValid())
		{
			TArray<FTransform3f> ComponentTransform;
			GeometryCollectionAlgo::GlobalMatrices(Transform.Get(), Parent.Get(), ComponentTransform);

			OutComponentSpaceVertices.SetNumUninitialized(Count);
			for (int i = 0; i < Count; i++)
			{
				int j = i + Start;
				if (0 < BoneMap[i] && BoneMap[i] < ComponentTransform.Num())
				{
					OutComponentSpaceVertices[i] = ComponentTransform[BoneMap[j]].TransformPosition(Vertex[j]);
				}
				else
				{
					OutComponentSpaceVertices[i] = Vertex[j];
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
