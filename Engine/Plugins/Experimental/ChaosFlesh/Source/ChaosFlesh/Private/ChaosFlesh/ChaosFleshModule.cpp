// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFlesh/ChaosFleshModule.h"
#include "Modules/ModuleManager.h"

//////////////////////////////////////////////////////////////////////////
// FChaosFleshModule

class FChaosFleshModule : public IChaosFleshModuleInterface
{
  public:
    virtual void StartupModule() override
    {
        check(GConfig);
    }

    virtual void ShutdownModule() override
    {
    }

};

//////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FChaosFleshModule, ChaosFlesh);
