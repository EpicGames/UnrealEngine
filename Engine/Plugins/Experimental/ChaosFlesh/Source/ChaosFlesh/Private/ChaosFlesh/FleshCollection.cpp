// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FleshCollection.cpp: FFleshCollection methods.
=============================================================================*/

#include "ChaosFlesh/FleshCollection.h"

#include "ChaosFlesh/FleshCollectionUtility.h"
#include "GeometryCollection/Facades/CollectionKinematicBindingFacade.h"

DEFINE_LOG_CATEGORY_STATIC(FFleshCollectionLogging, Log, All);


// Attributes
const FName FFleshCollection::MassAttribute("Mass");

FFleshCollection::FFleshCollection()
	: FTetrahedralCollection()
{
	Construct();
}


void FFleshCollection::Construct()
{
	FName SelectionGroup1 = FName(GeometryCollection::Facades::FSelectionFacade::BoundGroup.ToString() + "_" + FGeometryCollection::VerticesGroup.ToString());

	FManagedArrayCollection::FConstructionParameters TransformGroupDependency(FGeometryCollection::TransformGroup);
	FManagedArrayCollection::FConstructionParameters VertexGroupDependency(FGeometryCollection::VerticesGroup);
	FManagedArrayCollection::FConstructionParameters BindingGroupDependency(SelectionGroup1);

	// Vertices Group
	AddExternalAttribute<float>(FFleshCollection::MassAttribute, FFleshCollection::VerticesGroup, Mass);
	for (float& m : Mass) { m = (float)1; }

	// Kinematic Bindings
	GeometryCollection::Facades::FKinematicBindingFacade(*this).DefineSchema();
}


FFleshCollection* FFleshCollection::NewFleshCollection(const TArray<FVector>& Vertices, const TArray<FIntVector3>& SurfaceElements, const TArray<FIntVector4>& Elements, bool bReverseVertexOrder)
{

	FFleshCollection* Collection = new FFleshCollection();
	FFleshCollection::Init(Collection, Vertices,SurfaceElements, Elements, bReverseVertexOrder);
	return Collection;
}

FFleshCollection* FFleshCollection::NewFleshCollection(const TArray<FVector>& Vertices, const TArray<FIntVector4>& Elements, const bool bReverseVertexOrder, const bool KeepInteriorFaces, const bool InvertFaces)
{
	TArray<FIntVector3> SurfaceElements;
	ChaosFlesh::GetSurfaceElements(Elements, SurfaceElements, KeepInteriorFaces, InvertFaces);

	FFleshCollection* Collection = new FFleshCollection();
	FFleshCollection::Init(Collection, Vertices, SurfaceElements, Elements, bReverseVertexOrder);
	return Collection;
}

FFleshCollection* FFleshCollection::NewFleshCollection(const FTetrahedralCollection& Base)
{
	FFleshCollection* Collection = new FFleshCollection();
	Collection->CopyMatchingAttributesFrom(Base);
	for (float& m : Collection->Mass) { m = 1; }
	return Collection;
}

void FFleshCollection::Init(FFleshCollection* Collection, const TArray<FVector>& Vertices, const TArray<FIntVector3>& SurfaceElements, const TArray<FIntVector4>& Elements, bool bReverseVertexOrder)
{
	if (Collection)
	{
		FTetrahedralCollection::Init(Collection, Vertices, SurfaceElements, Elements, bReverseVertexOrder);
		for (float& m : Collection->Mass) { m = 1; }
	}
}

