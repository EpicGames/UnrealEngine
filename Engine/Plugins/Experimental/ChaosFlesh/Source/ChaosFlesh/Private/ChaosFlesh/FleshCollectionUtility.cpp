// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GeometryCollection->cpp: FGeometryCollection methods.
=============================================================================*/

#include "ChaosFlesh/FleshCollectionUtility.h"

#include "ChaosFlesh/ChaosFlesh.h"
#include "ChaosFlesh/FleshCollection.h"
#include "GeometryCollection/TransformCollection.h"
#include "Misc/Paths.h"

#include <fstream>

namespace ChaosFlesh
{
	void GetTetFaces(
		const FIntVector4& Tet,
		FIntVector3& Face1,
		FIntVector3& Face2,
		FIntVector3& Face3,
		FIntVector3& Face4,
		const bool invert)
	{
		const int32 i = Tet[0];
		const int32 j = Tet[1];
		const int32 k = Tet[2];
		const int32 l = Tet[3];
		if (invert)
		{
			Face1 = { i, k, j };
			Face2 = { i, j, l };
			Face3 = { i, l, k };
			Face4 = { j, k, l };
		}
		else
		{
			Face1 = { i, j, k };
			Face2 = { i, l, j };
			Face3 = { i, k, l };
			Face4 = { j, l, k };
		}
	}

	FIntVector3	GetOrdered(const FIntVector3& V)
	{
		TArray<int32, TInlineAllocator<3>> VA = { V[0], V[1], V[2] };
		VA.Sort();
		return FIntVector3(VA[0], VA[1], VA[2]);
	}
	FIntVector4	GetOrdered(const FIntVector4& V)
	{
		TArray<int32, TInlineAllocator<4>> VA = { V[0], V[1], V[2], V[3]};
		VA.Sort();
		return FIntVector4(VA[0], VA[1], VA[2], VA[3]);
	}

	void GetSurfaceElements(
		const TArray<FIntVector4>& Tets,
		TArray<FIntVector3>& SurfaceElements,
		const bool KeepInteriorFaces,
		const bool InvertFaces)
	{
		FIntVector3 Faces[4];

		if (KeepInteriorFaces)
		{
			SurfaceElements.Reserve(Tets.Num() * 4);
			for (const FIntVector4& Tet : Tets)
			{
				GetTetFaces(Tet, Faces[0], Faces[1], Faces[2], Faces[3], InvertFaces);
				for (int i = 0; i < 4; i++)
					SurfaceElements.Add(Faces[i]);
			}
		}
		else
		{
			typedef TMap<int32, TPair<uint8, FIntVector3>> ZToCount;
			typedef TMap<int32, ZToCount> YToZ;
			typedef TMap<int32, YToZ> XToY;
			XToY CoordToCount;

			int32 Idx = -1;
			int32 Count = 0;
			for (const FIntVector4& Tet : Tets)
			{
				++Idx;
				if (!(Tet[0] != Tet[1] &&
					Tet[0] != Tet[2] &&
					Tet[0] != Tet[3] &&
					Tet[1] != Tet[2] &&
					Tet[1] != Tet[3] &&
					Tet[2] != Tet[3]))
				{
					UE_LOGF(LogChaosFlesh, Display, "Skipping degenerate tet %d of %d.", Idx, Tets.Num());
					continue;
				}

				GetTetFaces(Tet, Faces[0], Faces[1], Faces[2], Faces[3], InvertFaces);
				for (int i = 0; i < 4; i++)
				{
					const FIntVector3 OFace = GetOrdered(Faces[i]);
					check(OFace[0] <= OFace[1] && OFace[1] <= OFace[2]);
					const int32 OA = OFace[0];
					const int32 OB = OFace[1];
					const int32 OC = OFace[2];

					auto& zc = CoordToCount.FindOrAdd(OFace[0]).FindOrAdd(OFace[1]);
					auto zcIt = zc.Find(OFace[2]);
					if (zcIt == nullptr)
					{
						zc.Add(OFace[2], TPair<uint8,FIntVector3>((uint8)1,Faces[i]) );
						// Increment our count of lone faces.
						Count++;
					}
					else
					{
						zcIt->Key++;
						// Since we're talking about shared faces, the only way we'd
						// get a face instanced more than twice is if we had a degenerate 
						// tet mesh.
						Count--;
					}
				}
			}

			size_t nonManifold = 0;
			SurfaceElements.Reserve(Count);
			for(auto &xyIt : CoordToCount)
			{
				const YToZ& yz = xyIt.Value;
				for(auto &yzIt : yz)
				{
					const ZToCount& zc = yzIt.Value;
					for(auto &zcIt : zc)
					{
						const uint8 FaceCount = zcIt.Value.Key;
						if (FaceCount == 1)
						{
							SurfaceElements.Add(zcIt.Value.Value);
						}
						else if (FaceCount > 2)
						{
							//UE_LOGF(LogChaosFlesh, Display, "WARNING: Non-manifold tetrahedral mesh detected (face [%d, %d, %d] use count %d).", zcIt->second.second[0], zcIt->second.second[1], zcIt->second.second[2], FaceCount);
							nonManifold++;
						}
					}
				}
			}
			if(nonManifold)
				UE_LOGF(LogChaosFlesh, Display, "WARNING: Encountered %zu non-manifold tetrahedral mesh faces.", nonManifold);
		}
	}

	TArray<int32> CompactSurfaceVertices(const TArray<FVector3f>& Vertices, const TArray<FIntVector3>& SurfaceElements, TArray<FVector3f>& OutVertices, TArray<FIntVector3>& OutSurfaceElements)
	{
		TArray<int32> OldToNewVertexMap;
		OldToNewVertexMap.Init(INDEX_NONE, Vertices.Num());
		TArray<int32> NewToOldVertexMap;
		OutVertices.Reset();
		OutSurfaceElements.Reset();
		for (int32 ElementIdx = 0; ElementIdx < SurfaceElements.Num(); ++ElementIdx)
		{
			const FIntVector3& Element = SurfaceElements[ElementIdx];
			if (!Vertices.IsValidIndex(Element[0]) ||
				!Vertices.IsValidIndex(Element[1]) ||
				!Vertices.IsValidIndex(Element[2]))
			{
				continue;
			}
			for (int32 Corner = 0; Corner < 3; ++Corner)
			{
				const int32 VertexIdx = Element[Corner];
				if (OldToNewVertexMap[VertexIdx] < 0)
				{
					OldToNewVertexMap[VertexIdx] = OutVertices.Add(Vertices[VertexIdx]);
					NewToOldVertexMap.Add(VertexIdx);
				}
			}
			OutSurfaceElements.Add(FIntVector3(
				OldToNewVertexMap[Element[0]],
				OldToNewVertexMap[Element[1]],
				OldToNewVertexMap[Element[2]]));
		}
		return NewToOldVertexMap;
	}
}
