// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosFlesh/TetrahedralCollection.h"
#include "GeometryCollection/Facades/CollectionPositionTargetFacade.h"
#include "GeometryCollection/Facades/CollectionMuscleActivationFacade.h"
#include "GeometryCollection/Facades/CollectionVolumeConstraintFacade.h"

DEFINE_LOG_CATEGORY_STATIC(FTetrahedralCollectionLogging, Log, All);

// Groups
const FName FTetrahedralCollection::TetrahedralGroup = "Tetrahedral";
const FName FTetrahedralCollection::BindingsGroup = "Bindings";

// Attributes
const FName FTetrahedralCollection::TetrahedronAttribute("Tetrahedron");
const FName FTetrahedralCollection::TetrahedronStartAttribute("TetrahedronStart");
const FName FTetrahedralCollection::TetrahedronCountAttribute("TetrahedronCount");
const FName FTetrahedralCollection::IncidentElementsAttribute("IncidentElements");
const FName FTetrahedralCollection::IncidentElementsLocalIndexAttribute("IncidentElementsLocalIndex");
const FName FTetrahedralCollection::GuidAttribute("Guid");

FTetrahedralCollection::FTetrahedralCollection()
	: Super::FGeometryCollection({ FLinearColor(0.6f, 0.6f, 0.6f).ToRGBE() })
{
	Construct();
}


void FTetrahedralCollection::Construct()
{
	FManagedArrayCollection::FConstructionParameters VerticesDependency(FGeometryCollection::VerticesGroup);
	FManagedArrayCollection::FConstructionParameters TetrahedronDependency(FTetrahedralCollection::TetrahedralGroup);

	// Tetrahedron Group
	AddExternalAttribute<FIntVector4>(FTetrahedralCollection::TetrahedronAttribute, FTetrahedralCollection::TetrahedralGroup, Tetrahedron, VerticesDependency);

	// Vertices Group
	AddExternalAttribute<TArray<int32>>(FTetrahedralCollection::IncidentElementsAttribute, FGeometryCollection::VerticesGroup, IncidentElements);
	AddExternalAttribute<TArray<int32>>(FTetrahedralCollection::IncidentElementsLocalIndexAttribute, FGeometryCollection::VerticesGroup, IncidentElementsLocalIndex);

	// Geometry Group
	AddExternalAttribute<int32>(FTetrahedralCollection::TetrahedronStartAttribute, FGeometryCollection::GeometryGroup, TetrahedronStart, TetrahedronDependency);
	AddExternalAttribute<int32>(FTetrahedralCollection::TetrahedronCountAttribute, FGeometryCollection::GeometryGroup, TetrahedronCount);
	AddExternalAttribute<FString>(FTetrahedralCollection::GuidAttribute, FGeometryCollection::GeometryGroup, Guid);
	for (FString& g : Guid) { g = FGuid::NewGuid().ToString(); }
}


void FTetrahedralCollection::SetDefaults(FName Group, uint32 StartSize, uint32 NumElements)
{
	Super::SetDefaults(Group, StartSize, NumElements);

	if (Group == FTetrahedralCollection::TetrahedralGroup)
	{
		for (uint32 Idx = StartSize; Idx < StartSize + NumElements; ++Idx)
		{
			Tetrahedron[Idx] = FIntVector4(INDEX_NONE);
		}
	}
}

FTetrahedralCollection* FTetrahedralCollection::NewTetrahedralCollection(
	const TArray<FVector>& Vertices, 
	const TArray<FIntVector3>& SurfaceElements, 
	const TArray<FIntVector4>& Elements, 
	bool bReverseVertexOrder)
{
	FTetrahedralCollection* Collection = new FTetrahedralCollection();
	FTetrahedralCollection::Init(Collection, Vertices, SurfaceElements, Elements, bReverseVertexOrder);
	for (FString& g : Collection->Guid) { g = FGuid::NewGuid().ToString(); }
	return Collection;
}
void FTetrahedralCollection::Init(
	FTetrahedralCollection* Collection, 
	const TArray<FVector>& Vertices, 
	const TArray<FIntVector3>& SurfaceElements, 
	const TArray<FIntVector4>& Elements, 
	bool bReverseVertexOrder)
{
	if (Collection)
	{
		TArray<float> RawVertexArray;
		RawVertexArray.SetNum(Vertices.Num()*3);
		for(int i=0;i<Vertices.Num();i++)
		{
			RawVertexArray[3 * i + 0] = Vertices[i].X;
			RawVertexArray[3 * i + 1] = Vertices[i].Y;
			RawVertexArray[3 * i + 2] = Vertices[i].Z;
		}
		TArray<int32> RawIndicesArray;
		RawIndicesArray.SetNum(SurfaceElements.Num()*3);
		for (int i = 0; i < SurfaceElements.Num(); i++)
		{
			RawIndicesArray[3 * i + 0] = SurfaceElements[i].X;
			RawIndicesArray[3 * i + 1] = SurfaceElements[i].Y;
			RawIndicesArray[3 * i + 2] = SurfaceElements[i].Z;
		}

		Super::Init(Collection, RawVertexArray, RawIndicesArray, bReverseVertexOrder);

		if (Collection->NumElements(FTetrahedralCollection::GeometryGroup))
		{
			Collection->TetrahedronStart[0] = Collection->Tetrahedron.Num();
			Collection->TetrahedronCount[0] = Elements.Num();
			Collection->AddElements(Elements.Num(), FTetrahedralCollection::TetrahedralGroup);
			for (int i = 0; i < Elements.Num(); i++)
			{
				Collection->Tetrahedron[i] = Elements[i];
			}
		}

		// Init aux structures that depend on topology.
		Collection->InitIncidentElements();
	}
}

void FTetrahedralCollection::UpdateBoundingBox()
{
	if (BoundingBox.Num())
	{
		// Initialize BoundingBox
		for (int32 Idx = 0; Idx < BoundingBox.Num(); ++Idx)
		{
			BoundingBox[Idx].Init();
		}

		// Compute BoundingBox
		for (int32 Idx = 0; Idx < Vertex.Num(); ++Idx)
		{
			const int32 TransformIndexValue = BoneMap[Idx];
			if (TransformIndexValue != INDEX_NONE)
			{
				const int32 GeometryIndex = TransformToGeometryIndex[TransformIndexValue];
				if (GeometryIndex != INDEX_NONE)
				{
					BoundingBox[GeometryIndex] += FVector(Vertex[Idx]);
				}
			}
		}
	}
}

void FTetrahedralCollection::Append(const FManagedArrayCollection& InCollection)
{
	Super::Append(InCollection);
	// InCollection data is appended to the front
	const int32 Offset = InCollection.NumElements(TetrahedralGroup);
	const int32 OtherSize = InCollection.NumElements(VerticesGroup);
	const int32 Size = NumElements(VerticesGroup);
	// IncidentElements does not have TetrahedralGroup dependency, update manually
	for (int32 Idx = OtherSize; Idx < Size; Idx++)
	{
		for (int32& TetIdx: IncidentElements[Idx])
		{
			// Offset by the number of tets of the second collection
			TetIdx += Offset;
		}
	}
}

void FTetrahedralCollection::AppendCollection(const FTetrahedralCollection& InCollection)
{
	Append(InCollection);
}

int32 FTetrahedralCollection::AppendGeometry(
	const FTetrahedralCollection& Other, 
	int32 MaterialIDOffset, 
	bool ReindexAllMaterials, 
	const FTransform& TransformRoot)
{
	const int32 VerticesIndex = NumElements(FGeometryCollection::VerticesGroup);
	const int32 NumGeometry = NumElements(FGeometryCollection::GeometryGroup);

	const int32 ID = 
		Super::AppendGeometry(
			*static_cast<const FGeometryCollection*>(&Other), 
			MaterialIDOffset, 
			ReindexAllMaterials, 
			TransformRoot);

	// --- TETRAHEDRAL GROUP --

	int NumTets = NumElements(FTetrahedralCollection::TetrahedralGroup);
	int NumOtherTets = Other.Tetrahedron.Num();
	int TetsIndex = AddElements(NumOtherTets, FTetrahedralCollection::TetrahedralGroup);
	for (int32 Idx = 0; Idx < NumOtherTets; Idx++)
	{
		Tetrahedron[TetsIndex + Idx] = FIntVector4(VerticesIndex, VerticesIndex, VerticesIndex, VerticesIndex) + Other.Tetrahedron[Idx];
	}

	// --- GEOMETRY GROUP --

	check(TetrahedronStart.Num() == NumGeometry + Other.TetrahedronStart.Num());
	for (int32 Idx = 0; Idx < Other.TetrahedronStart.Num(); Idx++)
	{
		TetrahedronStart[NumGeometry + Idx] = NumTets + Other.TetrahedronStart[Idx];
		TetrahedronCount[NumGeometry + Idx] = Other.TetrahedronCount[Idx];
	}

	check(Guid.Num() == NumGeometry + Other.Guid.Num());
	for (int32 Idx = 0; Idx < Other.Guid.Num(); Idx++)
	{
		Guid[NumGeometry + Idx] = Other.Guid[Idx];
	}

	// --- VERTICES GROUP --
	int32 NumVertices = VerticesIndex;
	int32 OtherNumVertices = Other.IncidentElements.Num();
	for (int32 Idx = 0; Idx < OtherNumVertices; Idx++)
	{
		const TArray<int32>& OtherIncidentElements = Other.IncidentElements[Idx];
		TArray<int32>& ThisIncidentElements = IncidentElements[VerticesIndex + Idx];
		ThisIncidentElements.SetNumUninitialized(OtherIncidentElements.Num());
		for (int32 i = 0; i < ThisIncidentElements.Num(); i++)
		{
			// Offset by the number of tets we started with
			ThisIncidentElements[i] = OtherIncidentElements[i] + NumTets;
		}

		const TArray<int32>& OtherIncidentElementsLocalIndex = Other.IncidentElementsLocalIndex[Idx];
		TArray<int32>& ThisIncidentElementsLocalIndex = IncidentElementsLocalIndex[VerticesIndex + Idx];
		// No offset necessary, just copy
		ThisIncidentElementsLocalIndex = OtherIncidentElementsLocalIndex;
	}

	return ID;
}

void FTetrahedralCollection::InitIncidentElements(const int32 GeometryIndex)
{
	const TArray<FIntVector4>& Tets = Tetrahedron.GetConstArray();

	// Clear entries for all the nodes we're going to touch.
	int32 NodeStart = 0;
	int32 NodeCount = INDEX_NONE;
	if(GeometryIndex < 0 || GeometryIndex >= VertexStart.Num())
	{
		IncidentElements.Fill(TArray<int32>());
		IncidentElementsLocalIndex.Fill(TArray<int32>());
	}
	else
	{
		NodeStart = VertexStart[GeometryIndex];
		NodeCount = VertexCount[GeometryIndex];
		for (int32 Idx = NodeStart; Idx < NodeStart + NodeCount; Idx++)
		{
			IncidentElements[Idx].SetNum(0);
			IncidentElementsLocalIndex[Idx].SetNum(0);
		}
	}

	// Add each tet index to each of its nodes.
	for (int32 TetIdx = 0; TetIdx < Tets.Num(); TetIdx++)
	{
		for (int32 NodeIdx = 0; NodeIdx < 4; NodeIdx++)
		{
			const int Node = Tets[TetIdx][NodeIdx];
			if (NodeCount == INDEX_NONE ||
				(Node >= NodeStart && Node - NodeStart > NodeCount))
			{
				IncidentElements[Node].Add(TetIdx);
				IncidentElementsLocalIndex[Node].Add(NodeIdx);
			}
		}
	}
}

void FTetrahedralCollection::ReorderElements(FName Group, const TArray<int32>& NewOrder)
{
	if (Group == FTetrahedralCollection::TetrahedralGroup)
	{
		ReorderTetrahedralElements(NewOrder);
	}
	else
	{
		Super::ReorderElements(Group, NewOrder);
	}
}

void FTetrahedralCollection::ReorderTetrahedralElements(const TArray<int32>& NewOrder)
{
	const int32 NumTets = NumElements(TetrahedralGroup);
	check(NumTets == NewOrder.Num());

	//Compute new order for vertices group and faces group
	TArray<int32> NewTetOrder;
	NewTetOrder.Reserve(NumElements(TetrahedralGroup));
	int32 TotalTetOffset = 0;

	for (int32 OldTetIdx = 0; OldTetIdx < NumTets; ++OldTetIdx)
	{
		const int32 NewTetIdx = NewOrder[OldTetIdx];

		// tets
		const int32 TetStartIdx = TetrahedronStart[NewTetIdx];
		const int32 TetCount = TetrahedronCount[NewTetIdx];
		for (int32 Idx = TetStartIdx; Idx < TetStartIdx + TetCount; ++Idx)
		{
			NewTetOrder.Add(Idx);
		}
		TotalTetOffset += TetCount;
	}

	Super::ReorderElements(TetrahedralGroup, NewTetOrder);
}

void FTetrahedralCollection::RemoveVertices(const TArray<int32>& SortedVertexIndices)
{
	FProcessingParameters Param;
	Param.bReindexDependentAttibutes = true;
	Param.bDoValidation = false;
	Super::RemoveElements(FGeometryCollection::VerticesGroup, SortedVertexIndices, Param);
	const int32 NumVertices = NumElements(FGeometryCollection::VerticesGroup);
	const int32 NumFaces = NumElements(FGeometryCollection::FacesGroup);
	const int32 NumGeometry = NumElements(FGeometryCollection::GeometryGroup);
	const int32 NumTets = NumElements(FTetrahedralCollection::TetrahedralGroup);
	const int32 NumTransforms = NumElements(FTetrahedralCollection::TransformGroup);

	// --- Tetrahedral Group --
	TArray<int32> DeletedTetrahedralElements;
	TArray<int32> TetrahedralElementsRemap;
	TetrahedralElementsRemap.Init(INDEX_NONE, NumTets);
	for (int32 Idx = 0; Idx < NumTets; ++Idx)
	{
		bool bKeepTet = true;
		for (int32 LocalIdx = 0; LocalIdx < 4; ++LocalIdx)
		{
			if (Tetrahedron[Idx][LocalIdx] < 0)
			{
				bKeepTet = false;
				DeletedTetrahedralElements.Add(Idx);
				break;
			}
		}
		if (bKeepTet)
		{
			TetrahedralElementsRemap[Idx] = Idx - DeletedTetrahedralElements.Num();
		}
	}
	Super::RemoveElements(TetrahedralGroup, DeletedTetrahedralElements, Param);

	// --- Faces Group --
	TArray<int32> DeletedFaceElements;
	for (int32 Idx = 0; Idx < NumFaces; ++Idx)
	{
		for (int32 LocalIdx = 0; LocalIdx < 3; ++LocalIdx)
		{
			if (Indices[Idx][LocalIdx] < 0)
			{
				DeletedFaceElements.Add(Idx);
				break;
			}
		}
	}
	Super::RemoveElements(FGeometryCollection::FacesGroup, DeletedFaceElements, Param);

	// --- Transform Group --
	TArray<int32> DeletedTransformElements;
	for (int32 Idx = 0; Idx < NumGeometry; ++Idx)
	{
		if (VertexCount[Idx] <= 0)
		{
			DeletedTransformElements.Add(TransformIndex[Idx]);
		}
	}
	Param.bDoValidation = true; //Validate after the last RemoveElement
	Super::RemoveElements(FGeometryCollection::TransformGroup, DeletedTransformElements, Param);

	// --- Incident Elements (Tetrahedral Dependency) --
	InitIncidentElements();

	// --- Position Targets (Vertex group dependency)--
	GeometryCollection::Facades::FPositionTargetFacade PositionTargetFacade(*this);
	PositionTargetFacade.RemoveInvalidPositionTarget();

	// --- Air Tetrahedral Constraints (vertex group dependency)--
	GeometryCollection::Facades::FVolumeConstraintFacade VolumeConstraintFacade(*this);
	VolumeConstraintFacade.RemoveInvalidVolumeConstraint();

	// --- Muscle Activation (Geometry group dependency)--
	GeometryCollection::Facades::FMuscleActivationFacade MuscleActivationFacade(*this);
	MuscleActivationFacade.RemoveInvalidMuscles();
}