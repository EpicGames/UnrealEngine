// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GeometryCollection->cpp: FGeometryCollection methods.
=============================================================================*/

#include "ChaosFlesh/TetrahedralExampleGeometry.h"

#include "ChaosFlesh/FleshCollection.h"
#include "GeometryCollection/TransformCollection.h"


namespace ChaosFlesh
{
	namespace ExampleGeometry
	{
		FVector TetrahedralCentroid(TArray<FVector> V)
		{
			return FVector(
				(V[0].X + V[1].X + V[2].X + V[3].X) / 4.0,
				(V[0].Y + V[1].Y + V[2].Y + V[3].Y) / 4.0,
				(V[0].Z + V[1].Z + V[2].Z + V[3].Z) / 4.0
			);
		}

		TUniquePtr<FTetrahedralCollection> UnitTetrahedron::Create()
		{
			float unitProj = FMath::Sin(PI / 3.0); // projection of unit vector at 60 deg
			float faceAngle = FMath::Acos(1. / 3); // angle between planes of a tetrahedron. 
			TArray<FVector> Vertices = {
				FVector(-0.5, 0.0, 0.0),
				FVector( 0.5, 0.0, 0.0),
				FVector( 0.0, unitProj, 0.0),
				FVector( 0.0, unitProj * FMath::Cos(faceAngle), unitProj * FMath::Sin(faceAngle)),
			};
			FVector Center = TetrahedralCentroid(Vertices);
			for (FVector & Vert : Vertices)
			{
				Vert -= Center;
			}

			TArray<FIntVector> Surface = {
				FIntVector(0,1,2),
				FIntVector(0,2,3),
				FIntVector(3,2,1),
				FIntVector(0,3,1)
			};
			TArray<FIntVector4> Tetrahedron = {
					FIntVector4(0,1,2,3)
			};
			return 	TUniquePtr<FFleshCollection>(FFleshCollection::NewFleshCollection(Vertices, Surface, Tetrahedron, false));
			;
		}
	}
}
