// Copyright Epic Games, Inc. All Rights Reserved.

#include "Meshing/ChaosFleshRadialMeshing.h"

using FReal = FVector::FReal;
void RadialTetMesh(const FReal InnerRadius, const FReal OuterRadius, const FReal Height, const int32 RadialSample, const int32 AngularSample, const int32 VerticalSample, const FReal BulgeDistance, TArray<FIntVector4>& TetElements, TArray<FVector>& TetVertices)
{
	TArray<FVector> HexVertices;
	TArray<int32> HexElements;
	RadialHexMesh(InnerRadius, OuterRadius, Height, RadialSample, AngularSample, VerticalSample, BulgeDistance, HexElements, HexVertices);
	RegularHexMesh2TetMesh(HexVertices, HexElements, TetVertices, TetElements);
}

void RadialHexMesh(const FReal InnerRadius, const FReal OuterRadius, const FReal Height, const int32 RadialSample, const int32 AngularSample, const int32 VerticalSample, const FReal BulgeDistance, TArray<int32>& HexElements, TArray<FVector>& HexVertices) 
{
	HexElements.SetNum(8 * (VerticalSample - 1) * (RadialSample - 1) * AngularSample);
	HexVertices.SetNum(VerticalSample * RadialSample * AngularSample);
	FReal dr = (OuterRadius - InnerRadius) / FReal(RadialSample - 1);
	FReal dtheta = FReal(2) * PI / FReal(AngularSample);
	FReal dz = Height / FReal(VerticalSample - 1);
	FReal l = (OuterRadius - InnerRadius) / (FReal)2.;
	FReal Mag = (l * l + BulgeDistance * BulgeDistance) / ((FReal)2. * BulgeDistance);
	for (int32 i = 0; i < VerticalSample; i++)
	{
		//start at top, work way down
		FReal ZVal = Height * FReal(0.5) - i * dz;
		for (int32 j = 0; j < AngularSample; j++)
		{
			FReal XVal = cos(j * dtheta);
			FReal YVal = sin(j * dtheta);
			for (int32 k = 0; k < RadialSample; k++)
			{
				FReal RVal = InnerRadius + k * dr;

				if (BulgeDistance > DOUBLE_SMALL_NUMBER)
				{
					if (i == 0)
					{
						FReal ZElevation = Height * FReal(0.5) - Mag * sin(PI * (Mag - l) / (2 * Mag));
						ZVal = Mag * sin(PI * (Mag - l + (FReal)k * dr) / (2 * Mag)) + ZElevation;
					}
					if (i == VerticalSample - 1)
					{
						FReal ZElevation = -Height * FReal(0.5) - Mag * sin(PI + PI * (Mag - l) / (FReal(2) * Mag));
						ZVal = Mag * sin(PI + PI * (Mag - l + (FReal)k * dr) / (FReal(2) * Mag)) + ZElevation;
					}
				}

				HexVertices[AngularSample * RadialSample * i + RadialSample * j + k] = { RVal * XVal,RVal * YVal,ZVal };

			}
		}
	}
	//CCW bottom then top local ordering of hex nodes
	for (int32 i = 0; i < (VerticalSample - 1); i++)
	{
		for (int32 j = 0; j < (AngularSample - 1); j++)
		{
			for (int32 k = 0; k < (RadialSample - 1); k++)
			{
				//lower level
				HexElements[8 * (i * (RadialSample - 1) * AngularSample + j * (RadialSample - 1) + k)] = RadialSample * AngularSample * (i + 1) + RadialSample * j + k;
				HexElements[8 * (i * (RadialSample - 1) * AngularSample + j * (RadialSample - 1) + k) + 1] = RadialSample * AngularSample * (i + 1) + RadialSample * j + k + 1;
				HexElements[8 * (i * (RadialSample - 1) * AngularSample + j * (RadialSample - 1) + k) + 2] = RadialSample * AngularSample * (i + 1) + RadialSample * (j + 1) + k + 1;
				HexElements[8 * (i * (RadialSample - 1) * AngularSample + j * (RadialSample - 1) + k) + 3] = RadialSample * AngularSample * (i + 1) + RadialSample * (j + 1) + k;
				//upper level
				HexElements[8 * (i * (RadialSample - 1) * AngularSample + j * (RadialSample - 1) + k) + 4] = RadialSample * AngularSample * i + RadialSample * j + k;
				HexElements[8 * (i * (RadialSample - 1) * AngularSample + j * (RadialSample - 1) + k) + 5] = RadialSample * AngularSample * i + RadialSample * j + k + 1;
				HexElements[8 * (i * (RadialSample - 1) * AngularSample + j * (RadialSample - 1) + k) + 6] = RadialSample * AngularSample * i + RadialSample * (j + 1) + k + 1;
				HexElements[8 * (i * (RadialSample - 1) * AngularSample + j * (RadialSample - 1) + k) + 7] = RadialSample * AngularSample * i + RadialSample * (j + 1) + k;
			}
		}
		//Gluing edges, wrap-around
		for (int32 k = 0; k < (RadialSample - 1); k++)
		{
			//lower level
			HexElements[8 * (i * (RadialSample - 1) * AngularSample + (AngularSample - 1) * (RadialSample - 1) + k)] = RadialSample * AngularSample * (i + 1) + RadialSample * (AngularSample - 1) + k;
			HexElements[8 * (i * (RadialSample - 1) * AngularSample + (AngularSample - 1) * (RadialSample - 1) + k) + 1] = RadialSample * AngularSample * (i + 1) + RadialSample * (AngularSample - 1) + k + 1;
			HexElements[8 * (i * (RadialSample - 1) * AngularSample + (AngularSample - 1) * (RadialSample - 1) + k) + 2] = RadialSample * AngularSample * (i + 1) + k + 1;
			HexElements[8 * (i * (RadialSample - 1) * AngularSample + (AngularSample - 1) * (RadialSample - 1) + k) + 3] = RadialSample * AngularSample * (i + 1) + k;
			//upper level
			HexElements[8 * (i * (RadialSample - 1) * AngularSample + (AngularSample - 1) * (RadialSample - 1) + k) + 4] = RadialSample * AngularSample * i + RadialSample * (AngularSample - 1) + k;
			HexElements[8 * (i * (RadialSample - 1) * AngularSample + (AngularSample - 1) * (RadialSample - 1) + k) + 5] = RadialSample * AngularSample * i + RadialSample * (AngularSample - 1) + k + 1;
			HexElements[8 * (i * (RadialSample - 1) * AngularSample + (AngularSample - 1) * (RadialSample - 1) + k) + 6] = RadialSample * AngularSample * i + k + 1;
			HexElements[8 * (i * (RadialSample - 1) * AngularSample + (AngularSample - 1) * (RadialSample - 1) + k) + 7] = RadialSample * AngularSample * i + k;
		}
	}
}

void RegularHexMesh2TetMesh(const TArray<FVector>& HexVertices, const TArray<int32>& HexElements, TArray<FVector>& TetVertices, TArray<FIntVector4>& TetElements)
{
	if (ensureMsgf(HexElements.Num() % 8 == 0, TEXT("The hex mesh is not divisible by 8."))) 
	{
		TArray<FIntVector4> FaceToNodes = { {int32(0), int32(1), int32(2), int32(3)},{int32(0), int32(1), int32(5), int32(4)},{int32(1), int32(2), int32(6), int32(5)},{int32(3), int32(2), int32(6), int32(7)},{int32(0), int32(3), int32(7), int32(4)},{int32(4), int32(5), int32(6), int32(7)} };
		TArray<FIntVector2> CommonFaces;
		int32 Offset = HexVertices.Num();
		TetVertices = HexVertices;
		TetElements.SetNum(0);
		for (int32 i = 0; i < int32(HexElements.Num() / 8); ++i)
		{
			FVector Center = FVector(0);
			for (int32 ie = 0; ie < 8; ie++)
			{
				for (int32 c = 0; c < 3; c++)
				{
					Center[c] += HexVertices[HexElements[8 * i + ie]][c] / FReal(8);
				}
			}
			TetVertices.Emplace(Center);
		}
		/*});*/
		int32 q0, f0, q1, h0, h1, Nexti;

		ComputeHexMeshFaces(HexElements, CommonFaces);

		for (int32 f = 0; f < CommonFaces.Num(); ++f)
		{
			q0 = int32(CommonFaces[f][0]) / 6;
			f0 = int32(CommonFaces[f][0]) % 6;
			h0 = Offset + q0;
			// The same face is shared by two hexs
			if (CommonFaces[f][1] > int32(-1))
			{
				q1 = int32(CommonFaces[f][1]) / 6;
				h1 = Offset + q1;
			}
			// The face is on the boundary
			else
			{
				h1 = TetVertices.Num();
				FVector FaceCenter = FVector(0);
				for (int32 ie = 0; ie < 4; ie++)
			{
					for (int32 c = 0; c < 3; c++)
					{
						FaceCenter[c] += HexVertices[HexElements[8 * q0 + FaceToNodes[f0][ie]]][c] / FReal(4);
					}
				}
				TetVertices.Emplace(FaceCenter);
			}
			for (int32 i = 0; i < 4; i++)
			{
				Nexti = (i + 1) % 4;
				FIntVector4 NewElement;
				NewElement[0] = h0;
				NewElement[1] = h1;
				if (f0 == 1 || f0 == 2 || f0 == 5)
				{
					NewElement[2] = HexElements[8 * q0 + FaceToNodes[f0][i]];
					NewElement[3] = HexElements[8 * q0 + FaceToNodes[f0][Nexti]];
				}
				else
				{
					NewElement[2] = HexElements[8 * q0 + FaceToNodes[f0][Nexti]];
					NewElement[3] = HexElements[8 * q0 + FaceToNodes[f0][i]];
				}
				TetElements.Emplace(NewElement);
			}
		}
	}
}

void ComputeHexMeshFaces(const TArray<int32>& HexElements, TArray<FIntVector2>& CommonFaces)
{
	auto FaceGreaterThan = [](FIntVector4& i1, FIntVector4& i2)
	{
		bool GreaterThan = false;
		if (i1[0] > i2[0])
			GreaterThan = true;
		else if (i1[0] == i2[0] && i1[1] > i2[1])
			GreaterThan = true;
		else if (i1[0] == i2[0] && i1[1] == i2[1] && i1[2] > i2[2])
			GreaterThan = true;
		else if (i1[0] == i2[0] && i1[1] == i2[1] && i1[2] == i2[2] && i1[3] > i2[3])
			GreaterThan = true;
		return GreaterThan;
	};
	auto FaceEqual = [](FIntVector4& i1, FIntVector4& i2)
	{
		return (i1[0] == i2[0] && i1[1] == i2[1] && i1[2] == i2[2] && i1[3] == i2[3]);
	};
	auto GreaterThan = [&FaceGreaterThan](const TArray<int32>& Faces, int32 e1, int32 e2)
	{
		FIntVector4 Face1 = { Faces[4 * e1 + 0],Faces[4 * e1 + 1],Faces[4 * e1 + 2],Faces[4 * e1 + 3] };
		FIntVector4 Face2 = { Faces[4 * e2 + 0],Faces[4 * e2 + 1],Faces[4 * e2 + 2],Faces[4 * e2 + 3] };
		return FaceGreaterThan(Face1, Face2);
	};

	auto Equal = [&FaceEqual](const TArray<int32>& Faces, const TArray<int32>& AllFaces, int32 i1, int32 i2)
	{
		int32 e1 = AllFaces[i1];
		int32 e2 = AllFaces[i2];
		FIntVector4 Face1 = { Faces[4 * e1 + 0],Faces[4 * e1 + 1],Faces[4 * e1 + 2],Faces[4 * e1 + 3] };
		FIntVector4 Face2 = { Faces[4 * e2 + 0],Faces[4 * e2 + 1],Faces[4 * e2 + 2],Faces[4 * e2 + 3] };
		return FaceEqual(Face1, Face2);
	};
	ComputeMeshFaces(HexElements, GreaterThan, Equal, CommonFaces, GenerateHexFaceMesh, 4);
}

template <typename Func1, typename Func2, typename Func3>
void ComputeMeshFaces(const TArray<int32>& mesh, Func1 GreaterThan, Func2 Equal, TArray<FIntVector2>& CommonFaces, Func3 GenerateFace, int32 PointsPerFace)
{
	TArray<int32> Faces;
	GenerateFace(mesh, Faces);
	int32 FaceNumber = Faces.Num() / PointsPerFace;
	TArray<int32> AllFaces;
	AllFaces.SetNum(FaceNumber);
	TArray<int32> Ranges;
	Ranges.SetNum(FaceNumber + 1);
	for (int32 f = 0; f < int32(FaceNumber); f++)
	{
		AllFaces[f] = f;
		Ranges[f] = f;
	}

	AllFaces.Sort([&GreaterThan, &Faces](int32 FaceA, int32 FaceB) { return GreaterThan(Faces, FaceA, FaceB); });
	TArray<int32> UniqueRanges({ 0 });

	for (int32 i = 0; i < Ranges.Num() - 2; ++i)
	{
		if (!Equal(Faces, AllFaces, Ranges[i], Ranges[i + 1]))
		{
			UniqueRanges.Emplace(i + 1);
		}
	}

	int32 TotalFaces = UniqueRanges.Num();
	UniqueRanges.SetNum(TotalFaces + 1);
	UniqueRanges[TotalFaces] = int32(FaceNumber);


	CommonFaces.SetNum(TotalFaces);
	for (int32 f = 0; f < int32(CommonFaces.Num()); ++f)
	{
		CommonFaces[f][0] = AllFaces[UniqueRanges[f]];
		if (UniqueRanges[f + 1] - UniqueRanges[f] == 1)
			CommonFaces[f][1] = -1;
		else
		{
			CommonFaces[f][1] = AllFaces[UniqueRanges[f] + 1];
		}
	}
}

void GenerateHexFaceMesh(const TArray<int32>& HexElements, TArray<int32>& Faces)
{
	if (ensureMsgf(HexElements.Num() % 8 == 0, TEXT("The hex mesh is not divisible by 8.")))
	{
		TArray<int32> FaceTemp = { int32(0), int32(1), int32(2), int32(3),int32(0), int32(1), int32(5), int32(4),int32(1), int32(2), int32(6), int32(5),int32(3), int32(2), int32(6), int32(7),int32(0), int32(3), int32(7), int32(4),int32(4), int32(5), int32(6), int32(7) };
		Faces.SetNum(HexElements.Num() / 8 * 24);
		for (int32 i = 0; i < HexElements.Num() / 8; i++)
		{
			for (int32 j = 0; j < int32(24); j++)
			{
				Faces[24 * i + j] = HexElements[8 * i + FaceTemp[j]];
			}
		}
	}
}