// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "ChaosFlesh/FleshCollection.h"
#include "GeometryCollection/ManagedArray.h"
#include "GeometryCollection/ManagedArrayAccessor.h"

namespace Chaos {

	class CHAOSFLESH_API FFleshCollectionFacade
	{
		const FFleshCollection& ConstCollection;
		FFleshCollection* Collection = nullptr;
	public:

		FFleshCollectionFacade(FFleshCollection& InCollection);

		FFleshCollectionFacade(const FFleshCollection& InCollection);
		
		/*Are all the public attributes avaiable.*/
		bool IsValid() const;
	
		/*Has tetrahedral attributes {Tetrahderon, Vertices}.*/
		bool IsTetrahedronValid() const;

		/*Has hierarchy attributes {BoneName, Transform, Parent, Child}.*/
		bool IsHierarchyValid() const;

		/*Has geometry attributes {TransformToGeometryIndex VertexStart VertexCount FaceStart FaceCount}.*/
		bool IsGeometryValid() const;

		/* Num of transform elements in the collection */
		int NumTransforms() const;

		/* Num of vertex elements in the collection */
		int NumVertices() const;

		/* Num of face elements in the collection */
		int NumFaces() const;

		/* Num of geometry elements in the collection */
		int NumGeometry() const;

		/* Add geometry to the collection*/
		int AppendGeometry(const FFleshCollection& NewGeomerty);

		/* Global Matrices of the collection */
		void GlobalMatrices(TArray<FTransform>& ComponentTransform);

		/* Single Global Matrix of index in the collection */
		FTransform3f GlobalMatrix3f(int32 InIndex);

		/*View a Attribute on the collection*/
		template<class T>
		const TManagedArray<T>* FindAttribute(FString AttributeName, FString Group) const;

		/*Edit a Attribute on the collection*/
		template<class T>
		TManagedArray<T>* ModifyAttribute(FString AttributeName, FString Group) const;

		/* All the vertices mapped into component space. */
		void ComponentSpaceVertices(TArray<FVector3f>& OutComponentSpaceVertices) const;

		/* Range of vertices mapped into component space*/
		void ComponentSpaceVertices(TArray<FVector3f>& OutComponentSpaceVertices, int32 Start, int32 Count) const;

		/*Public Attributes*/
		TManagedArrayAccessor<FString> BoneName;
		TManagedArrayAccessor<FTransform3f> Transform;
		TManagedArrayAccessor<int32> TransformToGeometryIndex;
		TManagedArrayAccessor<int32> Parent;
		TManagedArrayAccessor< TSet<int32> > Child;
		TManagedArrayAccessor<int32> BoneMap;
		TManagedArrayAccessor<FVector3f> Vertex;
		TManagedArrayAccessor<FIntVector3> Indices;
		TManagedArrayAccessor<FIntVector4> Tetrahedron;
		TManagedArrayAccessor<int32> GeometryToTransformIndex;
		TManagedArrayAccessor<int32> VertexStart;
		TManagedArrayAccessor<int32> VertexCount;
		TManagedArrayAccessor<int32> FaceStart;
		TManagedArrayAccessor<int32> FaceCount;
	};

}
