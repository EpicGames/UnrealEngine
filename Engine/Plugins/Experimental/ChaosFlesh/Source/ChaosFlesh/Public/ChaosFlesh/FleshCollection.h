// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ChaosFlesh/TetrahedralCollection.h"
#include "GeometryCollection/Facades/CollectionSelectionFacade.h"

namespace Chaos
{
	class FChaosArchive;
}

/**
* FFleshCollection (FTetrehidralCollection)
*/
class CHAOSFLESH_API FFleshCollection : public FTetrahedralCollection
{
public:
	typedef FTetrahedralCollection Super;
	typedef GeometryCollection::Facades::FSelectionFacade::FSelectionKey FBindingKey;

	FFleshCollection();
	FFleshCollection(FFleshCollection&) = delete;
	FFleshCollection& operator=(const FFleshCollection&) = delete;
	FFleshCollection(FFleshCollection&&) = default;
	FFleshCollection& operator=(FFleshCollection&&) = default;

	/**
	 * Create a GeometryCollection from Vertex and Indices arrays
	 */
	static FFleshCollection* NewFleshCollection(const FTetrahedralCollection& Base);
	static FFleshCollection* NewFleshCollection(const TArray<FVector>& Vertices, const TArray<FIntVector3>& SurfaceElements, const TArray<FIntVector4>& Elements, bool bReverseVertexOrder = true);
	static FFleshCollection* NewFleshCollection(const TArray<FVector>& Vertices, const TArray<FIntVector4>& Elements, const bool bReverseVertexOrder = true, const bool KeepInteriorFaces = false, const bool InvertFaces = false);
	static void Init(FFleshCollection* Collection, const TArray<FVector>& Vertices, const TArray<FIntVector3>& SurfaceElements, const TArray<FIntVector4>& Elements, bool bReverseVertexOrder = true);


	/*
	* Mass Attribute
	*   FManagedArray<float> Mass = this->FindAttribute<float>("Mass",FGeometryCollection::VerticesGroup);
	*/
	static const FName MassAttribute;
	TManagedArray<float> Mass;

protected:
	void Construct();

};

inline Chaos::FChaosArchive& operator<<(Chaos::FChaosArchive& Ar, FFleshCollection& Value)
{
	Value.Serialize(Ar);
	return Ar;
}
