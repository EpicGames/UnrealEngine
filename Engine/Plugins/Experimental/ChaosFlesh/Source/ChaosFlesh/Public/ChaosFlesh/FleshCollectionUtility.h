// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosFlesh/FleshCollection.h"
#include "CoreMinimal.h"
#include "Templates/UniquePtr.h"
#include "Misc/AssertionMacros.h"

namespace ChaosFlesh 
{
	void GetTetFaces(const FIntVector4& Tet, FIntVector3& Face1, FIntVector3& Face2, FIntVector3& Face3, FIntVector3& Face4, const bool invert = false);
	void GetSurfaceElements(const TArray<FIntVector4>& Tets, TArray<FIntVector3>& SurfaceElements, const bool KeepInteriorFaces, const bool InvertFaces = false);
	/** Delete vertices that are not used in SurfaceElements and returns a new-to-old vertex map. */
	CHAOSFLESH_API TArray<int32> CompactSurfaceVertices(const TArray<FVector3f>& Vertices, const TArray<FIntVector3>& SurfaceElements, TArray<FVector3f>& OutVertices, TArray<FIntVector3>& OutSurfaceElements);
}
