// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Misc/Crc.h"

namespace Chaos
{
	class FChaosArchive;
}


/**
* FTetrahedralCollection (FGeometryCollection)
*/
class FTetrahedralCollection : public FGeometryCollection
{
	typedef FGeometryCollection Super;

public:
	CHAOSFLESH_API FTetrahedralCollection();
	FTetrahedralCollection(FTetrahedralCollection &) = delete;
	FTetrahedralCollection& operator=(const FTetrahedralCollection &) = delete;
	FTetrahedralCollection(FTetrahedralCollection &&) = default;
	FTetrahedralCollection& operator=(FTetrahedralCollection &&) = default;

	/**
	 * Create a GeometryCollection from Vertex and Indices arrays
	 */
	CHAOSFLESH_API static FTetrahedralCollection* NewTetrahedralCollection(const TArray<FVector>& Vertices, const TArray<FIntVector3>& SurfaceElements, const TArray<FIntVector4>& Elements,bool bReverseVertexOrder = true);
	CHAOSFLESH_API static void Init(FTetrahedralCollection* Collection, const TArray<FVector>& Vertices, const TArray<FIntVector3>& SurfaceElements, const TArray<FIntVector4>& Elements, bool bReverseVertexOrder = true);

	CHAOSFLESH_API void UpdateBoundingBox();

	CHAOSFLESH_API void AppendCollection(const FTetrahedralCollection& InCollection);

	CHAOSFLESH_API int32 AppendGeometry(const FTetrahedralCollection& GeometryCollection, int32 MaterialIDOffset = 0, bool ReindexAllMaterials = true, const FTransform& TransformRoot = FTransform::Identity);

	/**
	* Build \c IncidentElements and \c IncidentElementsLocalIndex.
	* \p GeometryIndex - ID of the geometry entry to restrict operation to; -1 does all.
	*/
	CHAOSFLESH_API void InitIncidentElements(const int32 GeometryIndex=-1);

	/*
	*  SetDefaults for new entries on this collection. 
	*/
	CHAOSFLESH_API virtual void SetDefaults(FName Group, uint32 StartSize, uint32 NumElements) override;

	/**
	* Reorders elements in a group. NewOrder must be the same length as the group.
	*/
	CHAOSFLESH_API void ReorderElements(FName Group, const TArray<int32>& NewOrder) override;

	CHAOSFLESH_API void ReorderTetrahedralElements(const TArray<int32>& NewOrder);

	/**
	* Remove selected vertices.
	*/
	CHAOSFLESH_API void RemoveVertices(const TArray<int32>& SortedVertexIndices);

	/*
	*  Attribute Groups
	*/
	CHAOSFLESH_API static const FName TetrahedralGroup;
	CHAOSFLESH_API static const FName BindingsGroup;

	/*
	*  Tetrahedron Attribute
	*  TManagedArray<FIntVector4> Tetrahedron = this->FindAttribute<FIntVector4>(FTetrahedralCollection::TetrahedronAttribute,FTetrahedralCollection::TetrahedralGroup);
	*/
	CHAOSFLESH_API static const FName TetrahedronAttribute;
	TManagedArray<FIntVector4> Tetrahedron;

	CHAOSFLESH_API static const FName TetrahedronStartAttribute;
	TManagedArray<int32> TetrahedronStart;
	CHAOSFLESH_API static const FName TetrahedronCountAttribute;
	TManagedArray<int32> TetrahedronCount;

	/**
	* Incident Elements Attribute
	* For each vertex, a list of tetrahedra that includes that vertex.
	*/
	CHAOSFLESH_API static const FName IncidentElementsAttribute;
	TManagedArray<TArray<int32>> IncidentElements;

	/**
	* Incident Elements Attribute
	* For each incident element, the vertex's index in the tetrahedron.
	*/
	CHAOSFLESH_API static const FName IncidentElementsLocalIndexAttribute;
	TManagedArray<TArray<int32>> IncidentElementsLocalIndex;

	/*
	* Guid Attribute
	*/
	CHAOSFLESH_API static const FName GuidAttribute;
	TManagedArray<FString> Guid;

protected:

	CHAOSFLESH_API void Construct();

	CHAOSFLESH_API virtual void Append(const FManagedArrayCollection& InCollection) override;
};

inline Chaos::FChaosArchive& operator<<(Chaos::FChaosArchive& Ar, FTetrahedralCollection& Value)
{
	Value.Serialize(Ar);
	return Ar;
}

