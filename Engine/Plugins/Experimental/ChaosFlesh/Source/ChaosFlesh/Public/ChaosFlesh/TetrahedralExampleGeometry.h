// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosFlesh/TetrahedralCollection.h"
#include "Templates/UniquePtr.h"
#include "CoreFwd.h"

namespace ChaosFlesh 
{
	namespace ExampleGeometry
	{
		struct UnitTetrahedron 
		{
			static TUniquePtr<FTetrahedralCollection> CHAOSFLESH_API Create();
		};
	}
}
