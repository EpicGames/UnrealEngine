// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"

void CHAOSFLESH_API RadialTetMesh(const FVector::FReal InnerRadius, const FVector::FReal OuterRadius, const FVector::FReal Height, const int32 RadialSample, const int32 AngularSample, const int32 VerticalSample, const FVector::FReal BulgeDistance, TArray<FIntVector4>& TetElements, TArray<FVector>& TetVertices);
void CHAOSFLESH_API RadialHexMesh(const FVector::FReal InnerRadius, const FVector::FReal OuterRadius, const FVector::FReal Height, const int32 RadialSample, const int32 AngularSample, const int32 VerticalSample, const FVector::FReal BulgeDistance, TArray<int32>& HexElements, TArray<FVector>& HexVertices);
void CHAOSFLESH_API RegularHexMesh2TetMesh(const TArray<FVector>& HexVertices, const TArray<int32>& HexElements, TArray<FVector>& TetVertices, TArray<FIntVector4>& TetElements);
void CHAOSFLESH_API ComputeHexMeshFaces(const TArray<int32>& HexElements, TArray<FIntVector2>& CommonFaces);

template <typename Func1, typename Func2, typename Func3>
void CHAOSFLESH_API ComputeMeshFaces(const TArray<int32>& mesh, Func1 GreaterThan, Func2 Equal, TArray<FIntVector2>& CommonFaces, Func3 GenerateFace, int32 PointsPerFace);
void CHAOSFLESH_API GenerateHexFaceMesh(const TArray<int32>& HexElements, TArray<int32>& Faces);