// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ChaosFleshDeprecatedNodes : ModuleRules
	{
        public ChaosFleshDeprecatedNodes(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"ChaosCore",
					"Chaos",
					"DataflowCore",
					"DataflowEngine",
					"ChaosFlesh",
					"ChaosFleshEngine",
					"Engine",
					"GeometryAlgorithms",
					"GeometryCore",
					"MeshConversion",
					"MeshConversionEngineTypes",
					"MeshDescription",
					"TetMeshing",
					"ChaosFleshNodes"
				}
			);
		}
	}
}
