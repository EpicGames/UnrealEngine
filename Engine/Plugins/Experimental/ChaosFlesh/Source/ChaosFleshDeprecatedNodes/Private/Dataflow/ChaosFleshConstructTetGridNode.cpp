// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshConstructTetGridNode.h"

#include "Chaos/Utilities.h"
#include "ChaosFlesh/FleshCollection.h"
#include "ChaosFlesh/ChaosFlesh.h"
#include "ChaosLog.h"
#include "Dataflow/ChaosFleshNodesUtility.h"


//=============================================================================
// FConstructTetGridNode
//=============================================================================

void FConstructTetGridNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		TUniquePtr<FFleshCollection> InCollection(GetValue<DataType>(Context, &Collection).NewCopy<FFleshCollection>());

		Chaos::TVector<int32, 3> Counts(GridCellCount[0], GridCellCount[1], GridCellCount[2]);

		Chaos::TVector<double, 3> MinCorner = -.5 * GridDomain;
		Chaos::TVector<double, 3> MaxCorner = .5 * GridDomain;
		Chaos::TUniformGrid<double, 3> Grid(MinCorner, MaxCorner, Counts, 0);

		TArray<FIntVector4> Tets;
		TArray<FVector> X;
		Chaos::Utilities::TetMeshFromGrid<double>(Grid, Tets, X);

		UE_LOGF(LogChaosFlesh, Display, "TetGrid generated %d points and %d tetrahedra.", X.Num(), Tets.Num());

		TArray<FIntVector3> Tris = UE::Dataflow::GetSurfaceTriangles(Tets, !bDiscardInteriorTriangles);
		TUniquePtr<FTetrahedralCollection> TetCollection(
			FTetrahedralCollection::NewTetrahedralCollection(X, Tris, Tets));
		InCollection->AppendGeometry(*TetCollection.Get());

		SetValue<const DataType&>(Context, *InCollection, &Collection);
	}
}
