// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshCreateTetrahedralCollectionNode.h"

#include "Chaos/Utilities.h"
#include "ChaosFlesh/ChaosFlesh.h"
#include "ChaosFlesh/FleshCollection.h"
#include "ChaosFlesh/FleshCollectionUtility.h"
#include "ChaosLog.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "FTetWildWrapper.h"
#include "Generate/IsosurfaceStuffing.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "MeshDescription.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "Rendering/SkeletalMeshLODImporterData.h"
#include "Rendering/SkeletalMeshModel.h"
#include "SkeletalMeshLODModelToDynamicMesh.h" // MeshModelingBlueprints
#include "Spatial/FastWinding.h"
#include "Spatial/MeshAABBTree3.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFleshCreateTetrahedralCollectionNode)


//=============================================================================
// FGenerateTetrahedralCollectionDataflowNodes
//=============================================================================


void FGenerateTetrahedralCollectionDataflowNodes::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<DataType>(&Collection))
	{
		TUniquePtr<FFleshCollection> InCollection(GetValue<DataType>(Context, &Collection).NewCopy<FFleshCollection>());
		TObjectPtr<const UStaticMesh> InStaticMesh(GetValue<TObjectPtr<const UStaticMesh>>(Context, &StaticMesh));
		TObjectPtr<const USkeletalMesh> InSkeletalMesh(FindInput(&SkeletalMesh) ? GetValue<TObjectPtr<const USkeletalMesh>>(Context, &SkeletalMesh) : nullptr);

		if (InStaticMesh || InSkeletalMesh)
		{
#if WITH_EDITORONLY_DATA
			UE::Geometry::FDynamicMesh3 DynamicMesh;
			if (InStaticMesh)
			{
				// make a mesh description for UE::Geometry tools
				FMeshDescriptionToDynamicMesh GetSourceMesh;
				bool bUsingHiResSource = InStaticMesh->IsHiResMeshDescriptionValid();
				const FMeshDescription* UseSourceMeshDescription =
					(bUsingHiResSource) ? InStaticMesh->GetHiResMeshDescription() : InStaticMesh->GetMeshDescription(0);
				GetSourceMesh.Convert(UseSourceMeshDescription, DynamicMesh);
			}
			else if (InSkeletalMesh)
			{
				// Check first if we have bulk data available and non-empty.
				constexpr int32 LODIndex = 0;
				FMeshDescription SourceMesh;
				if (InSkeletalMesh->HasMeshDescription(LODIndex))
				{
					InSkeletalMesh->CloneMeshDescription(LODIndex, SourceMesh);
				}
				FMeshDescriptionToDynamicMesh Converter;
				Converter.Convert(&SourceMesh, DynamicMesh);
			}

			if (!bComputeByComponent)
			{
				if (Method == TetMeshingMethod::IsoStuffing)
				{
					EvaluateIsoStuffing(Context, InCollection, DynamicMesh);
				}
				else if (Method == TetMeshingMethod::TetWild)
				{
					EvaluateTetWild(Context, InCollection, DynamicMesh);
				}
				else
				{
					ensureMsgf(false, TEXT("FGenerateTetrahedralCollectionDataflowNodes unsupported Method."));
				}
			}
			else
			{
				TArray<TArray<int32>> ConnectedComponents;
				
				TArray<FIntVector3> Faces;
				Faces.SetNum(DynamicMesh.TriangleCount());
				for (int32 i = 0; i < DynamicMesh.TriangleCount(); ++i)
				{
					Faces[i] = FIntVector3(DynamicMesh.GetTriangle(i));
				}
				Chaos::Utilities::FindConnectedRegions(Faces, ConnectedComponents);
				TArray<TUniquePtr<FFleshCollection>> CollectionBuffer;
				for (int32 i = 0; i < ConnectedComponents.Num(); i++)
				{
					CollectionBuffer.Add(TUniquePtr<FFleshCollection>(new FFleshCollection()));
					//CollectionBuffer[i]->AddElement(1, FGeometryCollection::TransformGroup);
				}
				ParallelFor(ConnectedComponents.Num(), [&](int32 i)
				{		
					UE::Geometry::FDynamicMesh3 ComponentDynamicMesh;
					TArray<FIntVector3> ComponentFaces;
					ComponentFaces.SetNum(ConnectedComponents[i].Num());
					for (FVector V : DynamicMesh.VerticesItr())
					{
						ComponentDynamicMesh.AppendVertex(V);
					}
					for (int32 j = 0; j < ConnectedComponents[i].Num(); j++)
					{
						int32 ElementIndex = ConnectedComponents[i][j];
						for (int32 ie = 0; ie < 3; ie++)
						{
							ComponentDynamicMesh.AppendTriangle(Faces[ElementIndex][0], Faces[ElementIndex][1], Faces[ElementIndex][2]);
						}
					}
					ComponentDynamicMesh.CompactInPlace();
					if (Method == TetMeshingMethod::IsoStuffing)
					{
						EvaluateIsoStuffing(Context, CollectionBuffer[i], ComponentDynamicMesh);
					}
					else if (Method == TetMeshingMethod::TetWild)
					{
						EvaluateTetWild(Context, CollectionBuffer[i], ComponentDynamicMesh);
					}
				});
				for (int32 i = 0; i < ConnectedComponents.Num(); i++)
				{
					if (CollectionBuffer[i]->NumElements(FGeometryCollection::VerticesGroup))
					{
						TSet<int32> VertexToDeleteSet;
						GeometryCollectionAlgo::ComputeStaleVertices(CollectionBuffer[i].Get(), VertexToDeleteSet);
						TArray<int32> SortedVertices = VertexToDeleteSet.Array(); SortedVertices.Sort();
						if (VertexToDeleteSet.Num()) CollectionBuffer[i]->RemoveElements(FGeometryCollection::VerticesGroup, SortedVertices);

						int32 GeomIndex = InCollection->AppendGeometry(*CollectionBuffer[i].Get());
					}
				}
			}
#else
			ensureMsgf(false, TEXT("FGenerateTetrahedralCollectionDataflowNodes is an editor only node."));
#endif
		} // end if InStaticMesh || InSkeletalMesh
		SetValue<const DataType&>(Context, *InCollection, &Collection);
	}
}

void FGenerateTetrahedralCollectionDataflowNodes::EvaluateIsoStuffing(
	UE::Dataflow::FContext& Context, 
	TUniquePtr<FFleshCollection>& InCollection,
	const UE::Geometry::FDynamicMesh3& DynamicMesh) const
{
#if WITH_EDITORONLY_DATA
	if (NumCells > 0 && (-.5 <= OffsetPercent && OffsetPercent <= 0.5))
	{
		// Tet mesh generation
		UE::Geometry::TIsosurfaceStuffing<double> IsosurfaceStuffing;
		UE::Geometry::FDynamicMeshAABBTree3 Spatial(&DynamicMesh);
		UE::Geometry::TFastWindingTree<UE::Geometry::FDynamicMesh3> FastWinding(&Spatial);
		UE::Geometry::FAxisAlignedBox3d Bounds = Spatial.GetBoundingBox();
		IsosurfaceStuffing.Bounds = FBox(Bounds);
		double CellSize = Bounds.MaxDim() / NumCells;
		IsosurfaceStuffing.CellSize = CellSize;
		IsosurfaceStuffing.IsoValue = .5 + OffsetPercent;
		IsosurfaceStuffing.Implicit = [&FastWinding, &Spatial](FVector3d Pos)
		{
			FVector3d Nearest = Spatial.FindNearestPoint(Pos);
			double WindingSign = FastWinding.FastWindingNumber(Pos) - .5;
			return FVector3d::Distance(Nearest, Pos) * FMathd::SignNonZero(WindingSign);
		};

		UE_LOGF(LogChaosFlesh, Display, "Generating tet mesh via IsoStuffing...");
		IsosurfaceStuffing.Generate();
		if (IsosurfaceStuffing.Tets.Num() > 0)
		{
			TArray<FVector> Vertices; Vertices.SetNumUninitialized(IsosurfaceStuffing.Vertices.Num());
			TArray<FIntVector4> Elements; Elements.SetNumUninitialized(IsosurfaceStuffing.Tets.Num());
			TArray<FIntVector3> SurfaceElements = UE::Dataflow::GetSurfaceTriangles(IsosurfaceStuffing.Tets, !bDiscardInteriorTriangles);

			for (int32 Tdx = 0; Tdx < IsosurfaceStuffing.Tets.Num(); ++Tdx)
			{
				Elements[Tdx] = IsosurfaceStuffing.Tets[Tdx];
			}
			for (int32 Vdx = 0; Vdx < IsosurfaceStuffing.Vertices.Num(); ++Vdx)
			{
				Vertices[Vdx] = IsosurfaceStuffing.Vertices[Vdx];
			}

			TUniquePtr<FTetrahedralCollection> TetCollection(FTetrahedralCollection::NewTetrahedralCollection(Vertices, SurfaceElements, Elements));
			InCollection->AppendGeometry(*TetCollection.Get());

			UE_LOGF(LogChaosFlesh, Display,
				"Generated tet mesh via IsoStuffing, num vertices: %d num tets: %d", Vertices.Num(), Elements.Num());
		}
		else
		{
			UE_LOGF(LogChaosFlesh, Warning, "IsoStuffing produced 0 tetrahedra.");
		}
	}
#else
	ensureMsgf(false, TEXT("FGenerateTetrahedralCollectionDataflowNodes is an editor only node."));
#endif
}

void FGenerateTetrahedralCollectionDataflowNodes::EvaluateTetWild(
	UE::Dataflow::FContext& Context, 
	TUniquePtr<FFleshCollection>& InCollection,
	const UE::Geometry::FDynamicMesh3& DynamicMesh) const
{
#if WITH_EDITORONLY_DATA
	if (/* placeholder for conditions for exec */true)
	{
		// Pull out Vertices and Triangles
		TArray<FVector> Verts;
		TArray<FIntVector3> Tris;
		for (FVector V : DynamicMesh.VerticesItr())
		{
			Verts.Add(V);
		}
		for (UE::Geometry::FIndex3i Tri : DynamicMesh.TrianglesItr())
		{
			Tris.Emplace(Tri.A, Tri.B, Tri.C);
		}

		// Tet mesh generation
		UE::Geometry::FTetWild::FTetMeshParameters Params;
		Params.bCoarsen = bCoarsen;
		Params.bExtractManifoldBoundarySurface = bExtractManifoldBoundarySurface;
		Params.bSkipSimplification = bSkipSimplification;

		Params.EpsRel = EpsRel;
		Params.MaxIts = MaxIterations;
		Params.StopEnergy = StopEnergy;
		Params.IdealEdgeLengthRel = IdealEdgeLengthRel;

		Params.bInvertOutputTets = bInvertOutputTets;

		TArray<FVector> TetVerts;
		TArray<FIntVector4> Tets;
		FProgressCancel Progress;
		UE_LOGF(LogChaosFlesh, Display,"Generating tet mesh via TetWild...");
		if (UE::Geometry::FTetWild::ComputeTetMesh(Params, Verts, Tris, TetVerts, Tets, &Progress))
		{
			TArray<FIntVector3> SurfaceElements = UE::Dataflow::GetSurfaceTriangles(Tets, !bDiscardInteriorTriangles);
			TUniquePtr<FTetrahedralCollection> TetCollection(FTetrahedralCollection::NewTetrahedralCollection(TetVerts, SurfaceElements, Tets));
			InCollection->AppendGeometry(*TetCollection.Get());

			UE_LOGF(LogChaosFlesh, Display,
				"Generated tet mesh via TetWild, num vertices: %d num tets: %d", TetVerts.Num(), Tets.Num());
		}
		else
		{
			UE_LOGF(LogChaosFlesh, Error,
				"TetWild tetrahedral mesh generation failed.");
		}
	}
#else
	ensureMsgf(false, TEXT("FGenerateTetrahedralCollectionDataflowNodes is an editor only node."));
#endif
}
