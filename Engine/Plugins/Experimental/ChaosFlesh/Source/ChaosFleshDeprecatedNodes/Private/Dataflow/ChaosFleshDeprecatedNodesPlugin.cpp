// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshDeprecatedNodesPlugin.h"

#include "Dataflow/ChaosFleshCreateTetrahedralCollectionNode.h"
#include "Dataflow/ChaosFleshConstructTetGridNode.h"
#include "Dataflow/ChaosFleshImportGEO.h"
#include "Dataflow/ChaosFleshKinematicOriginInsertionInitializationNode.h"
#include "Dataflow/ChaosFleshKinematicTetrahedralConstraintNode.h"


#define LOCTEXT_NAMESPACE "ChaosFleshDeprecatedNodes"


void IChaosFleshDeprecatedNodesPlugin::StartupModule()
{
	UE::Dataflow::RegisterChaosFleshImportGEONodes();
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGenerateTetrahedralCollectionDataflowNodes);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FConstructTetGridNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FKinematicOriginInsertionInitializationDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FKinematicTetrahedralBindingsDataflowNode);
}

void IChaosFleshDeprecatedNodesPlugin::ShutdownModule()
{
}


IMPLEMENT_MODULE(IChaosFleshDeprecatedNodesPlugin, ChaosFleshDeprecatedNodes)


#undef LOCTEXT_NAMESPACE
