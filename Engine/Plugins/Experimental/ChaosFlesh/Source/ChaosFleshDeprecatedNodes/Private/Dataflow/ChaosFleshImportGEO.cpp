// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshImportGEO.h"
#include "Dataflow/ChaosFleshNodesUtility.h"
#include "ChaosFlesh/TetrahedralCollection.h"
#include "ChaosFlesh/FleshCollection.h"

#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"

#include "GEO/GEO.h"
#include "Misc/Paths.h"

DEFINE_LOG_CATEGORY(LogImportGEO);

namespace UE::Dataflow
{
	void RegisterChaosFleshImportGEONodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FImportGEO);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FExtractGEOInt);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FExtractGEOIntVector);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FExtractGEOFloatVector);
	}
}

//=============================================================================
// FExtractGEOInt, FExtractGEOIntVector, FExtractGEOFloatVector
//=============================================================================


void
FExtractGEOInt::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<int32>(&Value))
	{
		const FGEOMapStringInt& Values = GetValue<FGEOMapStringInt>(Context, &IntVars);
		const FString& TargetName = VarName;
		if (Values.IntVars.Contains(TargetName))
		{
			int32 OutValue = *Values.IntVars.Find(TargetName);
			SetValue<int32>(Context, MoveTemp(OutValue), &Value);
		}
	}
}

void
FExtractGEOIntVector::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<int32>>(&Value))
	{
		const FGEOMapStringArrayInt& Values = GetValue<FGEOMapStringArrayInt>(Context, &IntVectorVars);
		const FString& TargetName = VarName;
		if (Values.IntVectorVars.Contains(TargetName))
		{
			TArray<int32> OutValue = *Values.IntVectorVars.Find(TargetName);
			SetValue<TArray<int32>>(Context, MoveTemp(OutValue), &Value);
		}
	}
}

void
FExtractGEOFloatVector::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<float>>(&Value))
	{
		const FGEOMapStringArrayFloat& Values = GetValue<FGEOMapStringArrayFloat>(Context, &FloatVectorVars);
		const FString& TargetName = VarName;
		if (Values.FloatVectorVars.Contains(TargetName))
		{
			TArray<float> OutValue = *Values.FloatVectorVars.Find(TargetName);
			SetValue<TArray<float>>(Context, MoveTemp(OutValue), &Value);
		}
	}
}

//=============================================================================
// FImportGEO
//=============================================================================

bool
FImportGEO::ReadGEOFile(const bool PrintStats) const
{
#if WITH_EDITOR
	if (!FPaths::FileExists(Filename.FilePath))
	{
		UE_LOGF(LogImportGEO, Error, "'%ls' cannot find GEO file: '%ls'",
			*GetName().ToString(), *Filename.FilePath);
		return false;
	}

	FPlatformFileManager& FileManager = FPlatformFileManager::Get();
	IPlatformFile& PlatformFile = FileManager.GetPlatformFile();
	if (FilePath == Filename.FilePath)
	{
		FDateTime CurrModTime = PlatformFile.GetTimeStamp(*FilePath);
		if (FileModTime == CurrModTime)
		{
			return true;
		}
	}

	IntVars.Empty();
	IntVectorVars.Empty();
	FloatVectorVars.Empty();
	IndexedStringVars.Empty();

	std::string filestr(TCHAR_TO_ANSI(*Filename.FilePath));
	std::string errlog;
	std::stringstream out(errlog);
	if (!ChaosFlesh::ReadGEO(filestr, IntVars, IntVectorVars, FloatVectorVars, IndexedStringVars, &out))
	{
		FString Tmp(ANSI_TO_TCHAR(errlog.c_str()));
		UE_LOGF(LogImportGEO, Error, "'%ls' - Failed to read GEO file '%ls'; log:\n%ls",
			*GetName().ToString(), *Filename.FilePath, *Tmp);
		return false;
	}

	FilePath = Filename.FilePath;
	FileModTime = PlatformFile.GetTimeStamp(*FilePath);

	if (PrintStats)
	{
		FString OutStr = FString::Format(TEXT("'{0}' - GEO file '{1}' contents:\n"), { *GetName().ToString(), *Filename.FilePath });
		OutStr += FString::Format(TEXT("    Int variables: ({})\n"), { IntVars.Num() });
		{
			int32 i = 1;
			for (TMap<FString, int32>::TConstIterator It = IntVars.CreateConstIterator(); It; ++It, ++i)
			{
				OutStr += FString::Format(TEXT("        {0}: '{1}'\n"), { i, *It.Key() });
			}
		}
		OutStr += FString::Format(TEXT("    Int vector variables: ({0})\n"), { IntVectorVars.Num() });
		{
			int32 i = 1;
			for (TMap<FString, TArray<int32>>::TConstIterator It = IntVectorVars.CreateConstIterator(); It; ++It, ++i)
			{
				OutStr += FString::Format(TEXT("        {0}: '{1}', Num = {2}\n"), { i, *It.Key(), It.Value().Num() });
			}
		}
		OutStr += FString::Format(TEXT("    Float vector variables: ({0})\n"), { FloatVectorVars.Num() });
		{
			int32 i = 1;
			for (TMap<FString, TArray<float>>::TConstIterator It = FloatVectorVars.CreateConstIterator(); It; ++It, ++i)
			{
				OutStr += FString::Format(TEXT("        {0}: '{1}', Num = {2}\n"), { i, *It.Key(), It.Value().Num() });
			}
		}
/*		OutStr += FString::Format(TEXT("    Indexed string variables: ({})\n"), { IndexedStringVars.Num() });
		{
			int32 i = 1;
			for (TMap<FString, TPair<TArray<std::string>, TArray<int32>>>::TConstIterator It = IndexedStringVars.CreateConstIterator(); It; ++It, ++i)
			{
				const TPair<TArray<std::string>, TArray<int32>>& Value = It.Value();
				OutStr += FString::Format(
					TEXT("        {}: '{}', Num = {}\n"),
					{ i, *It.Key(), Value.Get<0>().Num() });

				const TArray<std::string>& Value0 = Value.Get<0>();
				const TArray<int32>& Value1 = Value.Get<1>();
				for (int32 j = 0; j < Value0.Num(); j++)
				{
					FString Tmp(ANSI_TO_TCHAR(Value0[j].c_str()));
					OutStr += FString::Format(
						TEXT("            {}: '{}', Num = {}\n"), { j, *Tmp, Value1.Num() });
				}
			}
		}
*/
		UE_LOGF(LogImportGEO, Log, "%ls", *OutStr);
	}
	return true;
#else // WITH_EDITOR
	UE_LOGF(LogImportGEO, Error, "ImportGEO node '%ls' is not available on this platform.", *GetName().ToString());
	return false;
#endif // WITH_EDITOR
}

void
FImportGEO::Evaluate(
	UE::Dataflow::FContext& Context,
	const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		TUniquePtr<FFleshCollection> InCollection(GetValue<FManagedArrayCollection>(Context, &Collection).NewCopy<FFleshCollection>());

		if (!ReadGEOFile())
		{
			return;
		}

		//
		// Tet Mesh
		//

		if (bImportTetrahedronMesh)
		{
			TArray<FVector> Vertices;
			TArray<FIntVector4> Elements;

			auto FvIt = FloatVectorVars.Find("position");
			if (FvIt == nullptr)
				FvIt = FloatVectorVars.Find("P");
			if (FvIt != nullptr)
			{
				const TArray<float>& Coords = *FvIt;
				Vertices.Reserve(Coords.Num() / 3);
				for (size_t i = 0; i < Coords.Num(); i += 3)
				{
					FVector pt;
					for (size_t j = 0; j < 3; j++)
						pt[j] = Coords[i + j];
					Vertices.Add(pt);
				}
			}
			UE_LOGF(LogImportGEO, Display, "Got %d points.", Vertices.Num());

			auto IvIt = IntVectorVars.Find("pointref.indices");
			if (IvIt != nullptr)
			{
				auto iIt = IntVars.Find("Tetrahedron_run:startvertex");
				int32 StartIndex = iIt == nullptr ? 0 : *iIt;

				iIt = IntVars.Find("Tetrahedron_run:nprimitives");
				int32 NumTets = iIt == nullptr ? -1 : *iIt;

				UE_LOGF(LogImportGEO, Display, "Tet start index: %d num tets: %d", StartIndex, NumTets);

				const TArray<int32>& Indices = *IvIt;
				Elements.Reserve(Indices.Num() / 4);
				size_t stopIndex = NumTets != -1 ? StartIndex + NumTets * 4 : Indices.Num();
				for (size_t i = StartIndex; i < stopIndex; i += 4)
				{
					FIntVector4 tet;
					for (size_t j = 0; j < 4; j++)
						tet[j] = Indices[i + j];
					Elements.Add(tet);
				}
				UE_LOGF(LogImportGEO, Display, "Got %d tets.", Elements.Num());
			}

			if (Vertices.Num() && Elements.Num())
			{
				TArray<FIntVector3> SurfaceElements = UE::Dataflow::GetSurfaceTriangles(Elements, !bDiscardInteriorTriangles);
				TUniquePtr<FTetrahedralCollection> OBJCollection(
					FTetrahedralCollection::NewTetrahedralCollection(Vertices, SurfaceElements, Elements));
				InCollection->AppendGeometry(*OBJCollection.Get());

				SetValue<const FManagedArrayCollection&>(Context, *InCollection, &Collection);

				UE_LOGF(LogImportGEO, Log,
					"'%ls' - Authored tetrahedral mesh of %d vertices and %d tetrahedra",
					*GetName().ToString(), Vertices.Num(), Elements.Num());
			}
		}
	}
	else if (Out->IsA<FGEOMapStringInt>(&IntVarsOutput))
	{
		FGEOMapStringInt Value;
		Value.IntVars = IntVars;
		SetValue<const FGEOMapStringInt&>(Context, Value, &IntVarsOutput);
	}
	else if (Out->IsA<FGEOMapStringArrayInt>(&IntVectorVarsOutput))
	{
		FGEOMapStringArrayInt Value;
		Value.IntVectorVars = IntVectorVars;
		SetValue<const FGEOMapStringArrayInt&>(Context, Value, &IntVectorVarsOutput);
	}
	else if (Out->IsA<FGEOMapStringArrayFloat>(&FloatVectorVarsOutput))
	{
		FGEOMapStringArrayFloat Value;
		Value.FloatVectorVars = FloatVectorVars;
		SetValue<const FGEOMapStringArrayFloat&>(Context, Value, &FloatVectorVarsOutput);
	}
}
