// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "ChaosFleshConstructTetGridNode.generated.h"


USTRUCT(meta = (DataflowFlesh, Deprecated = "5.4"))
struct FConstructTetGridNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FConstructTetGridNode, "TetGrid", "Flesh", "")

public:
	typedef FManagedArrayCollection DataType;

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "1"))
	FIntVector GridCellCount = FIntVector(10, 10, 10);

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "0.0"))
	FVector GridDomain = FVector(10.0, 10.0, 10.0);

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	bool bDiscardInteriorTriangles = true;

	FConstructTetGridNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};





