// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

/**
 * The public interface to this module
 */
class IChaosFleshDeprecatedNodesPlugin : public IModuleInterface
{
public:
	virtual void StartupModule();
	virtual void ShutdownModule();
};

