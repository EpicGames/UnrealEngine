// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "ChaosFleshImportGEO.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogImportGEO, Verbose, All);


/** Transient Dataflow container for \c TMap<FString, int32>. */
USTRUCT(meta = (DataflowFlesh, Deprecated = "5.4"))
struct FGEOMapStringInt
{
	GENERATED_USTRUCT_BODY()
	TMap<FString, int32> IntVars;
};

/** Transient Dataflow container for \c TMap<FString, TArray<int32>>. */
USTRUCT(meta = (DataflowFlesh, Deprecated = "5.4"))
struct FGEOMapStringArrayInt
{
	GENERATED_USTRUCT_BODY()
	TMap<FString, TArray<int32>> IntVectorVars;
};

/** Transient Dataflow container for \c TMap<FString, TArray<float>>. */
USTRUCT(meta = (DataflowFlesh, Deprecated = "5.4"))
struct FGEOMapStringArrayFloat
{
	GENERATED_USTRUCT_BODY()
	TMap<FString, TArray<float>> FloatVectorVars;
};

/** Extract a named integer from the results of an ImportGEO node. */
USTRUCT(meta = (DataflowFlesh, Deprecated = "5.4"))
struct FExtractGEOInt : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FExtractGEOInt, "ExtractGEOInt", "Flesh", "")

public:
	/** IntVars output from ImportGEO node. */
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowInput, DisplayName = "IntVars"))
	FGEOMapStringInt IntVars;

	/** Variable name to extract the value for. (See output log from ImportGEO for available list.) */
	UPROPERTY(EditAnywhere, Category = "Dataflow")
	FString VarName;

	/** Output value. */
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowOutput))
	int32 Value = 0;

	FExtractGEOInt(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&IntVars);
		RegisterOutputConnection(&Value);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/** Extract a named integer array from the results of an ImportGEO node. */
USTRUCT(meta = (DataflowFlesh, Deprecated = "5.4"))
struct FExtractGEOIntVector : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FExtractGEOIntVector, "ExtractGEOIntVector", "Flesh", "")

public:
	/** IntVectorVars output from ImportGEO node. */
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowInput, DisplayName = "IntVars"))
	FGEOMapStringArrayInt IntVectorVars;

	/** Variable name to extract the value for. (See output log from ImportGEO for available list.) */
	UPROPERTY(EditAnywhere, Category = "Dataflow")
	FString VarName;

	/** Output value. */
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowOutput))
	TArray<int32> Value;

	FExtractGEOIntVector(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&IntVectorVars);
		RegisterOutputConnection(&Value);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/** Extract a named float array from the results of an ImportGEO node. */
USTRUCT(meta = (DataflowFlesh, Deprecated = "5.4"))
struct FExtractGEOFloatVector : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FExtractGEOFloatVector, "ExtractGEOFloatVector", "Flesh", "")

public:
	/** FloatVectorVars output from ImportGEO node. */
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowInput, DisplayName = "IntVars"))
	FGEOMapStringArrayFloat FloatVectorVars;

	/** Variable name to extract the value for. (See output log from ImportGEO for available list.) */
	UPROPERTY(EditAnywhere, Category = "Dataflow")
	FString VarName;

	/** Output value. */
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowOutput))
	TArray<float> Value;

	FExtractGEOFloatVector(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&FloatVectorVars);
		RegisterOutputConnection(&Value);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/** Import data from GEO file. */
USTRUCT(meta = (DataflowFlesh, Deprecated = "5.4"))
struct FImportGEO : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FImportGEO, "ImportGEO", "Flesh", "")
public:
    
	/** GEO filename. */
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (FilePathFilter = "geo"))
	FFilePath Filename;

	/** Passthrough geometry collection, used only for tetrahedron mesh extraction. */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DisplayName = "Collection"))
	FManagedArrayCollection Collection;

	//
	// TetMesh 
	//

	/** Import the tetrahedron mesh from the GEO file. */
	UPROPERTY(EditAnywhere, Category = "Dataflow")
	bool bImportTetrahedronMesh = true;

	/** 
	 * If \c false, all triangles on the interior of the tetrahedral mesh are retained 
	 * in the mesh, which is useful for debugging, but makes rendering more costly. 
	 */
	UPROPERTY(EditAnywhere, Category = "Dataflow")
	bool bDiscardInteriorTriangles = true;

	//
	// GEO Variables
	//

	/** Integer outputs. Use ExtractGEOInt node. */
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowOutput, DisplayName = "IntVars"))
	FGEOMapStringInt IntVarsOutput;

	/** Integer array outputs. Use ExtractGEOIntVector node. */
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowOutput, DisplayName = "IntVectorVars"))
	FGEOMapStringArrayInt IntVectorVarsOutput;

	/** Float array outputs. Use ExtractGEOFloatVector node. */
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowOutput, DisplayName = "FloatVectorVars"))
	FGEOMapStringArrayFloat FloatVectorVarsOutput;

	FImportGEO(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&IntVarsOutput);
		RegisterOutputConnection(&IntVectorVarsOutput);
		RegisterOutputConnection(&FloatVectorVarsOutput);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

protected:
	bool ReadGEOFile(const bool PrintStats=true) const;

	mutable FString FilePath;
	mutable FDateTime FileModTime;
	mutable TMap<FString, int32> IntVars;
	mutable TMap<FString, TArray<int32>> IntVectorVars;
	mutable TMap<FString, TArray<float>> FloatVectorVars;
	mutable TMap<FString, TPair<TArray<std::string>, TArray<int32>>> IndexedStringVars;
};

namespace UE::Dataflow
{
	void RegisterChaosFleshImportGEONodes();
}
