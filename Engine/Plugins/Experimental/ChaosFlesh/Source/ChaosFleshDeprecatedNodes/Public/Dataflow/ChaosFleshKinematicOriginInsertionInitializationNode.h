// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "ChaosFleshKinematicOriginInsertionInitializationNode.generated.h"

class USkeletalMesh;

// @todo(deprecate), rename to FKinematicSomethingBetterThanThisNameDataflowNode
USTRUCT(meta = (DataflowFlesh, Deprecated = "5.4"))
struct FKinematicOriginInsertionInitializationDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FKinematicOriginInsertionInitializationDataflowNode, "KinematicOriginInsertionInitialization", "Flesh", "")

public:
	typedef FManagedArrayCollection DataType;

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
		FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowInput, DisplayName = "OriginSelectionSet"))
		TArray<int32> OriginVertexIndicesIn;

	UPROPERTY(meta = (DataflowInput, DisplayName = "InsertionSelectionSet"))
		TArray<int32> InsertionVertexIndicesIn;

	UPROPERTY(meta = (DataflowInput, DisplayName = "BoneSkeletalMesh"))
		TObjectPtr<USkeletalMesh> BoneSkeletalMeshIn;


	FKinematicOriginInsertionInitializationDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterInputConnection(&OriginVertexIndicesIn);
		RegisterInputConnection(&InsertionVertexIndicesIn);
		RegisterInputConnection(&BoneSkeletalMeshIn);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

