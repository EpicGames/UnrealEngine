// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "ChaosFleshKinematicTetrahedralConstraintNode.generated.h"

class USkeletalMesh;

// @todo(deprecate), rename to FKinematicTetrahedralConstraintDataflowNode
// ... This should really be a mode on the KinematicConstraint. 
USTRUCT(meta = (DataflowFlesh, Deprecated = "5.4"))
struct FKinematicTetrahedralBindingsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FKinematicTetrahedralBindingsDataflowNode, "KinematicTetrahedralBindings", "Flesh", "")

public:
	typedef FManagedArrayCollection DataType;

	UPROPERTY(meta = (DataflowInput, DisplayName = "SkeletalMesh"))
	TObjectPtr<USkeletalMesh> SkeletalMeshIn = nullptr;

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	FString ExclusionList = "twist foo";

	FKinematicTetrahedralBindingsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&SkeletalMeshIn);
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

