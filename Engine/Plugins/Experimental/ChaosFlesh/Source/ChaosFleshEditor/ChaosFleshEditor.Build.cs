// Copyright Epic Games, Inc. All Rights Reserved.

using System.Linq;

namespace UnrealBuildTool.Rules
{
	public class ChaosFleshEditor : ModuleRules
	{
        public ChaosFleshEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicIncludePaths.AddRange(
				new string[] {
					// ... add public include paths required here ...
				}
				);

			PrivateIncludePaths.AddRange(
				new string[] {
					// ... add other private include paths required here ...
				}
				);

			SetupModulePhysicsSupport(Target);
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"AdvancedPreviewScene",
					"AssetDefinition",
					"AssetRegistry",
					"AssetTools",
					"ChaosCaching",
					"ChaosCachingUSD",
					"ChaosFlesh",
					"ChaosFleshEngine",
					"ChaosFleshNodes",
					"Core",
					"CoreUObject",
					"DataflowCore",
					"DataflowEditor",
					"DataflowEngine",
					"DataflowEnginePlugin",
					"DataflowEditor",
					"DataflowSimulation",
					"EditorStyle",
					"Engine",
					"EditorFramework",
					"EditorStyle",
					"GeometryCore",
					"GraphEditor",
					"InputCore",
					"LevelEditor",
					"MeshConversion",
					"MeshDescription",
					"Slate",
					"GeometryCache",
					"Projects",
					"PropertyEditor",
					"RHI",
					"RawMesh",
					"RenderCore",
					"SceneOutliner",
					"SkeletonEditor",
					"Slate",
					"SlateCore",
					"StaticMeshDescription",
					"ToolMenus",
					"UnrealEd",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
				"UnrealUSDWrapper",
				"USDClasses",
				"USDUtilities",
				"ModelingComponents"
				}
			);

			DynamicallyLoadedModuleNames.AddRange(
				new string[]
				{
					// ... add any modules that your module loads dynamically here ...
				}
				);

			PrivateDefinitions.Add("CHAOS_INCLUDE_LEVEL_1=1");

			bool bIsUsdSdkEnabled = UnrealUSDWrapper.CheckAndSetupUsdSdk(Target, this);

			// Build flavor selection copied from UnrealUSDWrapper, then modified.
			// Currently only Win64 is supported.
			if (bIsUsdSdkEnabled && (Target.Type == TargetType.Editor && Target.Platform == UnrealTargetPlatform.Win64))
			{
				bUseRTTI = true;
				PublicDefinitions.Add("DO_USD_CACHING=1");
			}
			else
			{
				PublicDefinitions.Add("DO_USD_CACHING=0");
			}
		}
	}
}
