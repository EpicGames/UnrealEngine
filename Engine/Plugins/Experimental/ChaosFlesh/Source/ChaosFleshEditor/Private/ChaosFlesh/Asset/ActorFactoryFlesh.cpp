// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFlesh/Asset/ActorFactoryFlesh.h"

#include "ChaosFlesh/ChaosDeformableTetrahedralComponent.h"
#include "ChaosFlesh/FleshActor.h"
#include "AssetRegistry/AssetData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorFactoryFlesh)

#define LOCTEXT_NAMESPACE "ActorFactoryFlesh"

DEFINE_LOG_CATEGORY_STATIC(LogChaosFleshFactories, Log, All);

/*-----------------------------------------------------------------------------
UActorFactoryFlesh
-----------------------------------------------------------------------------*/
UActorFactoryFlesh::UActorFactoryFlesh(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("FleshDisplayName", "Flesh");
	NewActorClass = AFleshActor::StaticClass();
}

bool UActorFactoryFlesh::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (!AssetData.IsValid() || !AssetData.IsInstanceOf(UFleshAsset::StaticClass()))
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoFleshSpecified", "No Flesh asset was specified.");
		return false;
	}

	return true;
}

void UActorFactoryFlesh::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	UFleshAsset* Flesh = CastChecked<UFleshAsset>(Asset);
	AFleshActor* NewFleshActor = CastChecked<AFleshActor>(NewActor);

	// Term Component
	 NewFleshActor->GetFleshComponent()->UnregisterComponent();

	// Change properties
	NewFleshActor->GetFleshComponent()->SetRestCollection(Flesh);

	// Init Component
	NewFleshActor->GetFleshComponent()->RegisterComponent();
}

#undef LOCTEXT_NAMESPACE
