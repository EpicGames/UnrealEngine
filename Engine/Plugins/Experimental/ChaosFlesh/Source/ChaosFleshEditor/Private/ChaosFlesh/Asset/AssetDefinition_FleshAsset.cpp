// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFlesh/Asset/AssetDefinition_FleshAsset.h"
#include "ChaosFlesh/FleshAsset.h"
#include "Dataflow/AssetDefinition_DataflowAsset.h"
#include "Dataflow/DataflowEditor.h"
#include "Dataflow/DataflowEditorModule.h"
#include "Dataflow/DataflowEditorToolkit.h"
#include "Dataflow/DataflowSimulationScene.h"
#include "Math/Color.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "Toolkits/SimpleAssetEditor.h"


#define LOCTEXT_NAMESPACE "AssetActions_FleshAsset"


namespace UE::ChaosFlesh::FleshAsset
{
	struct FColorScheme
	{
		static inline const FLinearColor Asset = FColor(180, 120, 110);
		static inline const FLinearColor NodeHeader = FColor(180, 120, 110);
		static inline const FLinearColor NodeBody = FColor(18, 12, 11, 127);
	};
}

FText UAssetDefinition_FleshAsset::GetAssetDisplayName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_FleshAsset", "FleshAsset");
}

TSoftClassPtr<UObject> UAssetDefinition_FleshAsset::GetAssetClass() const
{
	return UFleshAsset::StaticClass();
}

FLinearColor UAssetDefinition_FleshAsset::GetAssetColor() const
{
	return UE::ChaosFlesh::FleshAsset::FColorScheme::Asset;
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_FleshAsset::GetAssetCategories() const
{
	static const auto Categories = { EAssetCategoryPaths::Physics };
	return Categories;
}

UThumbnailInfo* UAssetDefinition_FleshAsset::LoadThumbnailInfo(const FAssetData& InAsset) const
{
	return UE::Editor::FindOrCreateThumbnailInfo(InAsset.GetAsset(), USceneThumbnailInfo::StaticClass());
}

EAssetCommandResult UAssetDefinition_FleshAsset::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	TArray<UFleshAsset*> FleshObjects = OpenArgs.LoadObjects<UFleshAsset>();

	// For now the flesh editor only works on one asset at a time
	ensure(FleshObjects.Num() == 0 || FleshObjects.Num() == 1);
	if (FleshObjects.Num() == 1)
	{
		// Validate the asset
		if (UFleshAsset* const FleshAsset = Cast<UFleshAsset>(FleshObjects[0]))
		{
			if (!FDataflowEditorToolkit::HasDataflowAsset(FleshAsset))
			{
				if (UDataflow* const NewDataflowAsset = Cast<UDataflow>(UE::DataflowAssetDefinitionHelpers::NewOrOpenDataflowAsset(FleshAsset)))
				{
					FleshAsset->SetDataflowAsset(NewDataflowAsset);
				}
			}

			if (FDataflowEditorToolkit::HasDataflowAsset(FleshAsset))
			{
				UAssetEditorSubsystem* const AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
				UDataflowEditor* const AssetEditor = NewObject<UDataflowEditor>(AssetEditorSubsystem, NAME_None, RF_Transient);
				
				if (UDataflowSimulationSettings* SimulationSettings = NewObject<UDataflowSimulationSettings>())
				{
					SimulationSettings->bIsSimulationPlayingByDefault = false;
					SimulationSettings->bIsAsyncCachingSupported = true;
					SimulationSettings->bIsAsyncCachingEnabledByDefault = true;
					SimulationSettings->bIsGeometryCacheOutputSupported = true;
					AssetEditor->AddEditorSettings(SimulationSettings);
				}
				
				AssetEditor->RegisterToolCategories({"General"});

				const TSubclassOf<AActor> ActorClass = StaticLoadClass(AActor::StaticClass(), nullptr,
					TEXT("/ChaosFlesh/BP_FleshPreview.BP_FleshPreview_C"), nullptr, LOAD_None, nullptr);
				AssetEditor->Initialize({ FleshAsset }, ActorClass);
				
				return EAssetCommandResult::Handled;
			}

			FSimpleAssetEditor::CreateEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, FleshAsset );
			return EAssetCommandResult::Handled;
		}
	}
	return EAssetCommandResult::Unhandled;
}

#undef LOCTEXT_NAMESPACE
