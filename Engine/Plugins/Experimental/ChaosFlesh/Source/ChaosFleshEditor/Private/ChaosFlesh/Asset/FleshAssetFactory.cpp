// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFlesh/Asset/FleshAssetFactory.h"
#include "Dataflow/AssetDefinition_DataflowAsset.h"

#include "ChaosFlesh/FleshComponent.h"
#include "Editor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FleshAssetFactory)


#define LOCTEXT_NAMESPACE "ChaosFlesh"

/////////////////////////////////////////////////////
// FleshFactory

UFleshAssetFactory::UFleshAssetFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UFleshAsset::StaticClass();
}

UObject* UFleshAssetFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UFleshAsset* NewFlesh = NewObject<UFleshAsset>(InParent, Class, Name, Flags | RF_Transactional | RF_Public | RF_Standalone);
	if (NewFlesh)
	{
		UE::DataflowAssetDefinitionHelpers::SetDataflowFromTemplatePicker(NewFlesh);
		NewFlesh->Modify();
	}

	return NewFlesh;
}

#undef LOCTEXT_NAMESPACE




