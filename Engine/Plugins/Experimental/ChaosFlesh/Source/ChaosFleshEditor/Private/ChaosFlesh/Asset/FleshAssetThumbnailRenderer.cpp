// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFlesh/Asset/FleshAssetThumbnailRenderer.h"
#include "SceneView.h"
#include "ChaosFlesh/Asset/FleshAssetThumbnailScene.h"
#include "ChaosFlesh/FleshAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FleshAssetThumbnailRenderer)

UFleshAssetThumbnailRenderer::UFleshAssetThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	ThumbnailScene = nullptr;
}

void UFleshAssetThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UFleshAsset* FleshAsset = Cast<UFleshAsset>(Object);
	if (IsValid(FleshAsset))
	{
		if (ThumbnailScene == nullptr)
		{
			ThumbnailScene = new FFleshAssetThumbnailScene();
		}

		ThumbnailScene->SetFleshAsset(FleshAsset);

		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(RenderTarget, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game))
			.SetTime(UThumbnailRenderer::GetTime())
			.SetAdditionalViewFamily(bAdditionalViewFamily));

		ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
		ViewFamily.EngineShowFlags.MotionBlur = 0;
		ViewFamily.EngineShowFlags.LOD = 0;

		RenderViewFamily(Canvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, X, Y, Width, Height));
		ThumbnailScene->SetFleshAsset(nullptr);
	}
}

void UFleshAssetThumbnailRenderer::BeginDestroy()
{
	if (ThumbnailScene != nullptr)
	{
		delete ThumbnailScene;
		ThumbnailScene = nullptr;
	}

	Super::BeginDestroy();
}

