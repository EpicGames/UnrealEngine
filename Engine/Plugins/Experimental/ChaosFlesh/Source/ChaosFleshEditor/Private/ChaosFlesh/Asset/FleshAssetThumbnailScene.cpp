// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFlesh/Asset/FleshAssetThumbnailScene.h"
#include "ChaosFlesh/ChaosDeformableTetrahedralComponent.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "ChaosFlesh/FleshActor.h"

FFleshAssetThumbnailScene::FFleshAssetThumbnailScene()
{
	bForceAllUsedMipsResident = false;

	// Create preview actor
	// checked
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnInfo.bNoFail = true;
	SpawnInfo.ObjectFlags = RF_Transient;

	PreviewActor = GetWorld()->SpawnActor<AFleshActor>(SpawnInfo);
	PreviewActor->GetFleshComponent()->SetMobility(EComponentMobility::Movable);
	PreviewActor->SetActorEnableCollision(false);
}

void FFleshAssetThumbnailScene::SetFleshAsset(UFleshAsset* FleshAsset)
{
	PreviewActor->GetFleshComponent()->SetRestCollection(FleshAsset);

	if (FleshAsset)
	{
		FTransform MeshTransform = FTransform::Identity;

		PreviewActor->SetActorLocation(FVector(0, 0, 0), false);
		PreviewActor->GetFleshComponent()->UpdateBounds();

		// Center the mesh at the world origin then offset to put it on top of the plane
		const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetFleshComponent()->Bounds);
		PreviewActor->SetActorLocation(-PreviewActor->GetFleshComponent()->Bounds.Origin + FVector(0, 0, BoundsZOffset), false);
		PreviewActor->GetFleshComponent()->RecreateRenderState_Concurrent();
	}
}

void FFleshAssetThumbnailScene::GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const
{
	check(PreviewActor);
	check(PreviewActor->GetFleshComponent());
	check(PreviewActor->GetFleshComponent()->GetRestCollection());

	const float HalfFOVRadians = FMath::DegreesToRadians<float>(InFOVDegrees) * 0.5f;
	// Add extra size to view slightly outside of the sphere to compensate for perspective
	const float HalfMeshSize = PreviewActor->GetFleshComponent()->Bounds.SphereRadius * 1.15;
	const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetFleshComponent()->Bounds);
	const float TargetDistance = HalfMeshSize / FMath::Tan(HalfFOVRadians);

	USceneThumbnailInfo* ThumbnailInfo = Cast<USceneThumbnailInfo>(PreviewActor->GetFleshComponent()->GetRestCollection()->ThumbnailInfo);
	if (ThumbnailInfo)
	{
		if (TargetDistance + ThumbnailInfo->OrbitZoom < 0)
		{
			ThumbnailInfo->OrbitZoom = -TargetDistance;
		}
	}
	else
	{
		ThumbnailInfo = USceneThumbnailInfo::StaticClass()->GetDefaultObject<USceneThumbnailInfo>();
	}

	OutOrigin = FVector(0, 0, -BoundsZOffset);
	OutOrbitPitch = ThumbnailInfo->OrbitPitch;
	OutOrbitYaw = ThumbnailInfo->OrbitYaw;
	OutOrbitZoom = TargetDistance + ThumbnailInfo->OrbitZoom;
}

