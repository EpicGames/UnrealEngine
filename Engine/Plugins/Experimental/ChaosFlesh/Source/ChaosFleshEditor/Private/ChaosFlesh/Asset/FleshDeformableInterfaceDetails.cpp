// Copyright Epic Games, Inc. All Rights Reserved.

#include "FleshDeformableInterfaceDetails.h"

#include "DetailLayoutBuilder.h"
#include "Misc/AssertionMacros.h"
#include "BodyInstanceCore.h"
#include "DeformableInterface.h"

#define LOCTEXT_NAMESPACE "DeformableInterfaceDetails"


TSharedRef<IDetailCustomization> FDeformableInterfaceDetails::MakeInstance()
{
	return MakeShareable(new FDeformableInterfaceDetails);
}

void FDeformableInterfaceDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> DetailObjects;
	DetailBuilder.GetObjectsBeingCustomized(DetailObjects);
	for (TWeakObjectPtr<UObject> DetailObject : DetailObjects)
	{
		if (IDeformableInterface* DeformableInterface = Cast<IDeformableInterface>(DetailObject.Get()))
		{
			DeformableInterface->CustomizeDetails(DetailBuilder);
		}
	}
}



#undef LOCTEXT_NAMESPACE
