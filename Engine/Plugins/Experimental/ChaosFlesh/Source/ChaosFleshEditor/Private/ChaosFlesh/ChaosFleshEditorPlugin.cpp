// Copyright Epic Games, Inc. All Rights Reserved.


#include "ChaosFlesh/ChaosFleshEditorPlugin.h"

#include "ChaosFlesh/Asset/AssetDefinition_FleshAsset.h"
#include "ChaosFlesh/Asset/FleshDeformableInterfaceDetails.h"
#include "ChaosFlesh/Asset/FleshAssetThumbnailRenderer.h"
#include "ChaosFlesh/ChaosDeformableCollisionsActor.h"
#include "ChaosFlesh/ChaosDeformableConstraintsActor.h"
#include "ChaosFlesh/ChaosDeformableSolverComponent.h"
#include "ChaosFlesh/Cmd/ChaosFleshCommands.h"
#include "ChaosFlesh/FleshActor.h"
#include "ChaosFlesh/FleshAsset.h"
#include "ChaosFlesh/FleshComponent.h"
#include "Dataflow/AssetDefinition_DataflowAsset.h"
#include "Dataflow/DataflowTemplateRegistry.h"
#include "Editor/FleshEditorStyle.h"

#include "ChaosFleshRendering/FleshTetrahedronRenderableType.h"
#include "ChaosFleshRendering/FleshVectorFieldRenderableType.h"
#include "ChaosFleshRendering/FleshFiberFieldRenderableType.h"

#define LOCTEXT_NAMESPACE "FleshEditor"

TOptional<FChaosFleshEditorStyle> FChaosFleshEditorStyle::Singleton;

void IChaosFleshEditorPlugin::StartupModule()
{
	FChaosFleshEditorStyle::Get();

	if (GIsEditor && !IsRunningCommandlet())
	{
		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("ChaosDeformable.ImportFile"),
			TEXT("Creates a FleshAsset from the input file"),
			FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&FChaosFleshCommands::ImportFile),
			ECVF_Default
		));

		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("ChaosDeformable.FindQualifyingTetrahedra"),
			TEXT("From the selected actor's flesh components, prints indices of tetrahedra matching our search criteria. "
				"Use arg 'MinVol <value>' to specify a minimum tet volume; "
				"use arg 'MaxAR <value>' to specify a maximum aspect ratio; "
				"use 'XCoordGT <value>', 'YCoordGT <value>', 'ZCoordGT <value>' to select tets with all vertices greater than the specified value; "
				"use 'XCoordLT <value>', 'YCoordLT <value>', 'ZCoordLT <value>' to select tets with all vertices less than the specified value; "
				"use 'HideTets' to add indices to the flesh component's list of tets to skip drawing."),
			FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&FChaosFleshCommands::FindQualifyingTetrahedra),
			ECVF_Default
		));

		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("ChaosDeformable.CreateGeometryCache"),
			TEXT("With an actor with flesh component(s) and a chaos cache manager selected (or use arg UsdFile), "
				"generates a GeometryCache asset from the topology of associated SkeletalMeshComponent's import geometry, "
				"and the simulation results from the USD file.  Requires deformer bindings for the import geometry in the "
				"flesh component rest collection."
				"Use arg 'UsdFile </path/to/file.usd>' to specify a specific USD file, rather than infering it from a chaos cache manager."),
			FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&FChaosFleshCommands::CreateGeometryCache),
			ECVF_Default
		));
	}

	// register details customization
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyModule.RegisterCustomClassLayout(
		ADeformableCollisionsActor::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FDeformableInterfaceDetails::MakeInstance));

	PropertyModule.RegisterCustomClassLayout(
		ADeformableConstraintsActor::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FDeformableInterfaceDetails::MakeInstance));

	PropertyModule.RegisterCustomClassLayout(
		ADeformableSolverActor::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FDeformableInterfaceDetails::MakeInstance));

	PropertyModule.RegisterCustomClassLayout(
		AFleshActor::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FDeformableInterfaceDetails::MakeInstance));

	PropertyModule.RegisterCustomClassLayout(
		UDeformablePhysicsComponent::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FDeformableInterfaceDetails::MakeInstance));

	PropertyModule.RegisterCustomClassLayout(
		UDeformableSolverComponent::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FDeformableInterfaceDetails::MakeInstance));

	// Register render types
	UE::Flesh::Private::RegisterFleshTetrahedronRenderableTypes();
	UE::Flesh::Private::RegisterFleshVectorFieldRenderableTypes();
	UE::Flesh::Private::RegisterFleshFiberFieldRenderableTypes();

	FDataflowTemplateRegistry::Get().RegisterTemplateFolder(UFleshAsset::StaticClass(),
		{ TEXT("/ChaosFlesh/Templates"), FSlateIcon("ChaosFleshEditorStyle", "ClassThumbnail.FleshAsset") });
	
	DataflowMenusHandle = UE::DataflowAssetDefinitionHelpers::RegisterDataflowAssetMenus(UFleshAsset::StaticClass());

}

void IChaosFleshEditorPlugin::ShutdownModule()
{
	UE::DataflowAssetDefinitionHelpers::UnregisterDataflowAssetMenus(DataflowMenusHandle);
}

IMPLEMENT_MODULE(IChaosFleshEditorPlugin, ChaosFleshEditor)

#undef LOCTEXT_NAMESPACE
