// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFlesh/Cmd/ChaosFleshCommands.h"

#include "ChaosFlesh/Asset/FleshAssetFactory.h"
#include "ChaosFlesh/Cmd/FleshAssetConversion.h"
#include "ChaosFlesh/FleshAsset.h"
#include "ChaosFlesh/FleshCollection.h"
#include "ChaosFlesh/FleshCollectionEngineUtility.h"
#include "ChaosFlesh/ChaosDeformableTetrahedralComponent.h"

#include "Chaos/CacheManagerActor.h"
#include "Chaos/Matrix.h"
#include "Chaos/Tetrahedron.h"
#include "ChaosCache/FleshComponentCacheAdapter.h"
#include "Dataflow/ChaosFleshGenerateSurfaceBindingsNode.h"

#if USE_USD_SDK && DO_USD_CACHING
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "ChaosCachingUSD/Operations.h"
#include "USDConversionUtils.h"
#endif

#include "AssetRegistry/AssetRegistryModule.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Selection.h"
#include "Engine/World.h"
#include "EngineUtils.h"

#include "GeometryCache.h"
#include "GeometryCacheCodecV1.h"
#include "GeometryCacheMeshData.h"
#include "GeometryCacheTrackStreamable.h"
#include "GeometryCacheConstantTopologyWriter.h"

#include "GeometryCollection/Facades/CollectionTetrahedralBindingsFacade.h"
#include "GeometryCollection/TransformCollection.h"

#include "Engine/SkeletalMesh.h"
#include "Components/SkeletalMeshComponent.h"
#include "MeshDescription.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "Misc/Paths.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "Rendering/SkeletalMeshLODImporterData.h"
#include "Rendering/SkeletalMeshModel.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"

#include "FileHelpers.h"


DEFINE_LOG_CATEGORY_STATIC(UChaosFleshCommandsLogging, Log, All);

namespace ArgParse {

	int32 ArgIndex(const TArray<FString>& Args, const FString& Target)
	{
		return Args.Find(Target);
	}

	bool ArgExists(const TArray<FString>& Args, const FString& Target)
	{
		return ArgIndex(Args, Target) != INDEX_NONE;
	}

	bool ArgStringValue(const TArray<FString>& Args, const FString& Target, FString& Value)
	{
		for (int32 i = 0; i < Args.Num() - 1; i++)
		{
			if (Args[i] == Target)
			{
				Value = Args[i + 1];
				Value.RemoveFromStart(TEXT("'"));
				Value.RemoveFromStart(TEXT("\""));
				Value.RemoveFromEnd(TEXT("'"));
				Value.RemoveFromEnd(TEXT("\""));
				return true;
			}
		}
		return false;
	}

	bool ArgFloatValue(const TArray<FString>& Args, const FString& Target, float& Value)
	{
		FString StrValue;
		if (ArgStringValue(Args, Target, StrValue))
		{
			Value = FCString::Atof(*StrValue);
			return true;
		}
		return false;
	}

	bool ArgIntValue(const TArray<FString>& Args, const FString& Target, int32& Value)
	{
		FString StrValue;
		if (ArgStringValue(Args, Target, StrValue))
		{
			Value = FCString::Atoi(*StrValue);
			return true;
		}
		return false;
	}

} // namespace ArgParse

void FChaosFleshCommands::ImportFile(const TArray<FString>& Args, UWorld* World)
{
	if (Args.Num() == 1)
	{
		//FString BasePath = FPaths::ProjectDir() + "Imports/default.geo";// +FString(Args[0]);
		if (FPaths::FileExists(Args[0]))
		{
			auto Factory = NewObject<UFleshAssetFactory>();
			UPackage* Package = CreatePackage(TEXT("/Game/FleshAsset"));
			//UPackage* Package = CreatePackage( *FPaths::ProjectContentDir() );

			UFleshAsset* FleshAsset = static_cast<UFleshAsset*>(Factory->FactoryCreateNew(UFleshAsset::StaticClass(), Package, FName("FleshAsset"), RF_Standalone | RF_Public, NULL, GWarn));
			FAssetRegistryModule::AssetCreated(FleshAsset);
			{
				FFleshAssetEdit EditObject = FleshAsset->EditCollection();
				if (FFleshCollection* Collection = EditObject.GetFleshCollection())
				{
					UE_LOGF(UChaosFleshCommandsLogging, Log, "FChaosFleshCommands::ImportFile");
					if (TUniquePtr<FFleshCollection> InCollection = FFleshAssetConversion::ImportTetFromFile(Args[0]))
					{
						Collection->CopyMatchingAttributesFrom(*InCollection);
					}
				}
				Package->SetDirtyFlag(true);
			}
		}
	}
	else
	{
		UE_LOGF(UChaosFleshCommandsLogging, Error, "Failed to import file for flesh asset.");
	}
}

void
FChaosFleshCommands::FindQualifyingTetrahedra(const TArray<FString>& Args, UWorld* World)
{
	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		for (FSelectionIterator ActorIt(*SelectedActors); ActorIt; ++ActorIt)
		{
			if (AActor* Actor = Cast<AActor>(*ActorIt))
			{
				const TSet<UActorComponent*>& Components = Actor->GetComponents();
				for (TSet<UActorComponent*>::TConstIterator CompIt = Components.CreateConstIterator(); CompIt; ++CompIt)
				{
					//if (UFleshComponent* FleshComponent = Cast<UFleshComponent>(*CompIt))
					if (UDeformableTetrahedralComponent* FleshComponent = Cast<UDeformableTetrahedralComponent >(*CompIt))
					{
						if (const UFleshAsset* RestCollection = FleshComponent->GetRestCollection())
						{
							TSharedPtr<const FFleshCollection> FleshCollection = RestCollection->GetFleshCollection();

							const TManagedArray<FIntVector4>* TetMesh =
								FleshCollection->FindAttribute<FIntVector4>(
									FTetrahedralCollection::TetrahedronAttribute, FTetrahedralCollection::TetrahedralGroup);
							const TManagedArray<int32>* TetrahedronStart =
								FleshCollection->FindAttribute<int32>(
									FTetrahedralCollection::TetrahedronStartAttribute, FGeometryCollection::GeometryGroup);
							const TManagedArray<int32>* TetrahedronCount =
								FleshCollection->FindAttribute<int32>(
									FTetrahedralCollection::TetrahedronCountAttribute, FGeometryCollection::GeometryGroup);

							const UFleshDynamicAsset* DynamicCollection = FleshComponent->GetDynamicCollection();
							const TManagedArray<FVector3f>* Vertex =
								DynamicCollection && DynamicCollection->FindPositions() && DynamicCollection->FindPositions()->Num() ?
								DynamicCollection->FindPositions() :
								FleshCollection->FindAttribute<FVector3f>("Vertex", "Vertices");

							float MaxAR = TNumericLimits<float>::Max();
							float MinVol = -TNumericLimits<float>::Max();
							float XCoordGT = TNumericLimits<float>::Max();
							float YCoordGT = TNumericLimits<float>::Max();
							float ZCoordGT = TNumericLimits<float>::Max();
							float XCoordLT = -TNumericLimits<float>::Max();
							float YCoordLT = -TNumericLimits<float>::Max();
							float ZCoordLT = -TNumericLimits<float>::Max();
							bool bHideTets = false;

							ArgParse::ArgFloatValue(Args, FString(TEXT("MaxAR")), MaxAR);
							ArgParse::ArgFloatValue(Args, FString(TEXT("MinVol")), MinVol);
							ArgParse::ArgFloatValue(Args, FString(TEXT("XCoordGT")), XCoordGT);
							ArgParse::ArgFloatValue(Args, FString(TEXT("YCoordGT")), YCoordGT);
							ArgParse::ArgFloatValue(Args, FString(TEXT("ZCoordGT")), ZCoordGT);
							ArgParse::ArgFloatValue(Args, FString(TEXT("XCoordLT")), XCoordLT);
							ArgParse::ArgFloatValue(Args, FString(TEXT("YCoordLT")), YCoordLT);
							ArgParse::ArgFloatValue(Args, FString(TEXT("ZCoordLT")), ZCoordLT);
							bHideTets = ArgParse::ArgExists(Args, FString(TEXT("HideTets")));

							TArray<int32> Indices;
							for (int32 TetMeshIdx = 0; TetMeshIdx < TetrahedronStart->Num(); TetMeshIdx++)
							{
								const int32 TetMeshStart = (*TetrahedronStart)[TetMeshIdx];
								const int32 TetMeshCount = (*TetrahedronCount)[TetMeshIdx];

								for (int32 i = 0; i < TetMeshCount; i++)
								{
									const int32 Idx = TetMeshStart + i;
									const FIntVector4& Tet = (*TetMesh)[Idx];

									const int32 MaxIdx = FGenericPlatformMath::Max(
										FGenericPlatformMath::Max(Tet[0], Tet[1]),
										FGenericPlatformMath::Max(Tet[2], Tet[3]));
									if (MaxIdx >= Vertex->Num())
									{
										continue;
									}

									Chaos::TTetrahedron<Chaos::FReal> Tetrahedron(
										(*Vertex)[Tet[0]],
										(*Vertex)[Tet[1]],
										(*Vertex)[Tet[2]],
										(*Vertex)[Tet[3]]);

									if (MinVol != -TNumericLimits<float>::Max())
									{
										float Vol = Tetrahedron.GetSignedVolume();
										if (Vol < MinVol)
										{
											Indices.Add(Idx);
											continue;
										}
									}
									if (MaxAR != TNumericLimits<float>::Max())
									{
										float AR = Tetrahedron.GetAspectRatio();
										if (AR > MaxAR)
										{
											Indices.Add(Idx);
											continue;
										}
									}
									if (XCoordGT != TNumericLimits<float>::Max() ||
										YCoordGT != TNumericLimits<float>::Max() ||
										ZCoordGT != TNumericLimits<float>::Max())
									{
										bool bAdd = true;
										if (XCoordGT != TNumericLimits<float>::Max())
										{
											for (int32 j = 0; j < 4; j++) bAdd &= (*Vertex)[Tet[j]][0] > XCoordGT;
										}
										if (YCoordGT != TNumericLimits<float>::Max())
										{
											for (int32 j = 0; j < 4; j++) bAdd &= (*Vertex)[Tet[j]][1] > YCoordGT;
										}
										if (ZCoordGT != TNumericLimits<float>::Max())
										{
											for (int32 j = 0; j < 4; j++) bAdd &= (*Vertex)[Tet[j]][2] > ZCoordGT;
										}
										if (bAdd)
										{
											Indices.Add(Idx);
											continue;
										}
									}
									if (XCoordLT != -TNumericLimits<float>::Max() ||
										YCoordLT != -TNumericLimits<float>::Max() ||
										ZCoordLT != -TNumericLimits<float>::Max())
									{
										bool bAdd = true;
										if (XCoordLT != -TNumericLimits<float>::Max())
										{
											for (int32 j = 0; j < 4; j++) bAdd &= (*Vertex)[Tet[j]][0] < XCoordLT;
										}
										if (YCoordLT != -TNumericLimits<float>::Max())
										{
											for (int32 j = 0; j < 4; j++) bAdd &= (*Vertex)[Tet[j]][1] < YCoordLT;
										}
										if (ZCoordLT != -TNumericLimits<float>::Max())
										{
											for (int32 j = 0; j < 4; j++) bAdd &= (*Vertex)[Tet[j]][2] < ZCoordLT;
										}
										if (bAdd)
										{
											Indices.Add(Idx);
											continue;
										}
									}
								}
							}

							if (bHideTets)
							{
								FleshComponent->HideTetrahedra.Append(Indices);
							}

							if (Indices.Num())
							{
								FString IndicesStr(TEXT("["));
								int32 i = 0;
								for (; i < Indices.Num() - 1; i++)
								{
									IndicesStr.Append(FString::Printf(TEXT("%d "), Indices[i]));
								}
								if (i < Indices.Num())
								{
									IndicesStr.Append(FString::Printf(TEXT("%d]"), Indices[i]));
								}
								else
								{
									IndicesStr.Append(TEXT("]"));
								}
								UE_LOGF(UChaosFleshCommandsLogging, Log,
									"ChaosDeformableCommands.FindQualifyingTetrahedra - '%ls.%ls' Found %d qualifying tetrahedra: \n%ls",
									*Actor->GetName(),
									*FleshComponent->GetName(),
									Indices.Num(),
									*IndicesStr);
							}

						} // end if RestCollection
					}
				}
			}
		}
	}
}

namespace GeometryCacheTranslatorImpl {

	template<typename T>
	T* CreateObjectInstance(UObject*& InParent, const FString& ObjectName, const EObjectFlags Flags)
	{
		// Parent package to place new asset
		UPackage* Package = nullptr;
		FString NewPackageName;

		// Setup package name and create one accordingly
		NewPackageName = FPackageName::GetLongPackagePath(InParent->GetOutermost()->GetPathName()) + TEXT("/") + ObjectName;
		NewPackageName = UPackageTools::SanitizePackageName(NewPackageName);
		Package = CreatePackage(*NewPackageName);

		const FString SanitizedObjectName = ObjectTools::SanitizeObjectName(ObjectName);

		T* ExistingTypedObject = FindObject<T>(Package, *SanitizedObjectName);
		UObject* ExistingObject = FindObject<UObject>(Package, *SanitizedObjectName);

		if (ExistingTypedObject != nullptr)
		{
			ExistingTypedObject->PreEditChange(nullptr);
		}
		else if (ExistingObject != nullptr)
		{
			// Replacing an object.  Here we go!
			// Delete the existing object
			const bool bDeleteSucceeded = ObjectTools::DeleteSingleObject(ExistingObject);

			if (bDeleteSucceeded)
			{
				// Force GC so we can cleanly create a new asset (and not do an 'in place' replacement)
				CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

				// Create a package for each mesh
				Package = CreatePackage(*NewPackageName);
				InParent = Package;
			}
			else
			{
				// failed to delete
				return nullptr;
			}
		}

		return NewObject<T>(Package, FName(*SanitizedObjectName), Flags | RF_Public);
	}

	TOptional<TArray<int32>>
	GetMeshImportVertexMap(const USkinnedAsset& SkeletalMeshAsset)
	{
		constexpr int32 LODIndex = 0;
		const TOptional<TArray<int32>> None;

		const FSkeletalMeshModel* const MLDModel = SkeletalMeshAsset.GetImportedModel();
		if (!MLDModel || !MLDModel->LODModels.IsValidIndex(LODIndex))
		{
			return None;
		}

		const FSkeletalMeshLODModel& MLDLOD = MLDModel->LODModels[LODIndex];
		const TArray<int32>& Map = MLDLOD.MeshToImportVertexMap;
		if (Map.IsEmpty())
		{
			UE_LOGF(UChaosFleshCommandsLogging, Warning, 
				"MeshToImportVertexMap is empty. MLDeformer Asset should be an imported SkeletalMesh (e.g. from fbx).");
			return None;
		}

		return Map;
	}

} //namespace GeometryCacheTranslatorImpl

void FChaosFleshCommands::CreateGeometryCache(const TArray<FString>& Args, UWorld* World)
{
	FPlatformFileManager& FileManager = FPlatformFileManager::Get();
	IPlatformFile& PlatformFile = FileManager.GetPlatformFile();

	// Get the current selection.
	USelection* SelectedActors = GEditor->GetSelectedActors();
	if (!SelectedActors)
	{
		UE_LOGF(UChaosFleshCommandsLogging, Error,
			"CreateGeometryCache - No ChaosCacheManager or Actor with FleshComponent(s) found in selection.");
		return;
	}

	// Find cache file, if specified.
	FString UsdFileOverride;
	bool bUsdFile = ArgParse::ArgStringValue(Args, FString(TEXT("UsdFile")), UsdFileOverride);

	float FrameRate = 24.;
	ArgParse::ArgFloatValue(Args, FString(TEXT("FrameRate")), FrameRate);

	int MaxNumFrames = INT_MAX;
	ArgParse::ArgIntValue(Args, FString(TEXT("MaxNumFrames")), MaxNumFrames);

	// Find a ChaosCacheManager, if a cache file hasn't been specified.
	AChaosCacheManager* CacheManager = nullptr;
	if (bUsdFile)
	{
		if (!PlatformFile.FileExists(*UsdFileOverride))
		{
			UE_LOGF(UChaosFleshCommandsLogging, Error,
				"CreateGeometryCache - File not found: '%ls'", *UsdFileOverride);
			return;
		}
	}
	else
	{
		for (FSelectionIterator ActorIt(*SelectedActors); ActorIt; ++ActorIt)
		{
			if (AActor* Actor = Cast<AActor>(*ActorIt))
			{
				CacheManager = Cast<AChaosCacheManager>(Actor);
				if (CacheManager)
				{
					break;
				}
			}
		}
		if (!CacheManager)
		{
			UE_LOGF(UChaosFleshCommandsLogging, Error,
				"CreateGeometryCache - No ChaosCacheManager found in selection, and no cache file specified.");
			return;
		}
	}
	const TArray<FObservedComponent>& ObservedComponents = 
		CacheManager ? CacheManager->GetObservedComponents() : TArray<FObservedComponent>();

	// Find actors with flesh components.
	TArray<UPackage*> PackagesToSave;
	TArray<AActor*> Actors;
	for (FSelectionIterator ActorIt(*SelectedActors); ActorIt; ++ActorIt)
	{
		AActor* Actor = Cast<AActor>(*ActorIt);
		if (!Actor || Actor == CacheManager || Actors.Contains(Actor))
		{
			continue;
		}
		Actors.Add(Actor);

		//
		// Find flesh component w/cache file, and skeletal mesh components
		//

		TArray<UDeformableTetrahedralComponent*> FleshComponents;
		TArray<FString> USDCacheFilePaths;
		TArray<const USkeletalMesh*> SkeletalMeshComponents;

		const TSet<UActorComponent*>& Components = Actor->GetComponents();
		for (TSet<UActorComponent*>::TConstIterator CompIt = Components.CreateConstIterator(); CompIt; ++CompIt)
		{
			if (UDeformableTetrahedralComponent* FleshComponent = Cast<UDeformableTetrahedralComponent>(*CompIt))
			{
				if (FleshComponents.Contains(FleshComponent))
				{
					continue;
				}

#if USE_USD_SDK && DO_USD_CACHING
				if (bUsdFile)
				{
					FleshComponents.Add(FleshComponent);
					USDCacheFilePaths.Add(UsdFileOverride);
				}
				else
				{
					for (const FObservedComponent& ObservedComponent : ObservedComponents)
					{
						FString CacheFilePath = Chaos::FFleshCacheAdapter::GetUSDCacheFilePathRO(ObservedComponent, FleshComponent);
						if (PlatformFile.FileExists(*CacheFilePath))
						{
							FleshComponents.Add(FleshComponent);
							USDCacheFilePaths.Add(CacheFilePath);
						}
						else
						{
							UE_LOGF(UChaosFleshCommandsLogging, Warning,
								"CreateGeometryCache - Failed to find USD file: '%ls'",
								*CacheFilePath);
						}
					}
				}
#endif // USE_USD_SDK && DO_USD_CACHING
			}
			else if (const USkeletalMeshComponent* SkeletalMeshComponent = Cast<const USkeletalMeshComponent>(*CompIt))
			{
				if (const USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset())
				{
					SkeletalMeshComponents.Add(SkeletalMesh);
				}
			}
		}
		if (FleshComponents.IsEmpty() || SkeletalMeshComponents.IsEmpty())
		{
			UE_LOGF(UChaosFleshCommandsLogging, Warning,
				"CreateGeometryCache - Failed to find any FleshComponents with a valid simulation cache file and a skeletal mesh component for actor: '%ls'",
				*Actor->GetName());
			continue;
		}

		//
		// Create GeometryCache instance, named after the actor
		// 

		UObject* InParent = Actor;
		EObjectFlags Flags = RF_Public | RF_Standalone; //  RF_NoFlags;

		FString GCName = FPaths::GetBaseFilename(InParent->GetName());
		GCName = FPaths::SetExtension(GCName, FString(TEXT("uasset")));

		UGeometryCache* GeometryCache =
			GeometryCacheTranslatorImpl::CreateObjectInstance<UGeometryCache>(
				InParent,
				GCName,
				Flags);
		if (!GeometryCache)
		{
				UE_LOGF(UChaosFleshCommandsLogging, Error, 
					"CreateGeometryCache - Failed to create geometry cache instance for actor: '%ls'",
					*InParent->GetName());
				continue;
		}
		else
		{
			UE_LOGF(UChaosFleshCommandsLogging, Log,
				"CreateGeometryCache - Created geometry cache instance: '%ls'",
				*GeometryCache->GetName());
		}

		for (int32 i = 0; i < FleshComponents.Num(); i++)
		{
			UDeformableTetrahedralComponent* FleshComponent = FleshComponents[i];
			const FString& USDCacheFilePath = USDCacheFilePaths[i];

			const UFleshAsset* RestCollection = FleshComponent->GetRestCollection();
			if (!RestCollection)
			{
				continue;
			}
			TSharedPtr<const FFleshCollection> FleshCollection = RestCollection->GetFleshCollection();
			if (!FleshCollection)
			{
				continue;
			}

			const TManagedArray<FVector3f>* RestVertices =
				RestCollection->FindPositions();

			const TManagedArray<FIntVector4>* TetMesh =
				FleshCollection->FindAttribute<FIntVector4>(
					FTetrahedralCollection::TetrahedronAttribute, FTetrahedralCollection::TetrahedralGroup);
			const TManagedArray<int32>* TetrahedronStart =
				FleshCollection->FindAttribute<int32>(
					FTetrahedralCollection::TetrahedronStartAttribute, FGeometryCollection::GeometryGroup);
			const TManagedArray<int32>* TetrahedronCount =
				FleshCollection->FindAttribute<int32>(
					FTetrahedralCollection::TetrahedronCountAttribute, FGeometryCollection::GeometryGroup);

			GeometryCollection::Facades::FTetrahedralBindings TetBindings(*FleshCollection);

			const int32 LODIndex = 0;
			for (const USkeletalMesh* SkeletalMesh : SkeletalMeshComponents)
			{
				TOptional<TArray<int32>> OptionalMap = GeometryCacheTranslatorImpl::GetMeshImportVertexMap(*SkeletalMesh);
				const TArray<int32>* MeshToImportVertexMap = OptionalMap.IsSet() ? &OptionalMap.GetValue() : nullptr;

				//
				// Extract bindings from tet mesh to skel mesh import geometry in the rest collection.
				//

				FString MeshId = ChaosFlesh::GetMeshId(SkeletalMesh, false);
				FName MeshIdName(MeshId);
				const int32 TetIndex = TetBindings.GetTetMeshIndex(MeshIdName, LODIndex);
				if (TetIndex == INDEX_NONE)
				{
					UE_LOGF(UChaosFleshCommandsLogging, Error,
						"CreateGeometryCache - No tet mesh index associated with mesh '%ls' LOD: %d",
						*MeshId, LODIndex);
					continue;
				}
				if (!TetBindings.ReadBindingsGroup(TetIndex, MeshIdName, LODIndex))
				{
					UE_LOGF(UChaosFleshCommandsLogging, Error,
						"CreateGeometryCache - Failed to read bindings group associated with mesh '%ls' LOD: %d",
						*MeshId, LODIndex);
					continue;
				}

				TUniquePtr<GeometryCollection::Facades::FTetrahedralBindings::Evaluator> BindingsEvalPtr = 
					TetBindings.InitEvaluator(RestVertices);
				const GeometryCollection::Facades::FTetrahedralBindings::Evaluator& BindingsEval = *BindingsEvalPtr.Get();
				if (!BindingsEval.IsValid())
				{
					UE_LOGF(UChaosFleshCommandsLogging, Error,
						"CreateGeometryCache - Bindings group associated with mesh '%ls' LOD: %d has invalid data.",
						*MeshId, LODIndex);
					continue;
				}
#if USE_USD_SDK && DO_USD_CACHING
				const int32 NumVertices = BindingsEval.NumVertices();

				TArray<TArray<FVector3f>> FramePositions;
				//
				// Open the USD cache, get all time samples.
				//

				UE::FUsdStage USDStage;
				if (!UE::ChaosCachingUSD::OpenStage(USDCacheFilePath, USDStage))
				{
					// Warn
					continue;
				}
				else
				{
					UE_LOGF(UChaosFleshCommandsLogging, Log,
						"CreateGeometryCache - Opened USD stage: '%ls'",
						*USDCacheFilePath);
				}

				FString PrimPath = UsdUtils::GetPrimPathForObject(FleshComponent);
				TArray<double> TimeSamples;
				if (!UE::ChaosCachingUSD::ReadTimeSamples(USDStage, PrimPath, TimeSamples))
				{
					// Warn
					continue;
				}

				if (TimeSamples.Num() > 1)
				{
					float MinTime = TimeSamples[0];
					float MaxTime = TimeSamples[TimeSamples.Num() - 1];
					float DeltaTime = 1. / FMath::Max(1.,FrameRate);
					float TotalCacheTime = MaxTime - MinTime;

					int NumSamples = FMath::Min(MaxNumFrames, TotalCacheTime / DeltaTime);
					TimeSamples.SetNumUninitialized(NumSamples);

					float CurrentTime = MinTime;
					for (int SampleIdx = 0; SampleIdx < NumSamples; SampleIdx++)
					{
						ensure(MinTime <= CurrentTime && CurrentTime <= MaxTime);
						TimeSamples[SampleIdx] = CurrentTime;
						CurrentTime += DeltaTime;
					}
				}


				//
				// Deform render geometry, storing per frame data in CurrFramePositions.
				//

				UE_LOGF(UChaosFleshCommandsLogging, Log,
					"CreateGeometryCache - FleshComponent '%ls' deforming SkeleltalMesh '%ls' render geometry over %d time samples...",
					*FleshComponent->GetName(), *SkeletalMesh->GetName(), TimeSamples.Num());

				FramePositions.SetNum(TimeSamples.Num());

				TArray<Chaos::TVector<Chaos::FRealSingle, 3>> CurrTetVertices;
				int32 CurrTimeIndex = 0;
				for (const double Time : TimeSamples)
				{
					if (!UE::ChaosCachingUSD::ReadPoints(
						USDStage, PrimPath, UE::ChaosCachingUSD::GetPointsAttrName(), Time, CurrTetVertices))
					{
						// Warn
						continue;
					}

					TArray<FVector3f>& CurrFramePositions = FramePositions[CurrTimeIndex++];
					CurrFramePositions.SetNumUninitialized(NumVertices);
					for (int j = 0; j < NumVertices; j++)
					{
						CurrFramePositions[j] = BindingsEval.GetEmbeddedPosition(j, CurrTetVertices);
					}
				}

				UE::ChaosCachingUSD::CloseStage(USDStage);

				//
				// Write deformed render vertices to GeometryCache.
				//

				UE_LOGF(UChaosFleshCommandsLogging, Log,
					"CreateGeometryCache - FleshComponent '%ls' writing deformed SkeleltalMesh '%ls' render geometry to geometry cache: '%ls'",
					*FleshComponent->GetName(), *SkeletalMesh->GetName(), *GeometryCache->GetName());

				UE::GeometryCacheHelpers::FGeometryCacheConstantTopologyWriter::FConfig Config;
				Config.FPS = FrameRate;
				UE::GeometryCacheHelpers::FGeometryCacheConstantTopologyWriter Writer(*GeometryCache, Config);

				// Writes Indices, UV's, Colors, ImportedVertexNumbers, and "BatchesInfo".
				const int32 Index = UE::GeometryCacheHelpers::AddTrackWriterFromSkinnedAsset(Writer, *SkeletalMesh);
				if (Index == INDEX_NONE)
				{
					UE_LOGF(UChaosFleshCommandsLogging, Error,
						"CreateGeometryCache - FleshComponent '%ls' failed to write topology to geometry cache.",
						*FleshComponent->GetName());
					continue;
				}

				UE::GeometryCacheHelpers::FGeometryCacheConstantTopologyWriter::FTrackWriter& TrackWriter = Writer.GetTrackWriter(Index);
				if (MeshToImportVertexMap)
				{
					TrackWriter.ImportedVertexNumbers.SetNum(MeshToImportVertexMap->Num());
					for (int32 j = 0; j < MeshToImportVertexMap->Num(); j++)
					{
						TrackWriter.ImportedVertexNumbers[j] = (*MeshToImportVertexMap)[j];
					}
				}

				TArrayView<TArray<FVector3f>> FramePositionsView(FramePositions);
				if (!TrackWriter.WriteAndClose(FramePositionsView))
				{
					UE_LOGF(UChaosFleshCommandsLogging, Error,
						"CreateGeometryCache - FleshComponent '%ls' failed to write vertices track to geometry cache.",
						*FleshComponent->GetName());
					continue;
				}

				UE_LOGF(UChaosFleshCommandsLogging, Log,
					"CreateGeometryCache - FleshComponent '%ls' wrote %d frames of %d vertices to geometry cache: '%ls'",
					*FleshComponent->GetName(), FramePositions.Num(), NumVertices,
					*GeometryCache->GetName());
				PackagesToSave.AddUnique(GeometryCache->GetOutermost());

#else // USE_USD_SDK && DO_USD_CACHING
				// TODO: Get points from chaos cache?
				UE_LOGF(UChaosFleshCommandsLogging, Error, "USD Caching is not supported on this platform.");
				return;
#endif // USE_USD_SDK && DO_USD_CACHING
			} // end for USkeletalMesh
		} // end for FleshComponents
	} // end for SelectedActors

	if (!PackagesToSave.IsEmpty())
	{
		// ChaosClothGenerator::SavePackage()
		constexpr bool bCheckDirty = false;
		constexpr bool bPromptToSave = true;
		FEditorFileUtils::PromptForCheckoutAndSave(
			PackagesToSave, bCheckDirty, bPromptToSave, 
			FText::FromString(FString(TEXT("Save GeometryCache"))), 
			FText::FromString(FString(TEXT("Save new GeometryCache assets."))));
	}
}
