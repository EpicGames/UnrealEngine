// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFlesh/Cmd/FleshAssetConversion.h"

#include "Logging/LogMacros.h"

//#include "ChaosFlesh/PB.h"
//#include "ChaosFlesh/GEO.h"

DEFINE_LOG_CATEGORY_STATIC(UFleshAssetConversionLogging, Log, All);

/*
TUniquePtr<FFleshCollection> ReadTetPBDeformableGeometryCollection(const FString& Filename)
	{
		TUniquePtr<FFleshCollection> Collection;
		UE_LOGF(LogChaosFlesh, Display, "Reading Path %ls", *Filename);

		IO::DeformableGeometryCollectionReader Reader(Filename);
		if (!Reader.ReadPBScene())
			return Collection;

		TArray<FVector> *Vertices = nullptr;
		TArray<FVector> FrameVertices;
		TArray<FIntVector4> *Elements = nullptr;
		TArray<FIntVector3> SurfaceElements;

		TArray<IO::DeformableGeometryCollectionReader::TetMesh*> TetMeshes = Reader.GetTetMeshes();
		for (auto* It : TetMeshes)
		{
			Elements = reinterpret_cast<TArray<FIntVector4>*>(&It->Elements);
			Vertices = &It->Points;
			UE_LOGF(LogChaosFlesh, Display, "Got %d tets.", Elements->Num());
			UE_LOGF(LogChaosFlesh, Display, "Got %d rest points.", Vertices->Num());
			break;
		}
		TArray<FIntVector4> ElemTmp;
		if (!Elements)
		{
			Elements = &ElemTmp;
			UE_LOGF(LogChaosFlesh, Display, "Got %d tets.", Elements->Num());
		}

		if (!Vertices || Vertices->Num() == 0)
		{
			TPair<int32, int32> FrameRange = Reader.ReadFrameRange();
			if (!Reader.ReadPoints(FrameRange.Key, FrameVertices))
			{
				return Collection;
			}
			Vertices = &FrameVertices;
			UE_LOGF(LogChaosFlesh, Display, "Got %d points from frame %d.", Vertices->Num(), FrameRange.Key);
		}

		GetSurfaceElements(*Elements, SurfaceElements, false);
		UE_LOGF(LogChaosFlesh, Display, "Got %d tris.", SurfaceElements.Num());

		Collection.Reset(FFleshCollection::NewFleshCollection(*Vertices, SurfaceElements, *Elements));
		return Collection;
	}

	TUniquePtr<FFleshCollection> ReadGEOFile(const FString& Filename)
	{
		TUniquePtr<FFleshCollection> Collection;
		UE_LOGF(LogChaosFlesh, Display, "Reading Path %ls", *Filename);

		TMap<FString, int32> intVars;
		TMap<FString, TArray<int32>> intVectorVars;
		TMap<FString, TArray<float>> floatVectorVars;
		TMap<FString, TPair<TArray<std::string>, TArray<int32>>> indexedStringVars;
		if (!ReadGEO(std::string(TCHAR_TO_UTF8(*Filename)), intVars, intVectorVars, floatVectorVars, indexedStringVars))
		{
			UE_LOGF(LogChaosFlesh, Display, "Failed to open GEO file: '%ls'.", *Filename);
			return Collection;
		}

		TArray<FVector> Vertices;
		TArray<FIntVector4> Elements;
		TArray<FIntVector3> SurfaceElements;

		auto fvIt = floatVectorVars.Find("position");
		if (fvIt == nullptr)
			fvIt = floatVectorVars.Find("P");
		if (fvIt != nullptr)
		{
			const TArray<float>& coords = *fvIt;
			Vertices.Reserve(coords.Num() / 3);
			for (size_t i = 0; i < coords.Num(); i += 3)
			{
				FVector pt;
				for (size_t j = 0; j < 3; j++)
					pt[j] = coords[i + j];
				Vertices.Add(pt);
			}
		}
		UE_LOGF(LogChaosFlesh, Display, "Got %d points.", Vertices.Num());

		auto ivIt = intVectorVars.Find("pointref.indices");
		if (ivIt != nullptr)
		{
			auto iIt = intVars.Find("Tetrahedron_run:startvertex");
			int32 startIndex = iIt == nullptr ? 0 : *iIt;

			iIt = intVars.Find("Tetrahedron_run:nprimitives");
			int32 numTets = iIt == nullptr ? -1 : *iIt;

			UE_LOGF(LogChaosFlesh, Display, "Tet start index: %d num tets: %d", startIndex, numTets);

			const TArray<int32>& indices = *ivIt;
			Elements.Reserve(indices.Num() / 4);
			size_t stopIndex = numTets != -1 ? startIndex + numTets * 4 : indices.Num();
			for (size_t i = startIndex; i < stopIndex; i += 4)
			{
				FIntVector4 tet;
				for (size_t j = 0; j < 4; j++)
					tet[j] = indices[i + j];
				Elements.Add(tet);
			}
			UE_LOGF(LogChaosFlesh, Display, "Got %d tets.", Elements.Num());
		}

		GetSurfaceElements(Elements, SurfaceElements, false);
		UE_LOGF(LogChaosFlesh, Display, "Got %d tris.", SurfaceElements.Num());

		Collection.Reset(FFleshCollection::NewFleshCollection(Vertices, SurfaceElements, Elements));
		return Collection;
	}

	TUniquePtr<FFleshCollection> ReadTetFile(const FString& Filename)
	{
		UE_LOGF(LogChaosFlesh, Display, "Reading Path %ls", *Filename);
		TUniquePtr<FFleshCollection> Collection;

		TArray<FVector> Vertices;
		TArray<FIntVector4> Elements;  //UE::Math::TIntVector4<int32>
		TArray<FIntVector3> SurfaceElements;

		TArray<UE::Math::TVector<float>> FloatPos;
		TArray<Chaos::TVector<int32, 4>> Tets;
		if (IO::ReadStructure<4>(Filename, FloatPos, Tets))
		{
			Vertices.SetNum(FloatPos.Num());
			for (int32 i = 0; i < FloatPos.Num(); i++)
				for (int32 j = 0; j < 3; j++)
					Vertices[i][j] = FloatPos[i][j];

			Elements.SetNum(Tets.Num());
			for (int32 i = 0; i < Tets.Num(); i++)
				for (int32 j = 0; j < 4; j++)
					Elements[i][j] = Tets[i][j];

			UE_LOGF(LogChaosFlesh, Display, "Got %d tets.", Elements.Num());
		}

		GetSurfaceElements(Elements, SurfaceElements, false);
		UE_LOGF(LogChaosFlesh, Display, "Got %d tris.", SurfaceElements.Num());

		Collection.Reset(FFleshCollection::NewFleshCollection(Vertices, SurfaceElements, Elements));
		return Collection;
	}
*/

TUniquePtr<FFleshCollection> FFleshAssetConversion::ImportTetFromFile(const FString& Filename)
{
	ensure(false);
	/*
	if (FPaths::FileExists(Filename))
	{
		if (Filename.EndsWith(".tet") || Filename.EndsWith(".tet.gz"))
			return ReadTetFile(Filename);
		else if (Filename.EndsWith(".geo"))
			return ReadGEOFile(Filename);
		UE_LOGF(LogChaosFlesh, Warning, "Unsupported file type: '%ls'.", *Filename);
	}
	else
	{
		UE_LOGF(LogChaosFlesh, Warning, "Unknown file path: '%ls'.", *Filename);
	}
	*/
	return nullptr;
}

