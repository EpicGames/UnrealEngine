// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Interfaces/IPluginManager.h"

class FChaosFleshEditorStyle final : public FSlateStyleSet
{
public:
	FChaosFleshEditorStyle() : FSlateStyleSet("ChaosFleshEditorStyle")
	{
		const FVector2D Icon16x16(16.f, 16.f);
		const FVector2D Icon64x64(64.f, 64.f);

		SetContentRoot(IPluginManager::Get().FindPlugin("ChaosFlesh")->GetBaseDir() / TEXT("Resources"));
		Set("ClassIcon.ChaosDeformableSolver", new FSlateVectorImageBrush(RootToContentDir(TEXT("ChaosSolver_16.svg")), Icon16x16));
		Set("ClassThumbnail.ChaosDeformableSolver", new FSlateVectorImageBrush(RootToContentDir(TEXT("ChaosSolver_64.svg")), Icon64x64));
		Set("ClassIcon.FleshAsset", new FSlateVectorImageBrush(RootToContentDir(TEXT("FleshAsset_16x.svg")), Icon16x16));
		Set("ClassThumbnail.FleshAsset", new FSlateVectorImageBrush(RootToContentDir(TEXT("FleshAsset_16x.svg")), Icon64x64));
		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	~FChaosFleshEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

public:

	static FChaosFleshEditorStyle& Get()
	{
		if (!Singleton.IsSet())
		{
			Singleton.Emplace();
		}
		return Singleton.GetValue();
	}

	static void Destroy()
	{
		Singleton.Reset();
	}

private:
	static TOptional<FChaosFleshEditorStyle> Singleton;
};
