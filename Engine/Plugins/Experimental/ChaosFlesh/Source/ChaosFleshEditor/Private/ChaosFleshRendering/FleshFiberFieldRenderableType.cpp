// Copyright Epic Games, Inc. All Rights Reserved.

#include "FleshFiberFieldRenderableType.h"

#include "Drawing/LineSetComponent.h"

#include "Dataflow/DataflowRenderingViewMode.h"

#include "DataflowRendering/DataflowRenderableComponents.h"
#include "DataflowRendering/DataflowRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeInstance.h"
#include "DataflowRendering/DataflowRenderableTypeRegistry.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"
#include "Dataflow/DataflowEngineUtil.h"
#include "ChaosFlesh/TetrahedralCollection.h"

#include "UObject/ObjectPtr.h"

#include "Field/FieldSystemTypes.h"

#define LOCTEXT_NAMESPACE "DataflowFleshFiberFieldRenderableType"

namespace UE::Flesh::Private
{
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FFleshFiberFieldRenderableType : public UE::Dataflow::IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FManagedArrayCollection, Collection);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(FiberField);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(UE::Dataflow::FDataflowConstruction3DViewMode);
		UE_DATAFLOW_IRENDERABLE_SETTINGS(UDataflowFleshFiberFieldRenderSettings);

	public:
		FFleshFiberFieldRenderableType()
		{
			Material = UE::Dataflow::RenderMaterial::GetDataflowLinesMaterial();
		}

	private:
		virtual bool CanRender(const UE::Dataflow::FRenderableTypeInstance& Instance) const
		{
			const FManagedArrayCollection& Collection = GetCollection(Instance);

			return Collection.HasGroup(FTetrahedralCollection::TetrahedralGroup) &&
				Collection.HasAttribute("Tetrahedron", FTetrahedralCollection::TetrahedralGroup) &&
				Collection.HasAttribute("FiberDirection", FTetrahedralCollection::TetrahedralGroup) &&
				Collection.HasGroup(FGeometryCollection::VerticesGroup);
		}

		virtual void GetPrimitiveComponents(const UE::Dataflow::FRenderableTypeInstance& Instance, UE::Dataflow::FRenderableComponents& OutComponents) const override
		{
			const UDataflowFleshFiberFieldRenderSettings* Settings = Instance.GetTypedRenderSettings<UDataflowFleshFiberFieldRenderSettings>();
			if (Settings && Settings->bVisible)
			{
				const FManagedArrayCollection& Collection = GetCollection(Instance);

				if (Collection.HasGroup(FTetrahedralCollection::TetrahedralGroup) &&
					Collection.HasAttribute("Tetrahedron", FTetrahedralCollection::TetrahedralGroup) &&
					Collection.HasAttribute("FiberDirection", FTetrahedralCollection::TetrahedralGroup) &&
					Collection.HasGroup(FGeometryCollection::VerticesGroup) &&
					Collection.HasAttribute("Vertex", FGeometryCollection::VerticesGroup))
				{
					const TManagedArray<FVector3f>& Vertex = Collection.GetAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
					const TManagedArray<FIntVector4>& Tetrahedron = Collection.GetAttribute<FIntVector4>("Tetrahedron", FTetrahedralCollection::TetrahedralGroup);
					const TManagedArray<FVector3f>& FiberDirection = Collection.GetAttribute<FVector3f>("FiberDirection", FTetrahedralCollection::TetrahedralGroup);

					TArray<FRenderableLine> RenderableLines;
					RenderableLines.SetNumUninitialized(Tetrahedron.Num());

					if (Tetrahedron.Num() >  0)
					{
						for (int32 Idx = 0; Idx < Tetrahedron.Num(); ++Idx)
						{
							FColor Color = Settings->Color.ToFColor(true);
							Color = Color.WithAlpha(255);

							int32 A = Tetrahedron[Idx].X;
							int32 B = Tetrahedron[Idx].Y;
							int32 C = Tetrahedron[Idx].Z;
							int32 D = Tetrahedron[Idx].W;

							FVector3f TetrahedronCenter = 0.25 * (Vertex[A] + Vertex[B] + Vertex[C] + Vertex[D]);

							FVector Start = FVector(TetrahedronCenter);
							FVector End = FVector(TetrahedronCenter) + FVector(FiberDirection[Idx]) * Settings->LengthScalar;

							RenderableLines[Idx] = FRenderableLine(Start, End, Color, Settings->LineThickness);
						}

						static const FName LineComponentName = TEXT("FiberField");

						if (ULineSetComponent* LineComponent = OutComponents.AddNewComponent<ULineSetComponent>(LineComponentName))
						{
							LineComponent->AddLines(RenderableLines);
							LineComponent->SetLineMaterial(Material);
						}
					}
				}
			}
		}

		UMaterialInterface* Material = nullptr;
	};

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////
	
	void RegisterFleshFiberFieldRenderableTypes()
	{
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FFleshFiberFieldRenderableType);

		static const FName FiberFieldCategoryName(TEXT("FiberField"));

		UDataflowRenderableTypeSettings::RegisterSection(
			UDataflowFleshFiberFieldRenderSettings::StaticClass(),
			"FiberField",
			LOCTEXT("FiberField", "FiberField"),
			{ FiberFieldCategoryName }
		);
	}
}

#undef LOCTEXT_NAMESPACE 