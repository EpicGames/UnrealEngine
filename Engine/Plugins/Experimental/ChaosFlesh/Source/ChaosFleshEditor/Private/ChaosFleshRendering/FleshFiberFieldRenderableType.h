// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataflowRendering/DataflowRenderableTypeSettings.h"
#include "Math/Color.h"

#include "FleshFiberFieldRenderableType.generated.h"

UCLASS(MinimalAPI, AutoExpandCategories="")
class UDataflowFleshFiberFieldRenderSettings : public UDataflowRenderableTypeSettings
{
	GENERATED_BODY()
public:
	/** Controls the visibility of the fiber field */
	UPROPERTY(EditAnywhere, Category = "FiberField")
	bool bVisible = false;

	/** Color of the fibers */
	UPROPERTY(EditAnywhere, Category = "FiberField", meta = (EditCondition = "bVisible"))
	FLinearColor Color = FLinearColor::Red;

	/** Line thickness of the fiber field vectors */
	UPROPERTY(EditAnywhere, Category = "FiberField", meta = (EditCondition = "bVisible"))
	float LineThickness = 0.5f;

	/** Scalar to control the length of the fiber field vectors */
	UPROPERTY(EditAnywhere, Category = "FiberField", meta = (EditCondition = "bVisible"))
	float LengthScalar = 1.0f;
};

namespace UE::Flesh::Private
{
	void RegisterFleshFiberFieldRenderableTypes();
}
