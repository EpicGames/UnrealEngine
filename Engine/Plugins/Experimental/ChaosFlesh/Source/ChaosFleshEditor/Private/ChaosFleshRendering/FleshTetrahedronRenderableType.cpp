// Copyright Epic Games, Inc. All Rights Reserved.

#include "FleshTetrahedronRenderableType.h"

#include "Drawing/LineSetComponent.h"

#include "Dataflow/DataflowRenderingViewMode.h"

#include "DataflowRendering/DataflowRenderableComponents.h"
#include "DataflowRendering/DataflowRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeInstance.h"
#include "DataflowRendering/DataflowRenderableTypeRegistry.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/DataflowEngineUtil.h"

#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"

#include "UObject/ObjectPtr.h"
#include "Components/StaticMeshComponent.h"

#include "ChaosFlesh/TetrahedralCollection.h"

#define LOCTEXT_NAMESPACE "DataflowTetrahedronRenderableType"

namespace UE::Flesh::Private
{
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FFleshTetrahedronRenderableType : public UE::Dataflow::IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FManagedArrayCollection, Collection);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(TetraHedron);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(UE::Dataflow::FDataflowConstruction3DViewMode);
		UE_DATAFLOW_IRENDERABLE_SETTINGS(UDataflowFleshTetrahedronRenderSettings);

	public:
		FFleshTetrahedronRenderableType()
		{
			Material = UE::Dataflow::RenderMaterial::GetDataflowLinesMaterial();
		}

	private:
		virtual bool CanRender(const UE::Dataflow::FRenderableTypeInstance& Instance) const
		{
			const FManagedArrayCollection& Collection = GetCollection(Instance);
			return Collection.HasGroup(FTetrahedralCollection::TetrahedralGroup) &&
				Collection.HasAttribute(FTetrahedralCollection::TetrahedronAttribute, FTetrahedralCollection::TetrahedralGroup) &&
				Collection.HasAttribute(FTetrahedralCollection::TetrahedronStartAttribute, FGeometryCollection::GeometryGroup) &&
				Collection.HasAttribute(FTetrahedralCollection::TetrahedronCountAttribute, FGeometryCollection::GeometryGroup);
		}

		virtual void GetPrimitiveComponents(const UE::Dataflow::FRenderableTypeInstance& Instance, UE::Dataflow::FRenderableComponents& OutComponents) const override
		{
			const UDataflowFleshTetrahedronRenderSettings* Settings = Instance.GetTypedRenderSettings<UDataflowFleshTetrahedronRenderSettings>();
			if (!Settings || !Settings->bVisible)
			{
				return;
			}

			if (Settings)
			{
				using namespace UE::Dataflow;

				static FName BaseParentName = TEXT("Tetrahedrons");
				UPrimitiveComponent* ParentComponent = OutComponents.AddNewComponent<UStaticMeshComponent>(BaseParentName);
				if (!ParentComponent)
				{
					return;
				}

				const FManagedArrayCollection& Collection = GetCollection(Instance);

				if (Collection.HasGroup(FTetrahedralCollection::TetrahedralGroup) &&
					Collection.HasAttribute(FTetrahedralCollection::TetrahedronAttribute, FTetrahedralCollection::TetrahedralGroup) &&
					Collection.HasAttribute(FTetrahedralCollection::TetrahedronStartAttribute, FGeometryCollection::GeometryGroup) &&
					Collection.HasAttribute(FTetrahedralCollection::TetrahedronCountAttribute, FGeometryCollection::GeometryGroup) &&					
					Collection.HasAttribute(FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup) &&
					Collection.HasAttribute(FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup) &&
					Collection.HasAttribute("Vertex", FGeometryCollection::VerticesGroup) &&
					Collection.HasAttribute("VertexStart", FGeometryCollection::GeometryGroup) &&
					Collection.HasAttribute("VertexCount", FGeometryCollection::GeometryGroup) &&
					Collection.HasAttribute("TetrahedronStart", FGeometryCollection::GeometryGroup) &&
					Collection.HasAttribute("TetrahedronCount", FGeometryCollection::GeometryGroup) &&
					Collection.HasAttribute("Tetrahedron", "Tetrahedral") &&
					Collection.HasAttribute("TransformIndex", FGeometryCollection::GeometryGroup))
				{
					auto ToD = [](FVector3f V) { return FVector3d(V.X, V.Y, V.Z); };
					auto ToF = [](FVector3d V) { return FVector3f(V.X, V.Y, V.Z); };

					const TManagedArray<int32>& Parents = Collection.GetAttribute<int32>(FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup);
					const TManagedArray<FTransform3f>& Transforms = Collection.GetAttribute<FTransform3f>(FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup);
					const TManagedArray<FVector3f>& Vertex = Collection.GetAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
					const TManagedArray<int32>& VertexStart = Collection.GetAttribute<int32>("VertexStart", FGeometryCollection::GeometryGroup);
					const TManagedArray<int32>& VertexCount = Collection.GetAttribute<int32>("VertexCount", FGeometryCollection::GeometryGroup);
					const TManagedArray<int32>& TetrahedronStart = Collection.GetAttribute<int32>("TetrahedronStart", FGeometryCollection::GeometryGroup);
					const TManagedArray<int32>& TetrahedronCount = Collection.GetAttribute<int32>("TetrahedronCount", FGeometryCollection::GeometryGroup);
					const TManagedArray<FIntVector4>& Tetrahedrons = Collection.GetAttribute<FIntVector4>("Tetrahedron", "Tetrahedral");
					const TManagedArray<int32>& GeometryToTransformIndex = Collection.GetAttribute<int32>("TransformIndex", FGeometryCollection::GeometryGroup);

					TArray<FTransform> RootSpaceTransforms;
					GeometryCollectionAlgo::GlobalMatrices(Transforms, Parents, RootSpaceTransforms);

					for (int GeometryIdx = 0; GeometryIdx < Collection.NumElements(FGeometryCollection::GeometryGroup); GeometryIdx++)
					{
						TArray<FVector3f> VerticesInCollectionSpace; VerticesInCollectionSpace.AddUninitialized(VertexCount[GeometryIdx]);
						TArray<FVector3f> SplitVertices;

						const int32 TransformIndex = GeometryToTransformIndex[GeometryIdx];

						// Transform vertices to Collection space
						const int32 GlobalVertexOffset = VertexStart[GeometryIdx];
						for (int32 LocalVertexIdx = 0; LocalVertexIdx < VertexCount[GeometryIdx]; ++LocalVertexIdx)
						{
							VerticesInCollectionSpace[LocalVertexIdx] = ToF(RootSpaceTransforms[TransformIndex].TransformPosition(ToD(Vertex[GlobalVertexOffset + LocalVertexIdx])));
						}

						if (TetrahedronCount[GeometryIdx] > 0)
						{
							TArray<FIntVector4> Tetras; Tetras.AddUninitialized(TetrahedronCount[GeometryIdx]);

							const int32 GlobalTetrahedronOffset = TetrahedronStart[GeometryIdx];
							for (int32 LocalTetrahedronIdx = 0; LocalTetrahedronIdx < TetrahedronCount[GeometryIdx]; ++LocalTetrahedronIdx)
							{
								const FIntVector4& Tetra = Tetrahedrons[GlobalTetrahedronOffset + LocalTetrahedronIdx];
								const int32 VtxStart = SplitVertices.Num();

								SplitVertices.Add(VerticesInCollectionSpace[Tetra[0] - GlobalVertexOffset]);
								SplitVertices.Add(VerticesInCollectionSpace[Tetra[1] - GlobalVertexOffset]);
								SplitVertices.Add(VerticesInCollectionSpace[Tetra[2] - GlobalVertexOffset]);
								SplitVertices.Add(VerticesInCollectionSpace[Tetra[3] - GlobalVertexOffset]);

								Tetras[LocalTetrahedronIdx] = { VtxStart, VtxStart + 1, VtxStart + 2, VtxStart + 3 };
							}

							static const FName LineComponentName = TEXT("Tetrahedrons");
							constexpr int32 NumVertices = 4;
							const int32 NumTetras = Tetras.Num();

							TArray<FVector3f> Vertices;
							Vertices.AddUninitialized(NumVertices);

							TArray<FVector> LineStarts;
							TArray<FVector> LineEnds;

							LineStarts.Init(FVector(0.f), NumTetras * 6);
							LineEnds.Init(FVector(0.f), NumTetras * 6);

							if (ULineSetComponent* LineComponent = OutComponents.AddNewComponent<ULineSetComponent>(LineComponentName, ParentComponent))
							{
								for (int32 Idx = 0; Idx < NumTetras; ++Idx)
								{
									const FIntVector4& Indices = Tetras[Idx];

									Vertices[0] = SplitVertices[Indices[0]];
									Vertices[1] = SplitVertices[Indices[1]];
									Vertices[2] = SplitVertices[Indices[2]];
									Vertices[3] = SplitVertices[Indices[3]];

									LineStarts[Idx * 6 + 0] = (FVector(Vertices[0])); LineEnds[Idx * 6 + 0] = (FVector(Vertices[1]));
									LineStarts[Idx * 6 + 1] = (FVector(Vertices[0])); LineEnds[Idx * 6 + 1] = (FVector(Vertices[2]));
									LineStarts[Idx * 6 + 2] = (FVector(Vertices[0])); LineEnds[Idx * 6 + 2] = (FVector(Vertices[3]));
									LineStarts[Idx * 6 + 3] = (FVector(Vertices[1])); LineEnds[Idx * 6 + 3] = (FVector(Vertices[2]));
									LineStarts[Idx * 6 + 4] = (FVector(Vertices[1])); LineEnds[Idx * 6 + 4] = (FVector(Vertices[3]));
									LineStarts[Idx * 6 + 5] = (FVector(Vertices[2])); LineEnds[Idx * 6 + 5] = (FVector(Vertices[3]));
								}

								LineComponent->AddLines(LineStarts, LineEnds, Settings->LineColor, Settings->LineThickness);
								LineComponent->SetLineMaterial(Material);
							}
						}
					}
				}

			}
		}

		UMaterialInterface* Material = nullptr;
	};


	///////////////////////////////////////////////////////////////////////////////////////////////////////////////
	
	void RegisterFleshTetrahedronRenderableTypes()
	{
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FFleshTetrahedronRenderableType);

		static const FName TetrahedronsCategoryName(TEXT("Tetrahedrons"));

		UDataflowRenderableTypeSettings::RegisterSection(
			UDataflowFleshTetrahedronRenderSettings::StaticClass(),
			"Tetrahedrons",
			LOCTEXT("Tetrahedrons", "Tetrahedrons"),
			{ TetrahedronsCategoryName }
		);
	}
}

#undef LOCTEXT_NAMESPACE 
