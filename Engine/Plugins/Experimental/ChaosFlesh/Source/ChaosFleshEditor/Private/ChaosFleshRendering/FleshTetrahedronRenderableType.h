// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataflowRendering/DataflowRenderableTypeSettings.h"
#include "Math/Color.h"

#include "FleshTetrahedronRenderableType.generated.h"

UCLASS(MinimalAPI, AutoExpandCategories="")
class UDataflowFleshTetrahedronRenderSettings : public UDataflowRenderableTypeSettings
{
	GENERATED_BODY()
public:
	/** Controls the visibility of the tetrahedrons */
	UPROPERTY(EditAnywhere, Category = "Tetrahedrons")
	bool bVisible = false;

	/** Thickness of lines displaying the tetrahedrons */
	UPROPERTY(EditAnywhere, Category = "Tetrahedrons", meta = (EditCondition = "bVisible"))
	float LineThickness = 1.0f;

	/** Color of lines displaying the tetrahedrons */
	UPROPERTY(EditAnywhere, Category = "Tetrahedrons", meta = (EditCondition = "bVisible"))
	FColor LineColor = FColor::Yellow;
};

namespace UE::Flesh::Private
{
	void RegisterFleshTetrahedronRenderableTypes();
}
