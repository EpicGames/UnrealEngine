// Copyright Epic Games, Inc. All Rights Reserved.

#include "FleshVectorFieldRenderableType.h"

#include "Drawing/LineSetComponent.h"

#include "Dataflow/DataflowRenderingViewMode.h"

#include "DataflowRendering/DataflowRenderableComponents.h"
#include "DataflowRendering/DataflowRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeInstance.h"
#include "DataflowRendering/DataflowRenderableTypeRegistry.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"
#include "Dataflow/DataflowEngineUtil.h"

#include "UObject/ObjectPtr.h"

#include "Field/FieldSystemTypes.h"

#define LOCTEXT_NAMESPACE "DataflowFleshVectorFieldRenderableType"

namespace UE::Flesh::Private
{
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FFleshVectorFieldRenderableType : public UE::Dataflow::IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FFieldCollection, Collection);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(VectorField);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(UE::Dataflow::FDataflowConstruction3DViewMode);
		UE_DATAFLOW_IRENDERABLE_SETTINGS(UDataflowFleshVectorFieldRenderSettings);

	public:
		FFleshVectorFieldRenderableType()
		{
			Material = UE::Dataflow::RenderMaterial::GetDataflowLinesMaterial();
		}

	private:
		virtual bool CanRender(const UE::Dataflow::FRenderableTypeInstance& Instance) const
		{
			const FFieldCollection& Collection = GetCollection(Instance);
			return Collection.HasGroup("VectorField") &&
				Collection.HasAttribute("Start", "VectorField") &&
				Collection.HasAttribute("End", "VectorField") &&
				Collection.HasAttribute("Color", "VectorField");
		}

		virtual void GetPrimitiveComponents(const UE::Dataflow::FRenderableTypeInstance& Instance, UE::Dataflow::FRenderableComponents& OutComponents) const override
		{
			const UDataflowFleshVectorFieldRenderSettings* Settings = Instance.GetTypedRenderSettings<UDataflowFleshVectorFieldRenderSettings>();

			if (Settings && Settings->bVisible)
			{
				const FFieldCollection& Collection = GetCollection(Instance);

				if (Collection.HasGroup("VectorField") &&
					Collection.HasAttribute("Start", "VectorField") &&
					Collection.HasAttribute("End", "VectorField") &&
					Collection.HasAttribute("Color", "VectorField"))
				{
					TArray<TPair<FVector3f, FVector3f>> VectorField = Collection.GetVectorField();
					TArray<FLinearColor> Colors = Collection.GetVectorColor();

					TArray<FRenderableLine> RenderableLines;
					RenderableLines.SetNumUninitialized(VectorField.Num());

					if (VectorField.Num() == Colors.Num())
					{
						for (int32 Idx = 0; Idx < VectorField.Num(); ++Idx)
						{
							FColor Color = Colors[Idx].ToFColor(true);
							Color = Color.WithAlpha(255);

							RenderableLines[Idx] = FRenderableLine(FVector(VectorField[Idx].Key), FVector(VectorField[Idx].Value + (VectorField[Idx].Value - VectorField[Idx].Key) * Settings->LengthScalar), Color, Settings->LineThickness);
						}

						static const FName LineComponentName = TEXT("VectorField");

						if (ULineSetComponent* LineComponent = OutComponents.AddNewComponent<ULineSetComponent>(LineComponentName))
						{
							LineComponent->AddLines(RenderableLines);
							LineComponent->SetLineMaterial(Material);
						}
					}
				}
			}
		}

		UMaterialInterface* Material = nullptr;
	};


	///////////////////////////////////////////////////////////////////////////////////////////////////////////////
	
	void RegisterFleshVectorFieldRenderableTypes()
	{
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FFleshVectorFieldRenderableType);

		static const FName VectorFieldCategoryName(TEXT("VectorField"));

		UDataflowRenderableTypeSettings::RegisterSection(
			UDataflowFleshVectorFieldRenderSettings::StaticClass(),
			"VectorField",
			LOCTEXT("VectorField", "VectorField"),
			{ VectorFieldCategoryName }
		);
	}
}

#undef LOCTEXT_NAMESPACE 

