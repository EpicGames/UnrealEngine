// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataflowRendering/DataflowRenderableTypeSettings.h"
#include "Math/Color.h"

#include "FleshVectorFieldRenderableType.generated.h"

UCLASS(MinimalAPI, AutoExpandCategories="")
class UDataflowFleshVectorFieldRenderSettings : public UDataflowRenderableTypeSettings
{
	GENERATED_BODY()
public:
	/** Controls the visibility of the vector field */
	UPROPERTY(EditAnywhere, Category = "VectorField")
	bool bVisible = false;

	/** Thickness of lines displaying the vector field */
	UPROPERTY(EditAnywhere, Category = "VectorField", meta = (EditCondition = "bVisible"))
	float LineThickness = 1.0f;

	/** Scalar to modify the length of the vectors in the vector field */
	UPROPERTY(EditAnywhere, Category = "VectorField", meta = (EditCondition = "bVisible"))
	float LengthScalar = 1.0f;
};

namespace UE::Flesh::Private
{
	void RegisterFleshVectorFieldRenderableTypes();
}
