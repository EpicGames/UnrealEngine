// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"

#include "FleshAssetFactory.generated.h"

class UFleshAsset;
class UFleshComponent;

/**
* Factory for FleshAsset
*/
UCLASS()
class CHAOSFLESHEDITOR_API UFleshAssetFactory : public UFactory
{
    GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};


