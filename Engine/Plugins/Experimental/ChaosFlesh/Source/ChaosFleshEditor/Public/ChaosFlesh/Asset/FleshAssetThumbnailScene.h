// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ThumbnailHelpers.h"

class AFleshActor;
class UFleshAsset;

class FFleshAssetThumbnailScene : public FThumbnailPreviewScene
{
public:
	/** Constructor */
	FFleshAssetThumbnailScene();

	/** Sets the geometry collection to use in the next CreateView() */
	void SetFleshAsset(UFleshAsset* FleshAsset);

protected:
	// FThumbnailPreviewScene implementation
	virtual void GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const override;

private:
	/** The actor used to display all geometry collection thumbnails */
	AFleshActor* PreviewActor;
};
