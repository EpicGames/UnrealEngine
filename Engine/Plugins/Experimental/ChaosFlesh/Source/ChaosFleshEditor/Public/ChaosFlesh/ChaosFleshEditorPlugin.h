// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Toolkits/IToolkit.h"

class ISlateStyle;
class FSlateStyleSet;
class FAssetEditorToolkit;
class FAssetTypeActions_FleshAsset;
class FAssetTypeActions_ChaosDeformableSolver;
class FFleshEditorToolkit;

/**
 * The public interface to this module
 */
class IChaosFleshEditorPlugin : public IModuleInterface
{
public:
	virtual void StartupModule();
	virtual void ShutdownModule();

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IChaosFleshEditorPlugin& Get()
	{
		return FModuleManager::LoadModuleChecked< IChaosFleshEditorPlugin >( "ChaosFleshEditorPlugin" );
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "ChaosFleshEditorPlugin" );
	}

	TSharedPtr<FSlateStyleSet> GetStyleSet() { return StyleSet; }

	static FName GetEditorStyleName();
	static const ISlateStyle* GetEditorStyle();

private:
	void RegisterMenus();

private:
	TArray<IConsoleObject*> EditorCommands;

	// Asset actions for new asset types
	FAssetTypeActions_ChaosDeformableSolver* ChaosDeformableSolverAssetActions;

	// Styleset for flesh tool brushes/fonts etc.
	TSharedPtr<FSlateStyleSet> StyleSet;

	FDelegateHandle DataflowMenusHandle;
};

