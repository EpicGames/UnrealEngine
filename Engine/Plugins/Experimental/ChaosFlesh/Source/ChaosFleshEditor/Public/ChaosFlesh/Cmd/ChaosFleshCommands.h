// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Engine/World.h"

/**
* The public interface to this module
*/
class CHAOSFLESHEDITOR_API FChaosFleshCommands
{
public:


	/**
	*  Import file. 
	*/
	static void ImportFile(const TArray<FString>& Args, UWorld* World);

	/**
	* Commmand invoked from "FChaosDeformableCommands.FindHighAspectRatioTetrahedra", uses
	* the selected FleshComponent and outputs tetrahedra indices to the log.
	* @param Args - Supported arguments:
	*	'MaxAR <float>' selects tetrahedra with aspect ratio greater than this value.
	*	'MinVol <float>' selects tetrahedra with (signed) volume less than this value.
	*	'XCoordGT <float>', 'YCoordGT <float>', 'ZCoordGT <float>' selects tetrahedra with all vertices greater than these values.
	*	'XCoordLT <float>', 'YCoordLT <float>', 'ZCoordLT <float>' selects tetrahedra with all vertices less than these values.
	*   'HideTets' adds selected tets to the selected flesh component's list of tets to skip drawing.
	* @param World
	*/
	static void FindQualifyingTetrahedra(const TArray<FString>& Args, UWorld* World);

	/**
	* Create GeometryCache asset from a cached flesh simulation.
	* 
	* Command invoked from "FChaosDeformableCommands.CreateGeometryCache", uses the selected 
	* Actor's FleshComponent and SkeletalMeshComponent, optionally along with a Chaos Cache Manager.
	* Deforms the SkeletalMesh's import geometry by the cached simulation results, and stores that
	* surface in a new GeometryCache asset.  The rest collection asset on the flesh component(s)
	* needs to contain deformer bindings for the import geometry.
	* @param Args - Supported arguments:
	*   'UsdFile </path/to/file.usd>' overrides which usd file to use, rather than inferring it from a chaos cache manager.
	*   'FrameRate 24' overrides the default 24 FPS frame rate for the output cache samples.
	*   'MaxNumFrames <int>' clamps the number of frames to MaxNumFrames.
	*/
	static void CreateGeometryCache(const TArray<FString>& Args, UWorld* World);

};