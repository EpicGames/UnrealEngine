// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Engine/World.h"
#include "ChaosFlesh/FleshCollection.h"
#include "Templates/UniquePtr.h"


/**
* The public interface to this module
*/
class CHAOSFLESHEDITOR_API FFleshAssetConversion
{
public:

	/**
	* Supports reading .tet and .geo (compatible with version 19).
	* Currently disabled.
	*/
	static TUniquePtr<FFleshCollection> ImportTetFromFile(const FString& Filename);


	//TUniquePtr<FFleshCollection> ReadGEOFile(const FString& Filename);
	//TUniquePtr<FFleshCollection> ReadTetFile(const FString& Filename); // uncompressed tet
	//TUniquePtr<FFleshCollection> ReadTetPBDeformableGeometryCollection(const FString& Filename)


};