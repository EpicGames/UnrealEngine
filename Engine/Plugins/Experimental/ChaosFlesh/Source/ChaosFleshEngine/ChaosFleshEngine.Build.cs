// Copyright Epic Games, Inc. All Rights Reserved.

using System.Linq;

namespace UnrealBuildTool.Rules
{
	public class ChaosFleshEngine : ModuleRules
	{
        public ChaosFleshEngine(ReadOnlyTargetRules Target) : base(Target)
		{
			SetupModulePhysicsSupport(Target);

			PublicIncludePaths.AddRange(
				new string[] {
					// ... add public include paths required here ...
				}
				);

			PrivateIncludePaths.AddRange(
				new string[] {
					// ... add other private include paths required here ...
				}
				);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"ProceduralMeshComponent",
					"DataflowCore",
					"DataflowEngine",
					"DataflowSimulation",
					// ... add other public dependencies that you statically link with here ...
				}
				);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ClothingSystemRuntimeCommon",
					"ComputeFramework",
					"CoreUObject",
					"Chaos",
					"ChaosCaching",
					"ChaosFlesh",
					"DataflowCore",
					"DataflowEngine",
					"DataflowSimulation",
					"Engine",
					"FieldSystemEngine",
					"NetCore",
					"OptimusCore",
					"Projects",
					"RenderCore",
                    "RHI",
					"Renderer",
				}
				);

			if (Target.Platform == UnrealTargetPlatform.Win64)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"ChaosCachingUSD",
						"UnrealUSDWrapper",
						"USDClasses",
						"USDUtilities",
					});
			}
			else
			{
				PrivateDefinitions.Add("USE_USD_SDK=0");
			}

			DynamicallyLoadedModuleNames.AddRange(
				new string[]
				{
					// ... add any modules that your module loads dynamically here ...
				}
				);

			PrivateDefinitions.Add("CHAOS_INCLUDE_LEVEL_1=1");

			bool bIsUsdSdkEnabled = UnrealUSDWrapper.CheckAndSetupUsdSdk(Target, this);

			// Build flavor selection copied from UnrealUSDWrapper, then modified.
			// Currently only Win64 is supported.
			if (bIsUsdSdkEnabled && (Target.Type == TargetType.Editor && Target.Platform == UnrealTargetPlatform.Win64))
			{
				bUseRTTI = true;
				PublicDefinitions.Add("DO_USD_CACHING=1");
			}
			else
			{
				PublicDefinitions.Add("DO_USD_CACHING=0");
			}
		}
	}
}
