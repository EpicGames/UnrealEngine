// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosCache/FleshComponentCacheAdapter.h"

#include "Chaos/ChaosCache.h"
#include "Chaos/CacheManagerActor.h"
#include "Chaos/PBDEvolution.h"
#include "ChaosFlesh/ChaosDeformableSolverComponent.h"
#include "ChaosFlesh/FleshComponent.h"

#if USE_USD_SDK && DO_USD_CACHING

#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"

#include "ChaosCachingUSD/Operations.h"
#include "USDConversionUtils.h" // for GetPrimPathForObject()
#include "USDMemory.h"

#endif // USE_USD_SDK && DO_USD_CACHING

DEFINE_LOG_CATEGORY(LogChaosFleshCache)

namespace Chaos
{
	FFleshCacheAdapterCVarParams CVarParams;

#if USE_USD_SDK && DO_USD_CACHING

	FAutoConsoleVariableRef CVarDeformableFleshCacheWriteBinaryUSD(
		TEXT("p.Chaos.Caching.USD.WriteBinary"),
		CVarParams.bWriteBinary,
		TEXT("Write binary (usdc) cache files. [def: true]"));
	FAutoConsoleVariableRef CVarDeformableFleshCacheNoClobberUSD(
		TEXT("p.Chaos.Caching.USD.NoClobber"),
		CVarParams.bNoClobber,
		TEXT("Rename rather than over write existing cach files. [def: true]"));
	FAutoConsoleVariableRef CVarDeformableFleshCacheSaveFrequency(
		TEXT("p.Chaos.Caching.USD.SaveFrequency"),
		CVarParams.SaveFrequency,
		TEXT("Interval in frames to flush USD data to disk. 2 saves every other frame, 1 saves every frame, 0 caches in memory until complete. [def: 10]"));

#endif // USE_USD_SDK && DO_USD_CACHING

	FFleshCacheAdapter::~FFleshCacheAdapter()
	{
#if USE_USD_SDK && DO_USD_CACHING
		for(TPair<UPrimitiveComponent*,FPrimitiveStage>& PrimitiveStage : PrimitiveStages)
		{
			if(PrimitiveStage.Value.MonolithStage)
			{
				UE::ChaosCachingUSD::CloseStage(PrimitiveStage.Value.MonolithStage);
			}
		}
#endif // USE_USD_SDK && DO_USD_CACHING
	}

	FComponentCacheAdapter::SupportType FFleshCacheAdapter::SupportsComponentClass(UClass* InComponentClass) const
	{
		const UClass* Desired = GetDesiredClass();
		if(InComponentClass == Desired)
		{
			return FComponentCacheAdapter::SupportType::Direct;
		}
		else if(InComponentClass->IsChildOf(Desired))
		{
			return FComponentCacheAdapter::SupportType::Derived;
		}

		return FComponentCacheAdapter::SupportType::None;
	}

	UClass* FFleshCacheAdapter::GetDesiredClass() const
	{
		return UFleshComponent::StaticClass();
	}

	uint8 FFleshCacheAdapter::GetPriority() const
	{
		return EngineAdapterPriorityBegin;
	}

	void FFleshCacheAdapter::Record_PostSolve(UPrimitiveComponent* InComponent, const FTransform& InRootTransform, FPendingFrameWrite& OutFrame, Chaos::FReal InTime) const
	{
		if( FDeformableSolver* Solver = GetDeformableSolver(InComponent))
		{
			FDeformableSolver::FPhysicsThreadAccess PhysicsThreadAccess(Solver, Chaos::Softs::FPhysicsThreadAccessor());

			if (FEvolution* Evolution = PhysicsThreadAccess.GetEvolution())
			{
				const FParticles& Particles = Evolution->Particles();

				const uint32 NumParticles = Particles.Size();
				//We always write a cache even if there are no particles
				{
#if USE_USD_SDK && DO_USD_CACHING
					FScopedUsdAllocs UsdAllocs; // Use USD memory allocator

					FPrimitiveStage& PrimitiveStage = PrimitiveStages.FindOrAdd(InComponent);
					const Chaos::FRange ParticleRange = GetParticleRange(InComponent, NumParticles);

					// Update time range.
					PrimitiveStage.MinTime = FMath::Min(InTime, PrimitiveStage.MinTime);
					PrimitiveStage.MaxTime = FMath::Max(InTime, PrimitiveStage.MaxTime);

					if (PrimitiveStage.MonolithStage)
					{
						if (!UE::ChaosCachingUSD::WritePoints(PrimitiveStage.MonolithStage, PrimitiveStage.PrimPath, InTime, Particles.XArray(), Particles.GetV(), ParticleRange))
						{
							UE_LOGF(LogChaosFleshCache, Error,
								"Failed to write points '%ls' at time %g to file: '%ls'",
								*PrimitiveStage.PrimPath, InTime, *PrimitiveStage.MonolithStage.GetRootLayer().GetDisplayName());
							return;
						}

						// Write muscle activation
						const TArray<float> ParticleMuscleActivation = PhysicsThreadAccess.GetParticleMuscleActivation();
						if (!UE::ChaosCachingUSD::WriteActivations(PrimitiveStage.MonolithStage, PrimitiveStage.PrimPath, InTime, ParticleMuscleActivation, ParticleRange))
						{
							UE_LOGF(LogChaosFleshCache, Error,
								"Failed to write activations '%ls' at time %g to file: '%ls'",
								*PrimitiveStage.PrimPath, InTime, *PrimitiveStage.MonolithStage.GetRootLayer().GetDisplayName());
							return;
						}

						uint64 NumTimeSamples = UE::ChaosCachingUSD::GetNumTimeSamples(PrimitiveStage.MonolithStage, PrimitiveStage.PrimPath, UE::ChaosCachingUSD::GetPointsAttrName());
						if (CVarParams.SaveFrequency >= 1 && NumTimeSamples % CVarParams.SaveFrequency == 0)
						{
							if (!UE::ChaosCachingUSD::SaveStage(PrimitiveStage.MonolithStage, PrimitiveStage.MinTime, PrimitiveStage.MaxTime))
							{
								UE_LOGF(LogChaosFleshCache, Error,
									"Failed to save file: '%ls'",
									*PrimitiveStage.MonolithStage.GetRootLayer().GetDisplayName());
							}
						}
					}
#else // USE_USD_SDK && DO_USD_CACHING
					const Softs::FSolverVec3* ParticleXs = &Particles.X(0);
					const Softs::FSolverVec3* ParticleVs = &Particles.V(0);

					TArray<float> PendingVX, PendingVY, PendingVZ, PendingPX, PendingPY, PendingPZ;
					TArray<int32>& PendingID = OutFrame.PendingChannelsIndices;

					PendingID.Reserve(NumParticles);
					PendingVX.Reserve(NumParticles);
					PendingVY.Reserve(NumParticles);
					PendingVZ.Reserve(NumParticles);
					PendingPX.Reserve(NumParticles);
					PendingPY.Reserve(NumParticles);
					PendingPZ.Reserve(NumParticles);

					for (uint32 ParticleIndex = 0; ParticleIndex < NumParticles; ++ParticleIndex)
					{
						const Softs::FSolverVec3& ParticleV = ParticleVs[ParticleIndex];
						const Softs::FSolverVec3& ParticleX = ParticleXs[ParticleIndex];

						// Adding the vertices relative position to the particle write datas
						PendingID.Add(ParticleIndex);
						PendingVX.Add(ParticleV.X);
						PendingVY.Add(ParticleV.Y);
						PendingVZ.Add(ParticleV.Z);

						PendingPX.Add(ParticleX.X);
						PendingPY.Add(ParticleX.Y);
						PendingPZ.Add(ParticleX.Z);
					}

					OutFrame.PendingChannelsData.Add(VelocityXName, PendingVX);
					OutFrame.PendingChannelsData.Add(VelocityYName, PendingVY);
					OutFrame.PendingChannelsData.Add(VelocityZName, PendingVZ);
					OutFrame.PendingChannelsData.Add(PositionXName, PendingPX);
					OutFrame.PendingChannelsData.Add(PositionYName, PendingPY);
					OutFrame.PendingChannelsData.Add(PositionZName, PendingPZ);

#endif // USE_USD_SDK && DO_USD_CACHING
				}
			}
		}
	}

	void FFleshCacheAdapter::LoadCacheAtTime(UPrimitiveComponent* PrimitiveComponent, Chaos::FReal TargetTime, const int32 NumParticles, const bool bNeedsRange,
		const TFunction<void(const int32, const int32, const FVector3f&, const FVector3f&, const float Activation)>& LoadFunction) const
	{
#if USE_USD_SDK && DO_USD_CACHING

		const FPrimitiveStage& PrimitiveStage = PrimitiveStages.FindOrAdd(PrimitiveComponent);
		if (PrimitiveStage.MonolithStage)
		{
			// GetParticleRange only called in play/record mode (not load), so PT only
			const Chaos::FRange ParticleRange = bNeedsRange ? GetParticleRange(PrimitiveComponent, NumParticles) : Chaos::FRange(0,NumParticles);
			const int32 NumComponentParticles = ParticleRange.Count;
			
			FScopedUsdAllocs UEAllocs; // Use USD memory allocator
			pxr::VtArray<pxr::GfVec3f> Points0, Points1;
			pxr::VtArray<pxr::GfVec3f> Vels0, Vels1;
			pxr::VtArray<float> Activations0, Activations1;

			double Prev = -TNumericLimits<double>::Max();
			double Next = -TNumericLimits<double>::Max();
			double PrevV = -TNumericLimits<double>::Max();
			double NextV = -TNumericLimits<double>::Max();
			
			if (!UE::ChaosCachingUSD::GetBracketingTimeSamples(
					PrimitiveStage.MonolithStage, PrimitiveStage.PrimPath, UE::ChaosCachingUSD::GetPointsAttrName(), TargetTime, &Prev, &Next) ||
				!UE::ChaosCachingUSD::GetBracketingTimeSamples(
					PrimitiveStage.MonolithStage, PrimitiveStage.PrimPath, UE::ChaosCachingUSD::GetVelocityAttrName(), TargetTime, &PrevV, &NextV) ||
				Prev != PrevV ||
				Next != NextV)
			{
				UE_LOGF(LogChaosFleshCache, Error,
					"Inconsistent bracketing time samples for attributes '%ls' and '%ls' at frame %g from file: '%ls'",
					*UE::ChaosCachingUSD::GetPointsAttrName(),
					*UE::ChaosCachingUSD::GetVelocityAttrName(),
					TargetTime, *PrimitiveStage.MonolithStage.GetRootLayer().GetDisplayName());
				return;
			}

			if (!UE::ChaosCachingUSD::ReadPoints(PrimitiveStage.MonolithStage, PrimitiveStage.PrimPath, Prev, Points0, Vels0) ||
				Points0.size() != Vels0.size())
			{
				UE_LOGF(LogChaosFleshCache, Error,
					"Failed to read points '%ls' at time %g from file: '%ls'",
					*PrimitiveStage.PrimPath, Prev, *PrimitiveStage.MonolithStage.GetRootLayer().GetDisplayName());
				return;
			}

			// load muscle activation
			if (!UE::ChaosCachingUSD::ReadMuscleActivation(PrimitiveStage.MonolithStage, PrimitiveStage.PrimPath, Prev, Activations0) ||
				Points0.size() != Activations0.size())
			{
				Activations0.assign(Points0.size(), -1.f);
			}

			int32 NumCachedParticles = static_cast<int32>(Points0.size());
			if (NumCachedParticles > NumComponentParticles)
			{
				// Cached particles doesn't match solver particles.  Truncate.
				NumCachedParticles = NumComponentParticles;
			}

			// < time range start, > time range end, or exact hit
			if (FMath::IsNearlyEqual(Prev, Next))
			{
				// Directly set the result of the cache into the solver particles
				for (int32 CachedIndex = 0; CachedIndex < NumCachedParticles; ++CachedIndex)
				{
					// Note that VtArray::operator[] is non-const access and will cause trigger 
					// the copy-on-write memcopy!  VtArray::cdata() avoids that.
					const pxr::GfVec3f& Pos = Points0.cdata()[CachedIndex];
					const pxr::GfVec3f& Vel = Vels0.cdata()[CachedIndex];

					LoadFunction(CachedIndex, ParticleRange.Start, FVector3f(Pos[0], Pos[1], Pos[2]), FVector3f(Vel[0], Vel[1], Vel[2]), Activations0.cdata()[CachedIndex]);
				}
				return;
			}

			if (!UE::ChaosCachingUSD::ReadPoints(PrimitiveStage.MonolithStage, PrimitiveStage.PrimPath, Next, Points1, Vels1) ||
				Points1.size() != Vels1.size() ||
				Points0.size() != Points1.size())
			{
				UE_LOGF(LogChaosFleshCache, Error,
					"Failed to read points '%ls' at time %g from file: '%ls'",
					*PrimitiveStage.PrimPath, Next, *PrimitiveStage.MonolithStage.GetRootLayer().GetDisplayName());
				return;
			}

			// load muscle activation
			if (!UE::ChaosCachingUSD::ReadMuscleActivation(PrimitiveStage.MonolithStage, PrimitiveStage.PrimPath, Prev, Activations1) ||
				Points1.size() != Activations1.size())
			{
				Activations1.assign(Points1.size(), -1.f);
			}

			double Duration = Next - Prev;
			double Alpha = Duration > UE_SMALL_NUMBER ? (TargetTime - Prev) / Duration : 0.5;
			for (int32 CachedIndex = 0; CachedIndex < NumCachedParticles; ++CachedIndex)
			{
				// Note that VtArray::operator[] is non-const access and will cause trigger 
				// the copy-on-write memcopy!  VtArray::cdata() avoids that.
				const pxr::GfVec3f& P0 = Points0.cdata()[CachedIndex];
				const pxr::GfVec3f& P1 = Points1.cdata()[CachedIndex];
				const pxr::GfVec3f& V0 = Vels0.cdata()[CachedIndex];
				const pxr::GfVec3f& V1 = Vels1.cdata()[CachedIndex];
				pxr::GfVec3f Pos = (1.0 - Alpha) * P0 + Alpha * P1;
				pxr::GfVec3f Vel = (1.0 - Alpha) * V0 + Alpha * V1;
				float Activation = (1.0 - Alpha) * Activations0.cdata()[CachedIndex] + Alpha * Activations1.cdata()[CachedIndex];
				LoadFunction(CachedIndex, ParticleRange.Start, FVector3f(Pos[0], Pos[1], Pos[2]), FVector3f(Vel[0], Vel[1], Vel[2]), Activation);
			}
		}
	#endif
	}

	void FFleshCacheAdapter::Playback_PreSolve(UPrimitiveComponent* InComponent, UChaosCache* InCache, Chaos::FReal InTime, FPlaybackTickRecord& TickRecord, TArray<TPBDRigidParticleHandle<Chaos::FReal, 3>*>& OutUpdatedRigids) const
	{
		if (FDeformableSolver* Solver = GetDeformableSolver(InComponent))
		{
			FDeformableSolver::FPhysicsThreadAccess PhysicsThreadAccess(Solver, Softs::FPhysicsThreadAccessor());

			if (FEvolution* Evolution = PhysicsThreadAccess.GetEvolution())
			{
#if USE_USD_SDK && DO_USD_CACHING

				FParticles& Particles = Evolution->Particles();
				const int32 NumParticles = Particles.Size();
				Softs::FSolverVec3* ParticleXs = &Particles.X(0);
				Softs::FSolverVec3* ParticleVs = &Particles.V(0);
				Softs::FSolverReal* ParticleInvMs = &Particles.InvM(0);
				Softs::FPAndInvM* ParticlePAndInvMs = &Particles.PAndInvM(0);
				
				LoadCacheAtTime(InComponent, InTime, NumParticles, true, [&ParticleXs, &ParticleVs, &ParticleInvMs, &ParticlePAndInvMs](
					const int32 ParticleIndex, const int32 ParticleOffset, const FVector3f& ParticlePosition, const FVector3f& ParticleVelocity, const float MuscleActivation)
				{
					const int32 GlobalIndex = ParticleIndex + ParticleOffset;
					
					ParticleXs[GlobalIndex].Set(ParticlePosition[0], ParticlePosition[1], ParticlePosition[2]);
					ParticleVs[GlobalIndex].Set(ParticleVelocity[0], ParticleVelocity[1], ParticleVelocity[2]);
					ParticleInvMs[GlobalIndex] = Softs::FSolverReal(0);
					ParticlePAndInvMs[GlobalIndex].InvM = Softs::FSolverReal(0);
					ParticlePAndInvMs[GlobalIndex].P = ParticleXs[GlobalIndex];
				});

				if(UFleshComponent* FleshComp = CastChecked<UFleshComponent>(InComponent) )
				{
					if (const UDeformablePhysicsComponent::FThreadingProxy* PhysicsProxy = FleshComp->GetPhysicsProxy())
					{
						if (const Chaos::Softs::FFleshThreadingProxy* Proxy = PhysicsProxy->As<Chaos::Softs::FFleshThreadingProxy>())
						{
							Chaos::FRange ParticleRange = Proxy->GetSolverParticleRange();
							Evolution->ActivateParticleRange(ParticleRange.Start, false);
						}
					}
				}
				
#else // USE_USD_SDK && DO_USD_CACHING
				FCacheEvaluationContext Context(TickRecord);
				Context.bEvaluateTransform = false;
				Context.bEvaluateCurves = false;
				Context.bEvaluateEvents = false;
				Context.bEvaluateChannels = true;

				// The evaluated result are already in world space since we are passing the tickrecord.spacetransform in the context
				FCacheEvaluationResult EvaluatedResult = InCache->Evaluate(Context, nullptr);

				const TArray<float>* PendingVX = EvaluatedResult.Channels.Find(VelocityXName);
				const TArray<float>* PendingVY = EvaluatedResult.Channels.Find(VelocityYName);
				const TArray<float>* PendingVZ = EvaluatedResult.Channels.Find(VelocityZName);
				const TArray<float>* PendingPX = EvaluatedResult.Channels.Find(PositionXName);
				const TArray<float>* PendingPY = EvaluatedResult.Channels.Find(PositionYName);
				const TArray<float>* PendingPZ = EvaluatedResult.Channels.Find(PositionZName);

				const int32 NumCachedParticles = EvaluatedResult.ParticleIndices.Num();

				FParticles& Particles = Evolution->Particles();

				const int32 NumParticles = Particles.Size();
				if (NumCachedParticles > 0)
				{
					// Directly set the result of the cache into the solver particles

					Softs::FSolverVec3* ParticleXs = &Particles.X(0);
					Softs::FSolverVec3* ParticleVs = &Particles.V(0);

					for (int32 CachedIndex = 0; CachedIndex < NumCachedParticles; ++CachedIndex)
					{
						const int32 ParticleIndex = EvaluatedResult.ParticleIndices[CachedIndex];
						if (ensure(ParticleIndex < NumParticles))
						{
							Softs::FSolverVec3& ParticleV = ParticleVs[ParticleIndex];
							Softs::FSolverVec3& ParticleX = ParticleXs[ParticleIndex];
							ParticleV.X = (*PendingVX)[CachedIndex];
							ParticleV.Y = (*PendingVY)[CachedIndex];
							ParticleV.Z = (*PendingVZ)[CachedIndex];
							ParticleX.X = (*PendingPX)[CachedIndex];
							ParticleX.Y = (*PendingPY)[CachedIndex];
							ParticleX.Z = (*PendingPZ)[CachedIndex];
						}
					}
				}

#endif // USE_USD_SDK && DO_USD_CACHING
			}
		}
	}

	FGuid FFleshCacheAdapter::GetGuid() const
	{
		FGuid NewGuid;
		checkSlow(FGuid::Parse(TEXT("2C054706CB7441B582377B0EDACD12EE"), NewGuid));
		return NewGuid;
	}

	bool FFleshCacheAdapter::ValidForPlayback(UPrimitiveComponent* InComponent, UChaosCache* InCache) const
	{
		// If we have a flesh mesh we can play back any cache as long as it has one or more tracks
		const UFleshComponent* FleshComp = CastChecked<UFleshComponent>(InComponent);
#if USE_USD_SDK && DO_USD_CACHING
		return FleshComp != nullptr;
#else
		return FleshComp && InCache->ChannelCurveToParticle.Num() > 0;
#endif // USE_USD_SDK && DO_USD_CACHING
	}

	Chaos::Softs::FDeformableSolver* FFleshCacheAdapter::GetDeformableSolver(UPrimitiveComponent* InComponent)
	{
		if(InComponent)
		{
			if(UFleshComponent* FleshComp = CastChecked<UFleshComponent>(InComponent) )
			{
				if(FleshComp->GetDeformableSolver())
				{
					return FleshComp->GetDeformableSolver()->FleshSolverProxy.Solver.Get();
				}
			}
		}
		return nullptr;
	}

	Chaos::FRange FFleshCacheAdapter::GetParticleRange(UPrimitiveComponent* InComponent, const int32 NumParticles)
	{
		Chaos::FRange ParticleRange(0, NumParticles);
		if(InComponent)
		{
			if(UFleshComponent* FleshComp = CastChecked<UFleshComponent>(InComponent))
			{
				if(FleshComp->GetPhysicsProxy())
				{
					if (const Chaos::Softs::FFleshThreadingProxy* Proxy = FleshComp->GetPhysicsProxy()->As<Chaos::Softs::FFleshThreadingProxy>())
					{
						ParticleRange = Proxy->GetSolverParticleRange();
					}
				}
			}
		}
		return ParticleRange;
	}

	Chaos::FPhysicsSolver* FFleshCacheAdapter::GetComponentSolver(UPrimitiveComponent* InComponent) const
	{
		return nullptr;
	}
	
	Chaos::FPhysicsSolverEvents* FFleshCacheAdapter::BuildEventsSolver(UPrimitiveComponent* InComponent) const
	{
		if(UFleshComponent* FleshComp = CastChecked<UFleshComponent>(InComponent))
		{
			// Initialize the physics solver at the beginning of the play/record
			FleshComp->RecreatePhysicsState();
			return GetDeformableSolver(InComponent);
		}
		return nullptr;
	}

	void FFleshCacheAdapter::InitializeForLoad(UPrimitiveComponent* InComponent, FObservedComponent& InObserved)
	{
#if USE_USD_SDK && DO_USD_CACHING
		if(const UFleshComponent* FleshComp = CastChecked<UFleshComponent>(InComponent))
		{
			FPrimitiveStage& PrimitiveStage = PrimitiveStages.FindOrAdd(InComponent);
			
			PrimitiveStage.PrimPath = UsdUtils::GetPrimPathForObject(FleshComp);
			PrimitiveStage.FilePath = GetUSDCacheFilePathRO(InObserved, FleshComp);
			
			if (!PrimitiveStage.MonolithStage)
			{
				FPlatformFileManager& FileManager = FPlatformFileManager::Get();
				IPlatformFile& PlatformFile = FileManager.GetPlatformFile();
				
				if (PlatformFile.FileExists(*PrimitiveStage.FilePath))
				{
					UE::ChaosCachingUSD::OpenStage(PrimitiveStage.FilePath, PrimitiveStage.MonolithStage);
				}
				else
				{
					UE_LOGF(LogChaosFleshCache, Warning, "Read Failure: USD File Path = %ls", *PrimitiveStage.FilePath);
				}
			}
		}
#endif // USE_USD_SDK && DO_USD_CACHING
	}
	
	void FFleshCacheAdapter::SetRestState(UPrimitiveComponent* InComponent, UChaosCache* InCache, const FTransform& InRootTransform, Chaos::FReal InTime) const
	{
#if !USE_USD_SDK && DO_USD_CACHING
		if (!InCache || InCache->GetDuration() == 0.0f)
		{
			return;
		} 
#endif // USE_USD_SDK && DO_USD_CACHING

		if (UFleshComponent* FleshComp = CastChecked<UFleshComponent>(InComponent))
		{
#if USE_USD_SDK && DO_USD_CACHING
			
			FleshComp->ResetDynamicCollection();
			if (UFleshDynamicAsset* DynamicCollection = FleshComp->GetDynamicCollection())
			{
				TManagedArray<FVector3f>& DynamicVertex = DynamicCollection->GetPositions();
				TManagedArray<float>& DynamicVertexActivation = DynamicCollection->GetActivations();
				const int32 NumDynamicVertex = DynamicVertex.Num();

				auto UEVertd = [](Chaos::FVec3 V) { return FVector3d(V.X, V.Y, V.Z); };
				auto UEVertf = [](FVector3d V) { return FVector3f((float)V.X, (float)V.Y, (float)V.Z); };

				LoadCacheAtTime(InComponent, InTime, NumDynamicVertex, false, [&DynamicVertex, &DynamicVertexActivation, FleshComp, UEVertd, UEVertf](
					const int32 ParticleIndex, const int32 ParticleOffset, const FVector3f& ParticlePosition, const FVector3f&, const float& MuscleActivation)
				{
					const int32 GlobalIndex = ParticleIndex + ParticleOffset;

					DynamicVertex[GlobalIndex].Set(ParticlePosition[0], ParticlePosition[1], ParticlePosition[2]);
					DynamicVertex[GlobalIndex] =
									UEVertf(FleshComp->GetComponentTransform().InverseTransformPosition(
										UEVertd(DynamicVertex[GlobalIndex])));
					DynamicVertexActivation[GlobalIndex] = MuscleActivation;
				});
			}

			
#else // USE_USD_SDK && DO_USD_CACHING

			FPlaybackTickRecord TickRecord;
			TickRecord.SetLastTime(InTime);

			FCacheEvaluationContext Context(TickRecord);
			Context.bEvaluateTransform = false;
			Context.bEvaluateCurves = false;
			Context.bEvaluateEvents = false;
			Context.bEvaluateChannels = true;

			FCacheEvaluationResult EvaluatedResult = InCache->Evaluate(Context, nullptr);

			const TArray<float>* PendingVX = EvaluatedResult.Channels.Find(VelocityXName);
			const TArray<float>* PendingVY = EvaluatedResult.Channels.Find(VelocityYName);
			const TArray<float>* PendingVZ = EvaluatedResult.Channels.Find(VelocityZName);
			const TArray<float>* PendingPX = EvaluatedResult.Channels.Find(PositionXName);
			const TArray<float>* PendingPY = EvaluatedResult.Channels.Find(PositionYName);
			const TArray<float>* PendingPZ = EvaluatedResult.Channels.Find(PositionZName);

			const int32 NumCachedParticles = EvaluatedResult.ParticleIndices.Num();

			const bool bHasPositions = PendingPX && PendingPY && PendingPZ;
			const bool bHasVelocities = PendingVX && PendingVY && PendingVZ;


			FleshComp->ResetDynamicCollection();

			if (bHasPositions && NumCachedParticles > 0)
			{
				if (UFleshDynamicAsset* DynamicCollection = FleshComp->GetDynamicCollection())
				{
					TManagedArray<FVector3f>& DynamicVertex = DynamicCollection->GetPositions();
					//TManagedArray<FVector3f>& DynamicVertex = DynamicCollection->GetVelocities();
					const int32 NumDynamicVertex = DynamicVertex.Num();
					if (NumDynamicVertex == NumCachedParticles)
					{
						for (int32 CachedIndex = 0; CachedIndex < NumCachedParticles; ++CachedIndex)
						{
							const int32 ParticleIndex = EvaluatedResult.ParticleIndices[CachedIndex];

							if (ensure(ParticleIndex < NumDynamicVertex))
							{
								DynamicVertex[ParticleIndex].X = (*PendingPX)[CachedIndex];
								DynamicVertex[ParticleIndex].Y = (*PendingPY)[CachedIndex];
								DynamicVertex[ParticleIndex].Z = (*PendingPZ)[CachedIndex];

								auto UEVertd = [](Chaos::FVec3 V) { return FVector3d(V.X, V.Y, V.Z); };
								auto UEVertf = [](FVector3d V) { return FVector3f((float)V.X, (float)V.Y, (float)V.Z); };
								DynamicVertex[ParticleIndex] = UEVertf(FleshComp->GetComponentTransform().InverseTransformPosition(UEVertd(DynamicVertex[ParticleIndex])));
							}
						}
					}
				}
			}

#endif // USE_USD_SDK && DO_USD_CACHING
		}
	}

#if USE_USD_SDK && DO_USD_CACHING
	FString FFleshCacheAdapter::GetUSDCacheDirectory(const FObservedComponent& InObserved)
	{
		// USDCacheDirectory is relative to the content dir, but with "/Game" rather than just "/" or some relative path.
		FString CacheDir = InObserved.USDCacheDirectory.Path;
		if (CacheDir.IsEmpty())
		{
			CacheDir = FPaths::Combine(FPaths::ProjectSavedDir(), FString(TEXT("SimCache")));
		}
		FPaths::NormalizeDirectoryName(CacheDir);
		if (CacheDir.StartsWith(FString(TEXT("/Game"))))
		{
			CacheDir = FPaths::Combine(FPaths::ProjectContentDir(), CacheDir.RightChop(5));
		}
		return CacheDir;
	}

	FString FFleshCacheAdapter::GetUSDCacheFileName(const UDeformablePhysicsComponent* FleshComp)
	{
		const UObject* CurrObject = FleshComp;
		const AActor* Actor = nullptr;
		do {
			CurrObject = CurrObject->GetOuter();
			Actor = Cast<AActor>(CurrObject);
		} while (!Actor);
		const UObject* ActorParent = Actor ? Actor->GetOuter() : nullptr;

		FString CompName;
		if (ActorParent)
		{
			CompName = FleshComp->GetPathName(ActorParent);
		}
		else
		{
			FleshComp->GetName(CompName);
		}
		if (!CompName.Len())
		{
			CompName = FString(TEXT("FleshCache"));
		}
		return CompName;
	}

	FString FFleshCacheAdapter::GetUSDCacheFilePathRO(
		const FObservedComponent& InObserved, 
		const UDeformablePhysicsComponent* FleshComp)
	{
		FString CacheDir = GetUSDCacheDirectory(InObserved);
		FString CompName = GetUSDCacheFileName(FleshComp);
		FString PrimPath = UsdUtils::GetPrimPathForObject(FleshComp);

		// Look for an existing file of a supported format.
		TArray<FString> SupportedFormats = UnrealUSDWrapper::GetNativeFileFormats();
		for (const FString& Ext : SupportedFormats)
		{
			FString FileName = FString::Printf(TEXT("%s.%s"), *CompName, *Ext);
			FString FilePath = FPaths::Combine(CacheDir, FileName);
			if (FPaths::FileExists(FilePath))
			{
				return FilePath;
			}
		}

		// Fall back on the format we're currently set to write.
		FString Ext = CVarParams.bWriteBinary ? FString(TEXT("usd")) : FString(TEXT("usda"));
		FString FileName = FString::Printf(TEXT("%s.%s"), *CompName, *Ext);
		FString FilePath = FPaths::Combine(CacheDir, FileName);

		return FilePath;
	}

	FString FFleshCacheAdapter::GetUSDCacheFilePathRW(
		const FObservedComponent& InObserved,
		const UDeformablePhysicsComponent* FleshComp)
	{
		FString CacheDir = GetUSDCacheDirectory(InObserved);
		FString CompName = GetUSDCacheFileName(FleshComp);
		FString PrimPath = UsdUtils::GetPrimPathForObject(FleshComp);
		FString Ext = CVarParams.bWriteBinary ? FString(TEXT("usd")) : FString(TEXT("usda"));
		FString FileName = FString::Printf(TEXT("%s.%s"), *CompName, *Ext);
		FString FilePath = FPaths::Combine(CacheDir, FileName);

		
		return FilePath;
	}
#endif // USE_USD_SDK && DO_USD_CACHING

	bool FFleshCacheAdapter::InitializeForRecord(UPrimitiveComponent* InComponent, FObservedComponent& InObserved)
	{
		if( FDeformableSolver* Solver = GetDeformableSolver(InComponent))
		{
			FDeformableSolver::FGameThreadAccess GameThreadAccess(Solver, Softs::FGameThreadAccessor());
			GameThreadAccess.SetEnableSolver(true);
			if (UFleshComponent* FleshComp = CastChecked<UFleshComponent>(InComponent))
			{
				FleshComp->ResetDynamicCollection();

#if USE_USD_SDK && DO_USD_CACHING
				//
				// USD caching
				//

				FString CompName = GetUSDCacheFileName(FleshComp);
				FString CacheDir = GetUSDCacheDirectory(InObserved);
				FPlatformFileManager& FileManager = FPlatformFileManager::Get();
				IPlatformFile& PlatformFile = FileManager.GetPlatformFile();
				if (!PlatformFile.DirectoryExists(*CacheDir))
				{
					if (!PlatformFile.CreateDirectoryTree(*CacheDir))
					{
						UE_LOGF(LogChaosFleshCache, Error, "Failed to create output directory: '%ls'", *CacheDir);
						return false;
					}
				}

				const UFleshAsset* RestCollectionAsset = FleshComp->GetRestCollection();
				TSharedPtr<const FFleshCollection> RestCollection = RestCollectionAsset ? RestCollectionAsset->GetFleshCollection() : nullptr;
				if (!RestCollection)
				{
					UE_LOGF(LogChaosFleshCache, Error, "Failed to get rest collection from flesh component: '%ls'", *FleshComp->GetName());
					return false;
				}

				FPrimitiveStage& PrimitiveStage = PrimitiveStages.FindOrAdd(InComponent);

				PrimitiveStage.PrimPath = UsdUtils::GetPrimPathForObject(FleshComp);
				if (bUseMonolith)
				{
					bReadOnly = false;
					FString Ext = CVarParams.bWriteBinary ? FString(TEXT("usd")) : FString(TEXT("usda"));
					FString FileName = FString::Printf(TEXT("%s.%s"), *CompName, *Ext);
					PrimitiveStage.FilePath = FPaths::Combine(CacheDir, FileName);
					if (CVarParams.bNoClobber)
					{
						if (PlatformFile.FileExists(*PrimitiveStage.FilePath))
						{
							// Rename the file to 'path/to/file.usd' to 'path/to/file_#.usd', where '#' 
							// is a unique version number.
							FString UniqueFilePath;
							int32 i = 1;
							do {
								FString UniqueCompName = FString::Printf(TEXT("%s_%d.%s"), *CompName, i++, *Ext);
								UniqueFilePath = FPaths::Combine(CacheDir, UniqueCompName);
							} while (PlatformFile.FileExists(*UniqueFilePath));
							if (bRestartSimulation)
							{
								if (!PlatformFile.CopyFile(*UniqueFilePath, *PrimitiveStage.FilePath))
								{
									UE_LOGF(LogChaosFleshCache, Error, "Failed to copy file from '%ls' to '%ls'.", *PrimitiveStage.FilePath, *UniqueFilePath);
									return false;
								}
							}
							else
							{
								if (!PlatformFile.MoveFile(*UniqueFilePath, *PrimitiveStage.FilePath))
								{
									UE_LOGF(LogChaosFleshCache, Error, "Failed to rename file from '%ls' to '%ls'.", *PrimitiveStage.FilePath, *UniqueFilePath);
									return false;
								}
							}
						}
					}
					else
					{
						if (!bRestartSimulation && !PlatformFile.DeleteFile(*PrimitiveStage.FilePath))
						{
							UE_LOGF(LogChaosFleshCache, Error, "Failed to remove existing cache file: '%ls'", *PrimitiveStage.FilePath);
							return false;
						}
					}

					if (PrimitiveStage.MonolithStage)
					{
						UE::ChaosCachingUSD::CloseStage(PrimitiveStage.MonolithStage);
					}
					if (bRestartSimulation)
					{
						if (!UE::ChaosCachingUSD::OpenStage(PrimitiveStage.FilePath, PrimitiveStage.MonolithStage))
						{
							UE_LOGF(LogChaosFleshCache, Error, "Failed to read restart USD file: '%ls'", *PrimitiveStage.FilePath);
							return false;
						}
						if (!UE::ChaosCachingUSD::DeletePointsInTimeRange(PrimitiveStage.MonolithStage, PrimitiveStage.PrimPath, RestartTimeStart, RestartTimeEnd))
						{
							UE_LOGF(LogChaosFleshCache, Error, "Failed to delete USD data in file: '%ls', delete range: [%f, %f]", *PrimitiveStage.FilePath, RestartTimeStart, RestartTimeEnd);
							return false;
						}
					}
					else
					{
						if (!UE::ChaosCachingUSD::NewStage(PrimitiveStage.FilePath, PrimitiveStage.MonolithStage))
						{
							UE_LOGF(LogChaosFleshCache, Error, "Failed to create new USD file: '%ls'", *PrimitiveStage.FilePath);
							return false;
						}
						if (!UE::ChaosCachingUSD::WriteTetMesh(PrimitiveStage.MonolithStage, PrimitiveStage.PrimPath, *RestCollection))
						{
							UE_LOGF(LogChaosFleshCache, Error, "Failed to write tetrahedron mesh '%ls' to USD file: '%ls'", *PrimitiveStage.PrimPath, *PrimitiveStage.FilePath);
							return false;
						}
					}
				}

#endif // USE_USD_SDK && DO_USD_CACHING

			}
		}
		return true;
	}

	bool FFleshCacheAdapter::InitializeForPlayback(UPrimitiveComponent* InComponent, FObservedComponent& InObserved, float InTime)
	{
		EnsureIsInGameThreadContext();
		
		if (FDeformableSolver* Solver = GetDeformableSolver(InComponent))
		{
			FDeformableSolver::FGameThreadAccess GameThreadAccess(Solver, Softs::FGameThreadAccessor());
			if (UDeformableTetrahedralComponent* FleshComp = CastChecked<UDeformableTetrahedralComponent>(InComponent))
			{
				FleshComp->ResetDynamicCollection();

				if (FleshComp->GetPhysicsProxy()->As<Chaos::Softs::FFleshThreadingProxy>())
				{
					FleshComp->GetPhysicsProxy()->As<Chaos::Softs::FFleshThreadingProxy>()->SetIsCached(true);
				}
#if USE_USD_SDK && DO_USD_CACHING
				//
				// USD caching
				//

				FPrimitiveStage& PrimitiveStage = PrimitiveStages.FindOrAdd(InComponent);

				PrimitiveStage.PrimPath = UsdUtils::GetPrimPathForObject(FleshComp);
				if (bUseMonolith)
				{
					bReadOnly = true;
					PrimitiveStage.FilePath = GetUSDCacheFilePathRO(InObserved, FleshComp);

					FPlatformFileManager& FileManager = FPlatformFileManager::Get();
					IPlatformFile& PlatformFile = FileManager.GetPlatformFile();
					if (PlatformFile.FileExists(*PrimitiveStage.FilePath))
					{
						if (PrimitiveStage.MonolithStage)
						{
							UE::ChaosCachingUSD::CloseStage(PrimitiveStage.MonolithStage);
						}
						if (!UE::ChaosCachingUSD::OpenStage(PrimitiveStage.FilePath, PrimitiveStage.MonolithStage))
						{
							UE_LOGF(LogChaosFleshCache, Error, "Failed to open USD cache file: '%ls'", *PrimitiveStage.FilePath);
							return false;
						}
					}
					else
					{
						UE_LOGF(LogChaosFleshCache, Error, "USD cache file not found: '%ls'", *PrimitiveStage.FilePath);
						return false;
					}
				}
				if (!bUseMonolith)
				{
					UE_LOGF(LogChaosFleshCache, Warning, "No USD file structure selected (bMonolith).");
				}
#endif // USE_USD_SDK && DO_USD_CACHING

			}
		}
		return true;
	}

	void FFleshCacheAdapter::Finalize()
	{
#if USE_USD_SDK && DO_USD_CACHING

		for(TPair<UPrimitiveComponent*,FPrimitiveStage>& PrimitiveStage : PrimitiveStages)
		{
			// Detach shared memory arrays.
			if (PrimitiveStage.Value.MonolithStage)
			{
				if (!bReadOnly)
				{
					UE::ChaosCachingUSD::SaveStage(PrimitiveStage.Value.MonolithStage, PrimitiveStage.Value.MinTime, PrimitiveStage.Value.MaxTime);
				}
				UE::ChaosCachingUSD::CloseStage(PrimitiveStage.Value.MonolithStage);
				PrimitiveStage.Value.MonolithStage = UE::FUsdStage();
			}

			PrimitiveStage.Value.PrimPath.Empty();
			PrimitiveStage.Value.FilePath.Empty();
			PrimitiveStage.Value.MinTime = TNumericLimits<double>::Max();
			PrimitiveStage.Value.MaxTime = TNumericLimits<double>::Lowest();
		}
		PrimitiveStages.Reset();
#endif // USE_USD_SDK && DO_USD_CACHING
	}

}    // namespace Chaos
