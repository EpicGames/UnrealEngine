// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFlesh/ChaosDeformableCollisionsActor.h"

#include "ChaosFlesh/ChaosDeformableSolverComponent.h"
#include "ChaosFlesh/ChaosDeformableSolverActor.h"
#include "Engine/StaticMeshActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosDeformableCollisionsActor)

ADeformableCollisionsActor::ADeformableCollisionsActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DeformableCollisionsComponent = CreateDefaultSubobject<UDeformableCollisionsComponent>(TEXT("DeformableCollisionsComponent"));
	RootComponent = DeformableCollisionsComponent;
	PrimaryActorTick.bCanEverTick = false;
}

#if WITH_EDITOR
void ADeformableCollisionsActor::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedPtr<IPropertyHandle> bReplicatePhysicsProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ADeformableCollisionsActor, bAsyncPhysicsTickEnabled), AActor::StaticClass());
	bReplicatePhysicsProperty->MarkHiddenByCustomization();
}
#endif

void ADeformableCollisionsActor::EnableSimulation(ADeformableSolverActor* InActor)
{
	if (InActor)
	{
		if (DeformableCollisionsComponent)
		{
			DeformableCollisionsComponent->EnableSimulation(InActor->GetDeformableSolverComponent());
		}
	}
}


#if WITH_EDITOR
void ADeformableCollisionsActor::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);
	if (PropertyThatWillChange && PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(ADeformableCollisionsActor, PrimarySolver))
	{
		PreEditChangePrimarySolver = PrimarySolver;
	}
	if (PropertyThatWillChange && PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(ADeformableCollisionsActor, StaticCollisions))
	{
		PreEditChangeCollisionBodies = StaticCollisions;
	}
}


void ADeformableCollisionsActor::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADeformableCollisionsActor, StaticCollisions))
	{
		typedef TObjectPtr<AStaticMeshActor> FActorType;
		TSet< FActorType>  A(PreEditChangeCollisionBodies), B(StaticCollisions);
		for (auto AddedActor : B.Difference(A).Array())
		{
			if (AddedActor && AddedActor->GetStaticMeshComponent())
			{
				GetCollisionsComponent()->AddStaticMeshComponent(AddedActor->GetStaticMeshComponent());
			}
		}

		for (auto RemovedActor : A.Difference(B).Array())
		{
			if (RemovedActor && RemovedActor->GetStaticMeshComponent())
			{
				GetCollisionsComponent()->RemoveStaticMeshComponent(RemovedActor->GetStaticMeshComponent());
			}
		}

		PreEditChangeCollisionBodies.Empty();
	}
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADeformableCollisionsActor, PrimarySolver))
	{
		if (PrimarySolver)
		{
			if (UDeformableSolverComponent* SolverComponent = PrimarySolver->GetDeformableSolverComponent())
			{
				if (DeformableCollisionsComponent)
				{
					DeformableCollisionsComponent->PrimarySolverComponent = SolverComponent;
					if (!SolverComponent->ConnectedObjects.DeformableComponents.Contains(DeformableCollisionsComponent))
					{
						SolverComponent->ConnectedObjects.DeformableComponents.Add(TObjectPtr<UDeformablePhysicsComponent>(DeformableCollisionsComponent));
					}
				}
			}
		}
		else if (PreEditChangePrimarySolver)
		{
			if (UDeformableSolverComponent* SolverComponent = PreEditChangePrimarySolver->GetDeformableSolverComponent())
			{
				if (DeformableCollisionsComponent)
				{
					DeformableCollisionsComponent->PrimarySolverComponent = nullptr;
					if (SolverComponent->ConnectedObjects.DeformableComponents.Contains(DeformableCollisionsComponent))
					{
						SolverComponent->ConnectedObjects.DeformableComponents.Remove(DeformableCollisionsComponent);
					}
				}
			}
		}
	}
}
#endif


