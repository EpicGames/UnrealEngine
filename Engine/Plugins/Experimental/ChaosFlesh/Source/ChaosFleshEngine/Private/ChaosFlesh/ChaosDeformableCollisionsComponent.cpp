// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFlesh/ChaosDeformableCollisionsComponent.h"

#include "ChaosFlesh/ChaosDeformableSolverComponent.h"
#include "Chaos/Deformable/ChaosDeformableCollisionsProxy.h"
#include "Chaos/Sphere.h"
#include "Chaos/Convex.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/TriangleMeshImplicitObject.h"
#include "Components/StaticMeshComponent.h"
#include "PhysicsEngine/BodySetup.h"
#include "Engine/StaticMesh.h"

DEFINE_LOG_CATEGORY_STATIC(LogUDeformableCollisionsComponentInternal, Log, All);

#define PERF_SCOPE(X) SCOPE_CYCLE_COUNTER(X); TRACE_CPUPROFILER_EVENT_SCOPE(X);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.UDeformableCollisionsComponent.AddStaticMeshComponent"), STAT_ChaosDeformable_UDeformableCollisionsComponent_AddStaticMeshComponent, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.UDeformableCollisionsComponent.RemoveStaticMeshComponent"), STAT_ChaosDeformable_UDeformableCollisionsComponent_RemoveStaticMeshComponent, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.UDeformableCollisionsComponent.NewProxy"), STAT_ChaosDeformable_UDeformableCollisionsComponent_NewProxy, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.UDeformableCollisionsComponent.NewDeformableData"), STAT_ChaosDeformable_UDeformableCollisionsComponent_NewDeformableData, STATGROUP_Chaos);


UDeformableCollisionsComponent::UDeformableCollisionsComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = false;
	bTickInEditor = false;
}

void UDeformableCollisionsComponent::AddStaticMeshComponent(UStaticMeshComponent* StaticMeshComponent)
{
	PERF_SCOPE(STAT_ChaosDeformable_UDeformableCollisionsComponent_AddStaticMeshComponent);

	if (StaticMeshComponent)
	{
		if (!CollisionBodies.Contains(StaticMeshComponent))
		{
			CollisionBodies.Add(TObjectPtr<UStaticMeshComponent>(StaticMeshComponent) );
			AddedBodies.Add(StaticMeshComponent);
		}
	}
}

void UDeformableCollisionsComponent::RemoveStaticMeshComponent(UStaticMeshComponent* StaticMeshComponent)
{
	PERF_SCOPE(STAT_ChaosDeformable_UDeformableCollisionsComponent_RemoveStaticMeshComponent);

	if (StaticMeshComponent)
	{
		if (CollisionBodies.Contains(StaticMeshComponent))
		{
			CollisionBodies.Remove(TObjectPtr<UStaticMeshComponent>(StaticMeshComponent));
			RemovedBodies.Add(StaticMeshComponent);
		}
	}
}


UDeformablePhysicsComponent::FThreadingProxy* UDeformableCollisionsComponent::NewProxy()
{
	PERF_SCOPE(STAT_ChaosDeformable_UDeformableCollisionsComponent_NewProxy);

	for (auto& Body : CollisionBodies)
	{
		if (Body)
		{
			if (!AddedBodies.Contains(Body))
			{
				AddedBodies.Add(Body);
			}
		}
	}
	return new FCollisionThreadingProxy(this);
}

// template<> class Chaos::TSphere<float, 3>;

UDeformableCollisionsComponent::FDataMapValue 
UDeformableCollisionsComponent::NewDeformableData()
{
	PERF_SCOPE(STAT_ChaosDeformable_UDeformableCollisionsComponent_NewDeformableData);

	TArray<Chaos::Softs::FCollisionObjectAddedBodies> AddedBodiesData;
	TArray<Chaos::Softs::FCollisionObjectRemovedBodies> RemovedBodiesData;
	TArray<Chaos::Softs::FCollisionObjectUpdatedBodies> UpdateBodiesData;

	for (auto& CollisionBody : AddedBodies)
	{
		if( CollisionBody )
		{
			if (UStaticMesh* StaticMesh = CollisionBody->GetStaticMesh())
			{
				if (UBodySetup* BodySetup = StaticMesh->GetBodySetup())
				{
					using namespace Chaos;
					FImplicitObject* Geometry = nullptr;
					FTransform P = CollisionBody->GetComponentToWorld();
					FVector Scale = P.GetScale3D();
					P.RemoveScaling();

					const int32 NumShapes = BodySetup->AggGeom.SphereElems.Num() + BodySetup->AggGeom.BoxElems.Num() + BodySetup->AggGeom.ConvexElems.Num();

					TArray<FTransform>& PrevXfs = CollisionBodiesPrevXf.FindOrAdd(CollisionBody);
					PrevXfs.SetNum(NumShapes);
					int32 PrevXfIndex = 0;

					// Sphere
					for( int8 Index = 0; Index < BodySetup->AggGeom.SphereElems.Num(); Index++, PrevXfIndex++)
					{
						FKSphereElem& S = BodySetup->AggGeom.SphereElems[Index];
						FTransform ShapeTransform = FTransform(Scale * S.Center) * P;
						Geometry = new ::Chaos::FSphere(FVector(0), S.Radius * Scale.GetMax());
						PrevXfs[PrevXfIndex] = ShapeTransform;
						AddedBodiesData.Add(Chaos::Softs::FCollisionObjectAddedBodies(
							{ CollisionBody, Chaos::Softs::ERigidCollisionShapeType::Sphere, Index },
							ShapeTransform, "", Geometry));
					}

					// Box
					for (int8 Index = 0; Index < BodySetup->AggGeom.BoxElems.Num(); Index++, PrevXfIndex++)
					{
						FKBoxElem& B = BodySetup->AggGeom.BoxElems[Index];
						FTransform ShapeTransform = FTransform(Scale * B.Center) * P;
						Geometry = new TBox<Chaos::FReal, 3>(
							-0.5 * Scale * FVector(B.X, B.Y, B.Z),
							0.5 * Scale * FVector(B.X, B.Y, B.Z));
						PrevXfs[PrevXfIndex] = ShapeTransform;
						AddedBodiesData.Add(Chaos::Softs::FCollisionObjectAddedBodies(
							{ CollisionBody, Chaos::Softs::ERigidCollisionShapeType::Box, Index },
							ShapeTransform, "", Geometry));
					}

					// Convex
					for (int8 Index = 0; Index < BodySetup->AggGeom.ConvexElems.Num(); Index++, PrevXfIndex++)
					{
						FKConvexElem& C = BodySetup->AggGeom.ConvexElems[Index];
						if (C.VertexData.Num())
						{
							FTransform ShapeTransform = C.GetTransform() * P;
							FConvexPtr ConvexMesh = C.GetChaosConvexMesh();	
							TArray<FConvex::FVec3Type> Vertices;
							Vertices.SetNum(C.VertexData.Num());
							for (int i = 0; i < Vertices.Num(); i++)
							{
								Vertices[i] = C.VertexData[i] * Scale;
							}
							Geometry = new FConvex(Vertices, 0.f);
							PrevXfs[PrevXfIndex] = ShapeTransform;
							AddedBodiesData.Add(Chaos::Softs::FCollisionObjectAddedBodies(
								{ CollisionBody, Chaos::Softs::ERigidCollisionShapeType::Convex, Index},
								ShapeTransform, "", Geometry));
						}
					}
				}
			}
		}
	}

	for (auto& RemovedBody : RemovedBodies) 
	{
		if (RemovedBody)
		{
			RemovedBodiesData.Add({  {RemovedBody, Chaos::Softs::ERigidCollisionShapeType::Unknown, INDEX_NONE} });
			CollisionBodiesPrevXf.Remove(RemovedBody);
		}
	}
	AddedBodies.Empty();
	RemovedBodies.Empty();

	for (auto& CollisionBody : CollisionBodies) 
	{
		if (CollisionBody)
		{
			if (UStaticMesh* StaticMesh = CollisionBody->GetStaticMesh())
			{
				if (UBodySetup* BodySetup = StaticMesh->GetBodySetup())
				{
					FTransform P = CollisionBody->GetComponentToWorld();
					FVector Scale = P.GetScale3D();
					P.RemoveScaling();

					const int32 NumShapes = BodySetup->AggGeom.SphereElems.Num() + BodySetup->AggGeom.BoxElems.Num() + BodySetup->AggGeom.ConvexElems.Num();

					TArray<FTransform>& PrevXfs = CollisionBodiesPrevXf.FindOrAdd(CollisionBody);
					PrevXfs.SetNum(NumShapes);
					int32 PrevShapeXfIndex = 0;

					for (int8 Index = 0; Index < BodySetup->AggGeom.SphereElems.Num(); Index++, PrevShapeXfIndex++)
					{
						FKSphereElem& S = BodySetup->AggGeom.SphereElems[Index];
						FTransform ShapeTransform = FTransform(Scale * S.Center) * P;
						if (!ShapeTransform.Equals(PrevXfs[Index]))
						{
							PrevXfs[Index] = ShapeTransform;
							UpdateBodiesData.Add({ {CollisionBody, Chaos::Softs::ERigidCollisionShapeType::Sphere, Index},ShapeTransform });
						}
					}

					for (int8 Index = 0; Index < BodySetup->AggGeom.BoxElems.Num(); Index++, PrevShapeXfIndex++)
					{
						FKBoxElem& B = BodySetup->AggGeom.BoxElems[Index];
						FTransform ShapeTransform = FTransform(Scale * B.Center) * P;
						if (!ShapeTransform.Equals(PrevXfs[Index]))
						{
							PrevXfs[Index] = ShapeTransform;
							UpdateBodiesData.Add({ {CollisionBody, Chaos::Softs::ERigidCollisionShapeType::Box, Index},ShapeTransform });
						}
					}

					for (int8 Index = 0; Index < BodySetup->AggGeom.ConvexElems.Num(); Index++, PrevShapeXfIndex++)
					{
						FKConvexElem& C = BodySetup->AggGeom.ConvexElems[Index];
						if (C.VertexData.Num())
						{
							FTransform ShapeTransform = C.GetTransform() * P;
							if (!ShapeTransform.Equals(PrevXfs[Index]))
							{
								PrevXfs[Index] = ShapeTransform;
								UpdateBodiesData.Add({ {CollisionBody, Chaos::Softs::ERigidCollisionShapeType::Convex, Index},ShapeTransform });
							}
						}
					}
				}
			}
		}
	}

	return FDataMapValue(new Chaos::Softs::FCollisionManagerProxy::FCollisionsInputBuffer(
		AddedBodiesData, RemovedBodiesData, UpdateBodiesData, this));
}
