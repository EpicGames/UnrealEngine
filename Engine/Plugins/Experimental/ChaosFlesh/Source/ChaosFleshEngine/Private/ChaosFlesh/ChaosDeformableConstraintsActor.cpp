// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFlesh/ChaosDeformableConstraintsActor.h"

#include "ChaosFlesh/ChaosDeformableSolverComponent.h"
#include "ChaosFlesh/ChaosDeformableSolverActor.h"
#include "ChaosFlesh/FleshActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosDeformableConstraintsActor)

ADeformableConstraintsActor::ADeformableConstraintsActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DeformableConstraintsComponent = CreateDefaultSubobject<UDeformableConstraintsComponent>(TEXT("DeformableConstraintsComponent"));
	RootComponent = DeformableConstraintsComponent;
	PrimaryActorTick.bCanEverTick = false;
}

void ADeformableConstraintsActor::EnableSimulation(ADeformableSolverActor* InActor)
{
	if (InActor)
	{
		if (DeformableConstraintsComponent)
		{
			DeformableConstraintsComponent->EnableSimulation(InActor->GetDeformableSolverComponent());
		}
	}
}

#if WITH_EDITOR
void ADeformableConstraintsActor::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedPtr<IPropertyHandle> bReplicatePhysicsProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ADeformableConstraintsActor, bAsyncPhysicsTickEnabled), AActor::StaticClass());
	bReplicatePhysicsProperty->MarkHiddenByCustomization();
}

void ADeformableConstraintsActor::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);
	if (PropertyThatWillChange && PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(ADeformableConstraintsActor, PrimarySolver))
	{
		PreEditChangePrimarySolver = PrimarySolver;
	}
	else if (PropertyThatWillChange && PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(ADeformableConstraintsActor, SourceBodies))
	{
		PreEditChangeSourceBodies = SourceBodies;
	}
	else if (PropertyThatWillChange && PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(ADeformableConstraintsActor, TargetBodies))
	{
		PreEditChangeTargetBodies = TargetBodies;
	}
}


void ADeformableConstraintsActor::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADeformableConstraintsActor, TargetBodies))
	{
		typedef TObjectPtr<AFleshActor> FActorType;
		TSet< FActorType>  AT(PreEditChangeTargetBodies), BT(TargetBodies);
		TSet< FActorType>  AS(PreEditChangeSourceBodies), BS(SourceBodies);
		for (auto AddedTarget : BT.Difference(AT).Array())
		{
			for (auto AddedSource : BS.Difference(AS).Array())
			{
				if (AddedTarget && AddedTarget->GetFleshComponent())
				{
					if (AddedSource && AddedSource->GetFleshComponent())
					{
						GetConstraintsComponent()->AddConstrainedBodies(AddedSource->GetFleshComponent(), 
							AddedTarget->GetFleshComponent(), FDeformableConstraintParameters());
					}
				}
			}
		}

		TArray<FConstraintObject> ObjectsToRemove;
		for (auto RemovedSource : AS.Difference(BS).Array())
		{
			for (FConstraintObject& Object : GetConstraintsComponent()->Constraints)
			{
				if (RemovedSource && RemovedSource->GetFleshComponent())
				{
					if (Object.Source == RemovedSource->GetFleshComponent())
					{
						ObjectsToRemove.Add(Object);
					}
				}
			}
		}
		for (auto RemovedTarget : AT.Difference(BT).Array())
		{
			for (FConstraintObject& Object : GetConstraintsComponent()->Constraints)
			{
				if (RemovedTarget && RemovedTarget->GetFleshComponent())
				{
					if (Object.Target == RemovedTarget->GetFleshComponent())
					{
						ObjectsToRemove.Add(Object);
					}
				}
			}
		}

		GetConstraintsComponent()->Constraints.RemoveAll([&ObjectsToRemove](const FConstraintObject& Request) {return ObjectsToRemove.Contains(Request); });
		PreEditChangeSourceBodies.Empty();
		PreEditChangeTargetBodies.Empty();
	}
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADeformableConstraintsActor, PrimarySolver))
	{
		if (PrimarySolver)
		{
			if (UDeformableSolverComponent* SolverComponent = PrimarySolver->GetDeformableSolverComponent())
			{
				if (DeformableConstraintsComponent)
				{
					DeformableConstraintsComponent->PrimarySolverComponent = SolverComponent;
					if (!SolverComponent->ConnectedObjects.DeformableComponents.Contains(DeformableConstraintsComponent))
					{
						SolverComponent->ConnectedObjects.DeformableComponents.Add(TObjectPtr<UDeformablePhysicsComponent>(DeformableConstraintsComponent));
					}
				}
			}
		}
		else if (PreEditChangePrimarySolver)
		{
			if (UDeformableSolverComponent* SolverComponent = PreEditChangePrimarySolver->GetDeformableSolverComponent())
			{
				if (DeformableConstraintsComponent)
				{
					DeformableConstraintsComponent->PrimarySolverComponent = nullptr;
					if (SolverComponent->ConnectedObjects.DeformableComponents.Contains(DeformableConstraintsComponent))
					{
						SolverComponent->ConnectedObjects.DeformableComponents.Remove(DeformableConstraintsComponent);
					}
				}
			}
		}
	}
}
#endif






