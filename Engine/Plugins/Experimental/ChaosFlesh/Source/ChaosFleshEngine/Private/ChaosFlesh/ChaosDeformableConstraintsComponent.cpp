// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFlesh/ChaosDeformableConstraintsComponent.h"

#include "ChaosFlesh/ChaosDeformableSolverComponent.h"
#include "Chaos/Deformable/ChaosDeformableConstraintsProxy.h"


DEFINE_LOG_CATEGORY_STATIC(LogUDeformableConstraintsComponentInternal, Log, All);


UDeformableConstraintsComponent::UDeformableConstraintsComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = false;
	bTickInEditor = false;
}

void UDeformableConstraintsComponent::AddConstrainedBodies(
	UFleshComponent* SourceComponent,
	UFleshComponent* TargetComponent,
	FDeformableConstraintParameters InParameters)
{
	//PERF_SCOPE(STAT_ChaosDeformable_UDeformableConstraintsComponent_AddStaticMeshComponent);

	FConstraintObject Constraint(SourceComponent, TargetComponent, InParameters);
	if (IsValid(Constraint))
	{
		if (!Constraints.Contains(Constraint))
		{
			Constraints.Add(Constraint);
			AddedConstraints.Add(Constraint);
		}
	}
}


void UDeformableConstraintsComponent::RemoveConstrainedBodies(
	UFleshComponent* SourceComponent,
	UFleshComponent* TargetComponent,
	FDeformableConstraintParameters InParameters)
{
	//PERF_SCOPE(STAT_ChaosDeformable_UDeformableConstraintsComponent_RemoveStaticMeshComponent);

	FConstraintObject Constraint(SourceComponent, TargetComponent, InParameters);
	if (IsValid(Constraint))
	{
		if (Constraints.Contains(Constraint))
		{
			Constraints.Remove(Constraint);
			RemovedConstraints.Add(Constraint);
		}
	}
}


UDeformablePhysicsComponent::FThreadingProxy* UDeformableConstraintsComponent::NewProxy()
{
	//PERF_SCOPE(STAT_ChaosDeformable_UDeformableConstraintsComponent_NewProxy);

	for (auto& Constraint : Constraints)
	{
		if (IsValid(Constraint))
		{
			if (!AddedConstraints.Contains(Constraint))
			{
				AddedConstraints.Add(Constraint);
			}
		}
	}
	return new FConstraintThreadingProxy(this);
}

bool UDeformableConstraintsComponent::IsValid(const FConstraintObject& Key) const
{
	return Key.Source && Key.Target;
}


UDeformableConstraintsComponent::FDataMapValue 
UDeformableConstraintsComponent::NewDeformableData()
{
	//PERF_SCOPE(STAT_ChaosDeformable_UDeformableConstraintsComponent_NewDeformableData);

	TArray<Chaos::Softs::FConstraintObjectAdded> AddedConstraintsData;
	TArray<Chaos::Softs::FConstraintObjectRemoved> RemovedConstraintsData;
	TArray<Chaos::Softs::FConstraintObjectUpdated> UpdateConstraintData;

	for (auto& Constraint : AddedConstraints)
	{
		if (IsValid(Constraint))
		{
			AddedConstraintsData.Add(Constraint.ToChaos());
		}
	}

	for (auto& Constraint : RemovedConstraints)
	{
		if (IsValid(Constraint))
		{
			RemovedConstraintsData.Add(Constraint.ToChaos());
		}
	}

	AddedConstraints.Empty();
	RemovedConstraints.Empty();

	return FDataMapValue(new Chaos::Softs::FConstraintManagerProxy::FConstraintsInputBuffer(
		AddedConstraintsData, RemovedConstraintsData, UpdateConstraintData, this));
}
