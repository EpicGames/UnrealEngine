// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFlesh/ChaosDeformableGPUBuffers.h"
#include "ChaosFlesh/ChaosDeformableTypes.h"

#include "RHIResourceUtils.h"

DEFINE_LOG_CATEGORY_STATIC(LogFleshDeformerBuffers, Log, All);

namespace Local
{
	FChaosEngineDeformableCVarParams CVarParam;
	FAutoConsoleVariableRef CVarTestUnpacking(
		TEXT("p.Chaos.Deformable.FleshDeformer.TestUnpacking"),
		CVarParam.bTestUnpacking,
		TEXT("Test flesh deformer data compression. [def: false]"));
} // namespace Local

using namespace UE::ChaosDeformable;

// Render lingo cheat sheet:
//  RHI - Render Hardware Interface - C++ interface for D3D/OGL.
//  SRV - Shader Resource View

//=========================================================================
// FFloatArrayBufferWithSRV
//=========================================================================

void FFloatArrayBufferWithSRV::Init(const float* InArray, const int32 Num)
{
	Array.SetNumUninitialized(Num);
	for (int i = 0; i < Num; i++)
	{
		Array[i] = InArray[i];
	}
	NumValues = Num;
}

uint32 FFloatArrayBufferWithSRV::GetBufferSize() const
{
	return NumValues * sizeof(float);
}

void FFloatArrayBufferWithSRV::InitRHI(FRHICommandListBase& RHICmdList)
{
    if (Array.Num() > 0)
    {
		VertexBufferRHI = UE::RHIResourceUtils::CreateVertexBufferFromArray(
			RHICmdList,
			TEXT("UMLDeformerModel::FFloatArrayBufferWithSRV"),
			EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource,
			MakeConstArrayView(Array)
		);

		if (bEmptyArray)
		{
			Array.Empty();
		}

        ShaderResourceViewRHI = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_FLOAT));

		UE_LOGF(LogFleshDeformerBuffers, Log,
			"FFloatArrayBufferWithSRV - '%ls' - Buffer size: %zu, Array num: %d",
			*BufferName, Array.Num() * sizeof(float), Array.Num());
	}
    else
    {
        VertexBufferRHI = nullptr;
        ShaderResourceViewRHI = nullptr;
    }
}

//=========================================================================
// FHalfArrayBufferWithSRV
//=========================================================================

void FHalfArrayBufferWithSRV::Init(const FFloat16* InArray, const int32 Num)
{
    Array.SetNumUninitialized(Num);
    for (int32 i = 0; i < Num; i++)
    {
        Array[i] = InArray[i];
    }
	NumValues = Num;
}

void FHalfArrayBufferWithSRV::Init(const float* InArray, const int32 Num)
{
    Array.SetNumUninitialized(Num);
    for (int32 i = 0; i < Num; i++)
    {
        Array[i] = FFloat16(InArray[i]);
    }
	NumValues = Num;
}

void FHalfArrayBufferWithSRV::Init(const TArray<float>& InArray)
{
	if (InArray.IsEmpty())
	{
		Array.SetNum(0);
		NumValues = 0;
	}
	else
	{
		Init(&InArray[0], InArray.Num());
	}
}

uint32 FHalfArrayBufferWithSRV::GetBufferSize() const
{
	return NumValues * sizeof(FFloat16);
}

void FHalfArrayBufferWithSRV::InitRHI(FRHICommandListBase& RHICmdList)
{
    if (Array.Num() > 0)
    {
		VertexBufferRHI = UE::RHIResourceUtils::CreateVertexBufferFromArray(
			RHICmdList,
			TEXT("FHalfArrayBufferWithSRV"),
			EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource,
			MakeConstArrayView(Array)
		);

        Array.Empty();

        ShaderResourceViewRHI = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R16F));

		UE_LOGF(LogFleshDeformerBuffers, Log,
			"FHalfArrayBufferWithSRV - '%ls' - Buffer size: %d, Array num: %d",
			*BufferName, GetBufferSize(), NumValues);
	}
    else
    {
        VertexBufferRHI = nullptr;
        ShaderResourceViewRHI = nullptr;
    }
}

//=========================================================================
// FIndexArrayBufferWithSRV
//=========================================================================

void FIndexArrayBufferWithSRV::Init(const int32* InArray, const int32 Num)
{
	Array.SetNumUninitialized(Num);
	int32 MinV = TNumericLimits<int32>::Max();
	int32 MaxV = -TNumericLimits<int32>::Max();
	for (int32 i = 0; i < Num; i++)
	{
		MinV = InArray[i] < MinV ? InArray[i] : MinV;
		MaxV = InArray[i] > MaxV ? InArray[i] : MaxV;
		Array[i] = InArray[i];
	}
	NumValues = Array.Num();

	int32 Range = MaxV - MinV;
	bUint8 = Range <= TNumericLimits<uint8>::Max();
	bUint16 = Range <= TNumericLimits<uint16>::Max();
	Offset = -MinV; // Adding offset shifts to zero.
}

void FIndexArrayBufferWithSRV::Init(const uint32* InArray, const int32 Num)
{
	Array.SetNumUninitialized(Num);
	uint32 MinV = TNumericLimits<uint32>::Max();
	uint32 MaxV = 0;
	for (int32 i = 0; i < Num; i++)
	{
		MinV = InArray[i] < MinV ? InArray[i] : MinV;
		MaxV = InArray[i] > MaxV ? InArray[i] : MaxV;
		Array[i] = InArray[i];
	}
	NumValues = Array.Num();

	uint32 Range = MaxV - MinV;
	bUint8 = Range <= TNumericLimits<uint8>::Max();
	bUint16 = Range <= TNumericLimits<uint16>::Max();
	Offset = -static_cast<int32>(MinV); // Adding offset shifts to zero.
}

uint32 FIndexArrayBufferWithSRV::GetBufferSize() const
{
	const int32 NumIndices = Array.Num();
	const int32 ValuesPerSlot = bForce32 ? 1 : bUint8 ? 4 : bUint16 ? 2 : 1;
	const int32 NumSlots = static_cast<int32>(ceil(static_cast<float>(NumIndices) / ValuesPerSlot));
	const int32 BufferSize = NumSlots * sizeof(uint32);
	return BufferSize;
}

#if WITH_EDITOR
uint32 UnpackUInt(const uint32* Array, const uint32 Index, const uint32 Stride, const uint32 ArraySize)
{
	uint32 ValsPerEntry = 4 / Stride;
	uint32 BaseIndex = Index / ValsPerEntry;
	uint32 SubIndex = Index % ValsPerEntry;

	check(BaseIndex < ArraySize);

	if (ValsPerEntry == 1)
	{
		check(SubIndex == 0);
		return Array[BaseIndex];
	}
	else if (ValsPerEntry == 2)
	{
		const uint16* Array16 = reinterpret_cast<const uint16*>(Array);
		const uint16 Val16 = Array16[Index]; // the c++ way
		uint16 Val = (Array[BaseIndex] >> (SubIndex * 16)) & 0xFFFF; // the hlsl way
		check(Val16 == Val); // make sure they match
		return Val;
	}
 	else
	{
		const uint8* Array8 = reinterpret_cast<const uint8*>(Array);
		const uint8 Val8 = Array8[Index]; // the c++ way
		uint8 Val = (Array[BaseIndex] >> (SubIndex * 8)) & 0xFF; // the hlsl way
		check(Val8 == Val); // make sure they match
		return Val;
	}
}
#endif


FORCEINLINE uint32 GetBits(uint32 Value, uint32 NumBits, uint32 Offset)
{
	uint32 Mask = (1u << NumBits) - 1u;
	return (Value >> Offset) & Mask;
}

FORCEINLINE void SetBits(uint32& Value, uint32 Bits, uint32 NumBits, uint32 Offset)
{
	uint32 Mask = (1u << NumBits) - 1u;
	check(Bits <= Mask);
	Mask <<= Offset;
	Value = (Value & ~Mask) | (Bits << Offset);
}

void FIndexArrayBufferWithSRV::InitRHI(FRHICommandListBase& RHICmdList)
{
	// Buffer is comprised of 32 bit values.  We're packing 32, 16, or 8 bit ints into 32 bit slots.
	const int32 NumIndices = Array.Num();
	const int32 ValuesPerSlot = bForce32 ? 1 : bUint8 ? 4 : bUint16 ? 2 : 1;
	const int32 NumSlots = static_cast<int32>(ceil(static_cast<float>(NumIndices) / ValuesPerSlot));
	const int32 BufferSize = NumSlots * sizeof(uint32);

	// From the perspective of the gpu transport system, we're shipping across 32 bits, 
	// and so wherever that matters, it's hard coded to 'sizeof(uint32)'.
	const int32 DataStride = 
		bForce32 ? sizeof(uint32) :
			bUint8 ? sizeof(uint8) : 
				bUint16 ? sizeof(uint16) : sizeof(uint32);

    if (BufferSize)
    {
        // Create the index buffer.
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(*BufferName, BufferSize)
			.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
			.DetermineInitialState()
			.SetInitActionInitializer();

        // Initialize the buffer.
		
		if (!bForce32 && bUint8)
        {
			TRHIBufferInitializer<uint8> Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);

			// 0: 33 22 11 00
			// 1: 77 66 55 44
			// 2: 11 10 99 88

			int32 i = 0;
			for (; i < NumIndices; i++)
            {
				Initializer[i] = static_cast<uint8>(Array[i] + Offset); // add offset, sub in shader
            }
			// Zero tailing unused space.
			for (; i < NumSlots * 4; i++)
			{
				Initializer[i] = 0;
			}

			VertexBufferRHI = Initializer.Finalize();
		}
        else if (!bForce32 && bUint16)
        {
			TRHIBufferInitializer<uint16> Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);

			// 0: 1111 0000
			// 1: 3333 2222
			// 2: 5555 4444
			
			int32 i = 0;
			for (; i < NumIndices; i++)
			{
				Initializer[i] = static_cast<uint16>(Array[i] + Offset); // add offset, sub in shader
			}
			// Zero tailing unused space.
			for (; i < NumSlots * 2; i++)
			{
				Initializer[i] = 0;
			}

			VertexBufferRHI = Initializer.Finalize();
		}
        else // uint32
        {
			TRHIBufferInitializer<uint32> Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);

			for (int32 i = 0; i < NumIndices; i++)
			{
				Initializer[i] = static_cast<uint32>(Array[i] + Offset); // add offset, sub in shader
			}

			VertexBufferRHI = Initializer.Finalize();
        }

		// Commented out because we should not be reading from Mapped data
#if WITH_EDITOR && 0
		if (Local::CVarParam.bTestUnpacking)
		{
			uint32* Indices32Bit = reinterpret_cast<uint32*>(InitialData.GetDataForWriting());
			for (int32 i = 0; i < NumIndices; i++)
			{
				int32 UnpackedInt = static_cast<int>(UnpackUInt(Indices32Bit, i, DataStride, NumIndices)) - Offset; // sub offset for unpacking
				int32 Orig = Array[i];
				check(Orig == UnpackedInt);
			}
			UE_LOGF(LogFleshDeformerBuffers, Log,
				"FIndexArrayBufferWithSRV - TEST_UNPACKING succeeded for %d values, packed into buffer size %d, %d bits per value.",
				NumIndices, BufferSize, (BufferSize / NumIndices));
		}
#endif

        Array.Empty();

		EPixelFormat Format = PF_R32_UINT;
        ShaderResourceViewRHI = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI,
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(Format));

		UE_LOGF(LogFleshDeformerBuffers, Log,
			"FIndexArrayBufferWithSRV - '%ls' - Data stride: %d, Buffer size: %d, Input array size: %d",
			*BufferName, DataStride, BufferSize, NumValues);
	}
    else
    {
        VertexBufferRHI = nullptr;
        ShaderResourceViewRHI = nullptr;
    }
}
