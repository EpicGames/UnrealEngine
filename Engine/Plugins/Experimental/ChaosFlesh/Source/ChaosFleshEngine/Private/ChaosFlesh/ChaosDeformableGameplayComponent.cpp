// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFlesh/ChaosDeformableGameplayComponent.h"

#include "ChaosStats.h"
#include "Components/BoxComponent.h"
#include "Chaos/DebugDrawQueue.h"
#include "ChaosFlesh/ChaosDeformableTypes.h"
#include "Components/ActorComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Containers/Map.h"
#include "Dataflow/DataflowEngineUtil.h"
#include "GeometryCollection/Facades/CollectionConstraintOverrideFacade.h"
#include "GeometryCollection/Facades/CollectionTransformSourceFacade.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"
#include "HAL/IConsoleManager.h"
#include "WorldCollision.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "Engine/World.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosDeformableGameplayComponent)
#define LOCTEXT_NAMESPACE "DeformableGameplayComponent"

FChaosEngineDeformableCVarParams CVarParams2;
FAutoConsoleVariableRef CVarDeformableDebugParamsDrawSceneRaycasts(TEXT("p.Chaos.DebugDraw.Deformable.SceneRaycasts"), CVarParams2.bDoDrawSceneRaycasts, TEXT("Debug draw the deformables scene raycasts. [def: false]"));
FAutoConsoleVariableRef CVarDeformableDebugParamsDrawCandidateRaycasts(TEXT("p.Chaos.DebugDraw.Deformable.CandidateRaycasts"), CVarParams2.bDoDrawCandidateRaycasts, TEXT("Debug draw the deformables scene candidate raycasts. [def: false]"));
FAutoConsoleVariableRef CVarDeformableEnvCollisionsLineTracesBatchSize(TEXT("p.Chaos.Deformable.EnvCollisionsLineTracesBatchSize"), CVarParams2.EnvCollisionsLineTraceBatchSize, TEXT("Batch size for FleshComponent env collsions. [def: 10]"));

#define PERF_SCOPE(X) SCOPE_CYCLE_COUNTER(X); TRACE_CPUPROFILER_EVENT_SCOPE(X);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.UDeformableGameplayComponent.PreSolverUpdate"), STAT_ChaosDeformable_UDeformableGameplayComponent_PreSolverUpdate, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.UDeformableGameplayComponent.DetectEnvironmentCollisions"), STAT_ChaosDeformable_UDeformableGameplayComponent_DetectEnvironmentCollisions, STATGROUP_Chaos);

//DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.UDeformableGameplayComponent.StreamStaticMeshGeometry"), STAT_ChaosDeformable_UDeformableGameplayComponent_StreamStaticMeshGeometry, STATGROUP_Chaos);

UDeformableGameplayComponent::UDeformableGameplayComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


UDeformableGameplayComponent::~UDeformableGameplayComponent()
{
}


void UDeformableGameplayComponent::PreSolverUpdate()
{
	PERF_SCOPE(STAT_ChaosDeformable_UDeformableGameplayComponent_PreSolverUpdate)
	if (GameplayColllisions.RigBoundRayCasts.bEnableRigBoundRaycasts)
	{
		DetectEnvironmentCollisions(GameplayColllisions.RigBoundRayCasts.MaxNumTests, GameplayColllisions.RigBoundRayCasts.bTestDownOnly, GameplayColllisions.RigBoundRayCasts.TestRange, GameplayColllisions.RigBoundRayCasts.CollisionChannel.GetValue());
	}
}


void UDeformableGameplayComponent::DetectEnvironmentCollisions(const int32 MaxNumTests, const bool bTestDownOnly, const float TestRange, const ECollisionChannel CollisionChannel)
{
	PERF_SCOPE(STAT_ChaosDeformable_UDeformableGameplayComponent_DetectEnvironmentCollisions)

	auto ToDouble = [](FVector3f V) { return FVector(V[0], V[1], V[2]); };
	auto ToSingle = [](FVector V) { return FVector3f(static_cast<float>(V[0]), static_cast<float>(V[1]), static_cast<float>(V[2])); };

	if (!GetRestCollection() || !GetSimulationCollection() || !GetDynamicCollection())
	{
		return;
	}

	// Init outputs, and clear any existing.
	GeometryCollection::Facades::FConstraintOverrideTargetFacade CnstrOutputs(
		*GetSimulationCollection()->GetCollection());
	CnstrOutputs.DefineSchema();
	CnstrOutputs.Clear();

	// Init inputs, and return if we have no candidates.
	GeometryCollection::Facades::FConstraintOverrideCandidateFacade CnstrCandidates(
		*GetRestCollection()->GetFleshCollection());
	if (!CnstrCandidates.IsValid())
	{
		return;
	}
	if (CnstrCandidates.Num() == 0)
	{
		return;
	}

	// We need to test the current skinned positions of the rest collection against the 
	// environment.
	const TManagedArray<FVector3f>* Positions = GetRestCollection()->FindPositions();
	if (!Positions)
	{
		return;
	}

	// Get bone weights for skinning vertices of the tetrahedron mesh.
	GeometryCollection::Facades::FVertexBoneWeightsFacade BoneWeightsFacade(
		*GetRestCollection()->GetFleshCollection());
	if (!BoneWeightsFacade.IsValid())
	{
		return;
	}
	const TManagedArray<TArray<int32>>* BoneIndices = BoneWeightsFacade.FindBoneIndices();
	const TManagedArray<TArray<float>>* BoneWeights = BoneWeightsFacade.FindBoneWeights();

	const TManagedArray<FTransform3f>* RestTransforms = nullptr;
	if (const UFleshAsset* FleshAsset = GetRestCollection())
	{
		if (TSharedPtr<const FFleshCollection> Rest = FleshAsset->GetFleshCollection())
		{
			GeometryCollection::Facades::FTransformSource TransformSource(*Rest);
			if (TransformSource.IsValid())
			{
				RestTransforms = Rest->FindAttribute<FTransform3f>(
					FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup);
			}
		}
	}

	// Find the skeletal mesh containing the bone we're using for the origin of our rays.
	USkeletalMeshComponent* SkeletalMeshComponent = nullptr;
	TArray<USkeletalMeshComponent*> SkeletalMeshComponents;
	GetOwner()->GetComponents<USkeletalMeshComponent>(SkeletalMeshComponents);
	for (int32 i = 0; i < SkeletalMeshComponents.Num(); i++)
	{
		if (SkeletalMeshComponents[i]->GetSkeletalMeshAsset() == GetRestCollection()->SkeletalMesh)
		{
			SkeletalMeshComponent = SkeletalMeshComponents[i];
			break;
		}
	}
	if (!SkeletalMeshComponent)
	{
		return;
	}

	// Get the current animated bone transforms and then their positions.
	const TArray<FTransform>& AnimationInComponentSpaceTransforms = SkeletalMeshComponent->GetComponentSpaceTransforms();
	const TArray<FTransform>& RestInLocalSpaceTransforms = SkeletalMeshComponent->GetSkeletalMeshAsset()->GetSkeleton()->GetReferenceSkeleton().GetRefBonePose(); // Local Space

	TMap<int32, FVector> WorldBoneLocations;
	WorldBoneLocations.Reserve(16);

	struct CandidateData
	{
		CandidateData(const int32 IndexIn, const FVector& OriginIn, const FVector& TargetIn)
			: Index(IndexIn), Origin(OriginIn), Target(TargetIn)
		{}
		int32 Index;
		FVector Origin;
		FVector Target;
	};
	TArray<CandidateData> Candidates;

	const FTransform& ComponentToWorldXf = GetComponentToWorld();
	for (int32 i = 0; i < CnstrCandidates.Num(); i++)
	{
		GeometryCollection::Facades::FConstraintOverridesCandidateData Candidate = CnstrCandidates.Get(i);
		const int32 TetVertexIndex = Candidate.VertexIndex;
		const int32 OriginBoneIndex = Candidate.BoneIndex;

		if (TetVertexIndex < 0 || TetVertexIndex >= Positions->Num())
		{
			continue;
		}
		const FVector3f& RestPos = (*Positions)[TetVertexIndex];

		// Compute skinning, if present.
		bool bDidSkinnedPosition = false;
		FVector SkinnedPos(0.0,0.0,0.0);
		if (RestTransforms && RestTransforms->Num() &&
			BoneIndices && TetVertexIndex < BoneIndices->Num() && (*BoneIndices)[TetVertexIndex].Num() &&
			BoneWeights && TetVertexIndex < BoneWeights->Num() && (*BoneWeights)[TetVertexIndex].Num())
		{
			// Get Skinned bones and weights.
			const TArray<int32>& Bones = (*BoneIndices)[TetVertexIndex];
			const TArray<float>& Weights = (*BoneWeights)[TetVertexIndex];
			checkSlow(Bones.Num() == Weights.Num());

			// Compute skinned test mesh world position.
			
			for (int32 j = 0; j < Bones.Num(); j++)
			{
				if (AnimationInComponentSpaceTransforms.IsValidIndex(Bones[j]) && RestTransforms->Num() > Bones[j])
				{
					//auto ComponentSpaceMatrix = [](const FReferenceSkeleton& RefSkeleton, const TArray<FTransform>& RestTransforms, int32 Index)
					//{
					//	FTransform FlatRest = RestTransforms[Index];
					//	while (RefSkeleton.GetParentIndex(Index) != INDEX_NONE)
					//	{
					//		Index = RefSkeleton.GetParentIndex(Index);
					//		FlatRest *= RestTransforms[Index];
					//	}
					//	return FlatRest;
					//};
					//
					//const FReferenceSkeleton& RefSkeleton = SkeletalMeshComponent->GetSkeletalMeshAsset()->GetSkeleton()->GetReferenceSkeleton();
					//const TArray<FTransform>& RefMat = RefSkeleton.GetRefBonePose();
					//FTransform RestInComponent = ComponentSpaceMatrix(SkeletalMeshComponent->GetSkeletalMeshAsset()->GetSkeleton()->GetReferenceSkeleton(), RefMat, Bones[j]);
					//FVector LocalPoint = RestInComponent.TransformPosition(ToDouble(RestPos));
					
					TSharedPtr<const FFleshCollection> Collection = GetRestCollection()->GetFleshCollection();
					FVector3f LocalPoint = (*RestTransforms)[Bones[j]].InverseTransformPosition(RestPos);
					if (const TManagedArray<FTransform3f>* ConstTransformsPtr = Collection->FindAttributeTyped<FTransform3f>(FName("ComponentTransform"), FName("ComponentTransformGroup")))
					{
						LocalPoint = (*ConstTransformsPtr)[0].InverseTransformPosition(RestPos);
					}

					SkinnedPos += AnimationInComponentSpaceTransforms[Bones[j]].TransformPosition(FVector(LocalPoint)) * Weights[j];
					bDidSkinnedPosition = true;
				}
			}
		}
		if (!bDidSkinnedPosition)
		{
			SkinnedPos = FVector(RestPos[0], RestPos[1], RestPos[2]);
		}
		FVector WorldSkinnedPos = ComponentToWorldXf.TransformPosition(SkinnedPos);

		// Compute/lookup ray origin world position.
		FVector WorldOrigin;
		if (FVector* Existing = WorldBoneLocations.Find(OriginBoneIndex))
		{
			WorldOrigin = *Existing;
		}
		else
		{
			if (!AnimationInComponentSpaceTransforms.IsValidIndex(OriginBoneIndex))
			{
				continue;
			}
			const FTransform& BoneXf = AnimationInComponentSpaceTransforms[OriginBoneIndex];
			WorldOrigin = ComponentToWorldXf.TransformPosition(BoneXf.GetLocation());
			WorldBoneLocations.Add(TTuple<int32, FVector>(OriginBoneIndex, WorldOrigin));
		}

#if WITH_EDITOR
		if (CVarParams2.IsDebugDrawingEnabled() && CVarParams2.bDoDrawCandidateRaycasts)
		{
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(WorldOrigin, WorldSkinnedPos, 2.0f, FColor::Blue, false, -1.0f, 0, 0.75f);
		}
#endif

		if (bTestDownOnly)
		{
			FVector Direction = WorldSkinnedPos - WorldOrigin;
			Direction.Normalize();
			if (FVector::DotProduct(Direction, FVector::DownVector) < TestRange) // +1 down, -1 up, 0 orthogonal - DownVector:(0,0,-1)
			{
				continue;
			}
		}

		// Queue the query.
		Candidates.Add(CandidateData(TetVertexIndex, WorldOrigin, WorldSkinnedPos));
	}
	if (!Candidates.Num())
	{
		return;
	}

	// Randomly discard potentials until we get under MaxNumTests.
	if (EnvCollisionsPreviousHits.Num() > MaxNumTests)
	{
		// MaxNumTests changed, which puts us in danger of never exiting the while loop.
		// Blow the buffer.
		EnvCollisionsPreviousHits.Reset();
	}
	while (Candidates.Num() > MaxNumTests)
	{
		const int32 Index = static_cast<int32>(FGenericPlatformMath::Rand() / RAND_MAX * (Candidates.Num() - 1));
		if (!EnvCollisionsPreviousHits.Contains(Index))
		{
			Candidates.RemoveAtSwap(Index);
		}
	}
	EnvCollisionsPreviousHits.Reset();

	// Run scene queries.
	const FString QueryName =
		FText::Format(
		LOCTEXT("DetectEnvironmentCollisions", "FleshComponent: '{0}'"),
		FText::FromString(GetName())).ToString();
	FCollisionQueryParams TraceParams(
		FName(*QueryName),
		false,			// false to ignore complex collisions 
		GetOwner());	// GetOwner() to ignore self
	const FTransform WorldToComponentXf(ComponentToWorldXf.ToMatrixWithScale().Inverse());

	TArray<FHitResult> HitBuffer;
	HitBuffer.SetNum(Candidates.Num());
	TArray<bool> HitResult;
	HitResult.SetNum(Candidates.Num());

	Chaos::PhysicsParallelForRange(Candidates.Num(), [&](const int32 StartIndex, const int32 StopIndex)
								   {
									   for (int32 Index = StartIndex; Index < StopIndex; ++Index)
									   {
										   const int32 TetVertexIndex = Candidates[Index].Index;
										   const FVector& WorldOrigin = Candidates[Index].Origin;
										   const FVector& WorldSkinnedPos = Candidates[Index].Target;
										   HitResult[Index] = GetWorld()->LineTraceSingleByChannel(HitBuffer[Index], WorldOrigin, WorldSkinnedPos, CollisionChannel, TraceParams);
									   }
								   },
								   CVarParams2.EnvCollisionsLineTraceBatchSize,
									   Candidates.Num() < CVarParams2.EnvCollisionsLineTraceBatchSize);

	// Process results.
	for (int32 i = 0; i < Candidates.Num(); i++)
	{
		const int32 TetVertexIndex = Candidates[i].Index;
		const FVector& WorldOrigin = Candidates[i].Origin;
		const FVector& WorldSkinnedPos = Candidates[i].Target;
		if (HitResult[i])
		{
			FHitResult& Hit = HitBuffer[i];

			if (GameplayColllisions.RigBoundRayCasts.EnvironmentCollisionsSkipList.Contains(Hit.Component))
			{
				continue;
			}
			GeometryCollection::Facades::FConstraintOverridesTargetData CnstrData;
			CnstrData.VertexIndex = TetVertexIndex;
			CnstrData.PositionTarget = ToSingle(WorldToComponentXf.TransformPosition(Hit.ImpactPoint)); // Component space
			CnstrOutputs.Add(CnstrData);

			EnvCollisionsPreviousHits.Add(TetVertexIndex);

#if WITH_EDITOR
			if (CVarParams2.IsDebugDrawingEnabled() && CVarParams2.bDoDrawSceneRaycasts)
			{
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(WorldOrigin, WorldSkinnedPos, 2.0f, FColor::Red, false, -1.0f, 0, 0.75f);
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugPoint(Hit.ImpactPoint, FColor::Yellow, false, -1.0f, 0, 5);
			}
#endif
		}
		else
		{
#if WITH_EDITOR
			if (CVarParams2.IsDebugDrawingEnabled() && CVarParams2.bDoDrawSceneRaycasts)
			{
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(WorldOrigin, WorldSkinnedPos, 2.0f, FColor::Green, false, -1.0f, 0, 0.5f);
			}
#endif
		}
	}
}

#undef LOCTEXT_NAMESPACE
