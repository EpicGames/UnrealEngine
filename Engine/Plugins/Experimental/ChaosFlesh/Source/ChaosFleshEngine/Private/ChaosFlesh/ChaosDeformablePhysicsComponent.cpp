// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFlesh/ChaosDeformablePhysicsComponent.h"

#include "ChaosFlesh/ChaosDeformableSolverActor.h"
#include "ChaosFlesh/ChaosDeformableSolverComponent.h"
#include "ChaosFlesh/FleshComponent.h"


#if WITH_EDITOR
void UDeformablePhysicsComponent::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedPtr<IPropertyHandle>bApplyImpulseOnDamageProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFleshComponent, bApplyImpulseOnDamage), UPrimitiveComponent::StaticClass());
	bApplyImpulseOnDamageProperty->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> bIgnoreRadialImpulseProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFleshComponent, bIgnoreRadialImpulse), UPrimitiveComponent::StaticClass());
	bIgnoreRadialImpulseProperty->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> bIgnoreRadialForceProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFleshComponent, bIgnoreRadialForce), UPrimitiveComponent::StaticClass());
	bIgnoreRadialForceProperty->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> bReplicatePhysicsToAutonomousProxyProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFleshComponent, bReplicatePhysicsToAutonomousProxy), UPrimitiveComponent::StaticClass());
	bReplicatePhysicsToAutonomousProxyProperty->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> ShouldUpdatePhysicsVolumnProperty = DetailBuilder.GetProperty("bShouldUpdatePhysicsVolume", USceneComponent::StaticClass());
	ShouldUpdatePhysicsVolumnProperty->MarkHiddenByCustomization();

	//
	// Some BodyInstance properties are managed through FBodyInstanceCustomizationHelper
	// For example, this is not needed here as its overwritten in the helper.
	//
	// TSharedPtr<IPropertyHandle> BodyInstance_bSimulatePhysicsProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFleshComponent, BodyInstance.bSimulatePhysics), UPrimitiveComponent::StaticClass());
	// BodyInstance_bSimulatePhysicsProperty->MarkHiddenByCustomization();
	//
	// Properties not managed in the Helper are hidden here:

	TSharedPtr<IPropertyHandle> SleepFamilyProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFleshComponent, BodyInstance.SleepFamily), UPrimitiveComponent::StaticClass());
	SleepFamilyProperty->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> InertiaTensorScaleProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFleshComponent, BodyInstance.InertiaTensorScale), UPrimitiveComponent::StaticClass());
	InertiaTensorScaleProperty->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> CustomSleepThresholdMultiplierProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFleshComponent, BodyInstance.CustomSleepThresholdMultiplier), UPrimitiveComponent::StaticClass());
	CustomSleepThresholdMultiplierProperty->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> StabilizationThresholdMultiplierProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFleshComponent, BodyInstance.StabilizationThresholdMultiplier), UPrimitiveComponent::StaticClass());
	StabilizationThresholdMultiplierProperty->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> GenerateWakeEventsProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFleshComponent, BodyInstance.bGenerateWakeEvents), UPrimitiveComponent::StaticClass());
	GenerateWakeEventsProperty->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> UpdateMassWhenScaleChangesProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFleshComponent, BodyInstance.bUpdateMassWhenScaleChanges), UPrimitiveComponent::StaticClass());
	UpdateMassWhenScaleChangesProperty->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> MaxAngularVelocityProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFleshComponent, BodyInstance.MaxAngularVelocity), UPrimitiveComponent::StaticClass());
	MaxAngularVelocityProperty->MarkHiddenByCustomization();
}
#endif

void UDeformablePhysicsComponent::OnCreatePhysicsState()
{
	Super::OnCreatePhysicsState();
	if (PrimarySolverComponent)
	{
		FDeformableSolver::FGameThreadAccess GameThreadSolver = PrimarySolverComponent->GameThreadAccess();
		if (GameThreadSolver())
		{
			AddProxy(GameThreadSolver);
		}
	}
}

void UDeformablePhysicsComponent::OnDestroyPhysicsState()
{
	Super::OnDestroyPhysicsState();
	if (PrimarySolverComponent)
	{
		FDeformableSolver::FGameThreadAccess GameThreadSolver = PrimarySolverComponent->GameThreadAccess();
		if (GameThreadSolver())
		{
			RemoveProxy(GameThreadSolver);
		}
	}
	PhysicsProxy = nullptr;
}

bool UDeformablePhysicsComponent::ShouldCreatePhysicsState() const
{
	return true;
}
bool UDeformablePhysicsComponent::HasValidPhysicsState() const
{
	return PhysicsProxy != nullptr;
}

void UDeformablePhysicsComponent::AddProxy(Chaos::Softs::FDeformableSolver::FGameThreadAccess& GameThreadSolver)
{
	PhysicsProxy = NewProxy();
	if (PhysicsProxy)
	{
		// PhysicsProxy is created on game thread but is owned by physics thread. This is the handoff. 
		GameThreadSolver.AddProxy(PhysicsProxy);
	}
}

void UDeformablePhysicsComponent::RemoveProxy(Chaos::Softs::FDeformableSolver::FGameThreadAccess& GameThreadSolver)
{
	if (PhysicsProxy)
	{
		GameThreadSolver.RemoveProxy(PhysicsProxy);
		PhysicsProxy = nullptr; // destroyed on physics thread. 
	}
}

UDeformableSolverComponent* UDeformablePhysicsComponent::GetDeformableSolver()
{
	return PrimarySolverComponent.Get();
}
const UDeformableSolverComponent* UDeformablePhysicsComponent::GetDeformableSolver() const
{ 
	return PrimarySolverComponent.Get();
}

UDeformablePhysicsComponent::UDeformablePhysicsComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UDeformablePhysicsComponent::DisableSimulation()
{
	if (PrimarySolverComponent)
	{
		if (!PrimarySolverComponent->ConnectedObjects.DeformableComponents.Contains(this))
		{
			PrimarySolverComponent->ConnectedObjects.DeformableComponents.Remove(this);
		}
		PrimarySolverComponent->RemoveDeformableProxy(this);
	}
	PrimarySolverComponent = nullptr;
}

void UDeformablePhysicsComponent::EnableSimulation(UDeformableSolverComponent* DeformableSolverComponent)
{
	if (DeformableSolverComponent)
	{
		PrimarySolverComponent = DeformableSolverComponent;
		if (!DeformableSolverComponent->ConnectedObjects.DeformableComponents.Contains(this))
		{
			DeformableSolverComponent->ConnectedObjects.DeformableComponents.Add(this);
		}
		DeformableSolverComponent->AddDeformableProxy(this);
	}
}


void UDeformablePhysicsComponent::EnableSimulationFromActor(ADeformableSolverActor* DeformableSolverActor)
{
	if (DeformableSolverActor && DeformableSolverActor->GetDeformableSolverComponent())
	{
		PrimarySolverComponent = DeformableSolverActor->GetDeformableSolverComponent();
		if (!DeformableSolverActor->GetDeformableSolverComponent()->ConnectedObjects.DeformableComponents.Contains(this))
		{
			DeformableSolverActor->GetDeformableSolverComponent()->ConnectedObjects.DeformableComponents.Add(this);
		}
		DeformableSolverActor->GetDeformableSolverComponent()->AddDeformableProxy(this);
	}
}







