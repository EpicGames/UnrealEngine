// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFlesh/ChaosDeformableSolverActor.h"

#include "ChaosFlesh/ChaosDeformableSolverComponent.h"
#include "Engine/Texture2D.h"
#include "UObject/ConstructorHelpers.h"

DEFINE_LOG_CATEGORY_STATIC(LogChaosDeformableSolverActor, Log, All);

ADeformableSolverActor::ADeformableSolverActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	//UE_LOGF(ADeformableSolverActor, Verbose, "ADeformableSolverActor::ADeformableSolverActor()");
	SolverComponent = CreateDefaultSubobject<UDeformableSolverComponent>(TEXT("DeformableSolverComponent0"));
	RootComponent = SolverComponent;
	RootComponent->Mobility = EComponentMobility::Static;
	CreateBillboardIcon(ObjectInitializer);
}

#if WITH_EDITOR
void ADeformableSolverActor::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedPtr<IPropertyHandle> bAsyncPhysicsTickEnabledProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ADeformableSolverActor, bAsyncPhysicsTickEnabled), AActor::StaticClass());
	bAsyncPhysicsTickEnabledProperty->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> ShouldUpdatePhysicsVolumnProperty = DetailBuilder.GetProperty("bShouldUpdatePhysicsVolume", USceneComponent::StaticClass());
	ShouldUpdatePhysicsVolumnProperty->MarkHiddenByCustomization();
}

#endif

void ADeformableSolverActor::CreateBillboardIcon(const FObjectInitializer& ObjectInitializer)
{
	/*
	* Display icon in the editor
	*/
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		// A helper class object we use to find target UTexture2D object in resource package
		ConstructorHelpers::FObjectFinderOptional<UTexture2D> SolverTextureObject;

		// Icon sprite category name
		FName ID_Solver;

		// Icon sprite display name
		FText NAME_Solver;

		FConstructorStatics()
			// Use helper class object to find the texture
			// "/Engine/EditorResources/S_Solver" is resource path
			: SolverTextureObject(TEXT("/Engine/EditorResources/S_Solver.S_Solver"))
			, ID_Solver(TEXT("Solver"))
			, NAME_Solver(NSLOCTEXT("SpriteCategory", "Solver", "Solver"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;


#if WITH_EDITORONLY_DATA
	SpriteComponent = ObjectInitializer.CreateEditorOnlyDefaultSubobject<UBillboardComponent>(this, TEXT("Sprite"));
	if (SpriteComponent)
	{
		SpriteComponent->Sprite = ConstructorStatics.SolverTextureObject.Get();		// Get the sprite texture from helper class object
		SpriteComponent->SpriteInfo.Category = ConstructorStatics.ID_Solver;		// Assign sprite category name
		SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Solver;	// Assign sprite display name
		SpriteComponent->AttachToComponent(RootComponent, FAttachmentTransformRules::KeepRelativeTransform);
		SpriteComponent->Mobility = EComponentMobility::Static;
	}
#endif // WITH_EDITORONLY_DATA
}


