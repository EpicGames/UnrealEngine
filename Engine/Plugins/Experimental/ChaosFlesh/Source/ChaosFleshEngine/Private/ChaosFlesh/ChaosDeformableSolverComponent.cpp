// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFlesh/ChaosDeformableSolverComponent.h"

#include "ChaosFlesh/ChaosDeformableCollisionsComponent.h"
#include "ChaosFlesh/ChaosDeformablePhysicsComponent.h"
#include "ChaosFlesh/ChaosDeformableSolverActor.h"
#include "Engine/World.h"
#include "ChaosStats.h"

DEFINE_LOG_CATEGORY_STATIC(LogDeformableSolverComponentInternal, Log, All);

DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.UDeformableSolverComponent.UpdateDeformableEndTickState"), STAT_ChaosDeformable_UDeformableSolverComponent_UpdateDeformableEndTickState, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.UDeformableSolverComponent.TickComponent"), STAT_ChaosDeformable_UDeformableSolverComponent_TickComponent, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.UDeformableSolverComponent.Reset"), STAT_ChaosDeformable_UDeformableSolverComponent_Reset, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.UDeformableSolverComponent.AddDeformableProxy"), STAT_ChaosDeformable_UDeformableSolverComponent_AddDeformableProxy, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.UDeformableSolverComponent.RemoveDeformableProxy"), STAT_ChaosDeformable_UDeformableSolverComponent_RemoveDeformableProxy, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.UDeformableSolverComponent.Simulate"), STAT_ChaosDeformable_UDeformableSolverComponent_Simulate, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.UDeformableSolverComponent.UpdateFromGameThread"), STAT_ChaosDeformable_UDeformableSolverComponent_UpdateFromGameThread, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.UDeformableSolverComponent.UpdateFromSimulation"), STAT_ChaosDeformable_UDeformableSolverComponent_UpdateFromSimulation, STATGROUP_Chaos);

static TAutoConsoleVariable<int32> CVarDeformablePhysicsTickWaitForParallelDeformableTask(
	TEXT("p.ClothPhysics.WaitForParallelDeformableTask"), 0, 
	TEXT("If 1, always wait for deformable task completion in the Deformable Tick function. "\
		 "If 0, wait at end - of - frame updates instead if allowed by component settings"));

FChaosEngineDeformableCVarParams GChaosEngineDeformableCVarParams;
FAutoConsoleVariableRef CVarChaosEngineDeformableSolverbEnabled(TEXT("p.Chaos.Deformable.EnableSimulation"), GChaosEngineDeformableCVarParams.bEnableDeformableSolver, TEXT("Enable the deformable simulation. [default : true]"));

UDeformableSolverComponent::UDeformableSolverComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, FleshSolverProxy()
{
	PrimaryComponentTick.bCanEverTick = true;
	bTickInEditor = false;

	UpdateTickGroup();
}

UDeformableSolverComponent::~UDeformableSolverComponent()
{
}

#if WITH_EDITOR
void UDeformableSolverComponent::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedPtr<IPropertyHandle> ShouldUpdatePhysicsVolumnProperty = DetailBuilder.GetProperty("bShouldUpdatePhysicsVolume", USceneComponent::StaticClass());
	ShouldUpdatePhysicsVolumnProperty->MarkHiddenByCustomization();
}
#endif

void UDeformableSolverComponent::UpdateTickGroup()
{
	//
	// OR THIS SolverData->PreSolveHandle  = Solver->AddPreAdvanceCallback(FSolverPreAdvance::FDelegate::CreateUObject(this, &AChaosCacheManager::HandlePreSolve, Solver));
	// see : CacheManagerActor.cpp::348
	
	//
	if (SolverTiming.ExecutionModel == EDeformableExecutionModel::Chaos_Deformable_PrePhysics)
	{
		PrimaryComponentTick.TickGroup = TG_PrePhysics;
		DeformableEndTickFunction.TickGroup = TG_PrePhysics;
	}
	else if (SolverTiming.ExecutionModel == EDeformableExecutionModel::Chaos_Deformable_PostPhysics)
	{
		PrimaryComponentTick.TickGroup = TG_PostPhysics;
		DeformableEndTickFunction.TickGroup = TG_LastDemotable;
	}
	else //EDeformableExecutionModel::Chaos_Deformable_DuringPhysics
	{
		PrimaryComponentTick.TickGroup = TG_PrePhysics;
		DeformableEndTickFunction.TickGroup = TG_PostPhysics;
	}

	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bTickEvenWhenPaused = false;

	if (SolverTiming.bDoThreadedAdvance)
	{
		DeformableEndTickFunction.bCanEverTick = true;
		DeformableEndTickFunction.bStartWithTickEnabled = true;
	}
	else
	{
		DeformableEndTickFunction.bCanEverTick = false;
		DeformableEndTickFunction.bStartWithTickEnabled = false;
	}
}


UDeformableSolverComponent::FDeformableSolver::FGameThreadAccess
UDeformableSolverComponent::GameThreadAccess()
{
	return FDeformableSolver::FGameThreadAccess(FleshSolverProxy.Solver.Get(), Chaos::Softs::FGameThreadAccessor());
}

UDeformableSolverComponent::FDeformableSolver::FPhysicsThreadAccess
UDeformableSolverComponent::PhysicsThreadAccess()
{
	return FDeformableSolver::FPhysicsThreadAccess(FleshSolverProxy.Solver.Get(), Chaos::Softs::FPhysicsThreadAccessor());
}

bool UDeformableSolverComponent::IsSimulatable() const
{
	return true;
}

bool UDeformableSolverComponent::IsSimulating(UDeformablePhysicsComponent* InComponent) const
{
	if (InComponent)
	{
		const UDeformableSolverComponent* ComponentSolver = InComponent->PrimarySolverComponent.Get();
		return (void*)ComponentSolver == (void*)this;
	}
	return false;
}




void UDeformableSolverComponent::UpdateDeformableEndTickState(bool bRegister)
{
	SCOPE_CYCLE_COUNTER(STAT_ChaosDeformable_UDeformableSolverComponent_UpdateDeformableEndTickState);
	TRACE_CPUPROFILER_EVENT_SCOPE(ChaosDeformable_UDeformableSolverComponent_UpdateDeformableEndTickState);

	bRegister &= PrimaryComponentTick.IsTickFunctionRegistered();
	if (SolverTiming.bDoThreadedAdvance)
	{
		if (bRegister != DeformableEndTickFunction.IsTickFunctionRegistered())
		{
			if (bRegister)
			{
				UWorld* World = GetWorld();
				if (World->EndPhysicsTickFunction.IsTickFunctionRegistered() && SetupActorComponentTickFunction(&DeformableEndTickFunction))
				{
					DeformableEndTickFunction.DeformableSolverComponent = this;
					// Make sure our EndPhysicsTick gets called after physics simulation is finished
					if (World != nullptr)
					{
						DeformableEndTickFunction.AddPrerequisite(this, PrimaryComponentTick);
					}
				}
			}
			else
			{
				DeformableEndTickFunction.UnRegisterTickFunction();
			}
		}
	}
	else if(DeformableEndTickFunction.IsTickFunctionRegistered())
	{
		DeformableEndTickFunction.UnRegisterTickFunction();
	}

}

void UDeformableSolverComponent::BeginPlay()
{
	Super::BeginPlay();
	if(!SimulationAsset.DataflowAsset)
	{
		BuildSimulationProxy();
	}
}

void UDeformableSolverComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	SCOPE_CYCLE_COUNTER(STAT_ChaosDeformable_UDeformableSolverComponent_TickComponent);
	TRACE_CPUPROFILER_EVENT_SCOPE(ChaosDeformable_UDeformableSolverComponent_TickComponent);

	if (GChaosEngineDeformableCVarParams.bEnableDeformableSolver)
	{
		UpdateTickGroup();

		UpdateDeformableEndTickState(IsSimulatable());

		// We only run the simulation if no dataflow solver has been defined
		if(!SimulationAsset.DataflowAsset)
		{
			if(bSimulationTicking)
			{
				WriteToSimulation(DeltaTime, false);

				if (SolverTiming.bDoThreadedAdvance)
				{
					// see FParallelClothCompletionTask
					FGraphEventArray Prerequisites;
					Prerequisites.Add(ParallelDeformableTask);
					FGraphEventRef DeformableCompletionEvent = TGraphTask<FParallelDeformableTask>::CreateTask(&Prerequisites, ENamedThreads::GameThread).ConstructAndDispatchWhenReady(this, DeltaTime);
					ThisTickFunction->GetCompletionHandle()->DontCompleteUntil(DeformableCompletionEvent);
				}
				else
				{
					Simulate(DeltaTime);

					ReadFromSimulation(DeltaTime, false);
				}
			}
		}
	}
}

void UDeformableSolverComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);
	if(!SimulationAsset.DataflowAsset)
	{
		ResetSimulationProxy();
	}
}

void UDeformableSolverComponent::ResetSimulationProperties(const FSolverTimingGroup& TimingGroup, const FSolverEvolutionGroup& EvolutionGroup,
		FSolverCollisionsGroup CollisionsGroup, FSolverConstraintsGroup ConstraintsGroup, FSolverForcesGroup ForcesGroup,
		FSolverDebuggingGroup DebuggingGroup, FSolverMuscleActivationGroup MuscleActivationGroup)
{
	SolverTiming = TimingGroup;
	SolverEvolution = EvolutionGroup;
	SolverCollisions = CollisionsGroup;
	SolverConstraints = ConstraintsGroup;
	SolverForces = ForcesGroup;
	SolverDebugging = DebuggingGroup;
	SolverMuscleActivation = MuscleActivationGroup;

	if(FleshSolverProxy.IsValid())
	{
		FDeformableSolver::FPhysicsThreadAccess PhysicsThreadSolver = PhysicsThreadAccess();
		PhysicsThreadSolver.Reset(Chaos::Softs::FDeformableSolverProperties(
			SolverTiming.NumSubSteps
			, SolverTiming.NumSolverIterations
			, SolverTiming.FixTimeStep
			, SolverTiming.TimeStepSize
			, SolverDebugging.CacheToFile
			, SolverConstraints.bEnableKinematics
			, SolverCollisions.bUseFloor
			, false /*SolverCollisions.SolverGridBasedCollisions.bUseGridBasedConstraints*/
			, 25. /*SolverCollisions.SolverGridBasedCollisions.GridDx*/
			, SolverEvolution.SolverQuasistatics.bDoQuasistatics
			, SolverConstraints.CorotatedConstraints.bDoBlended
			, SolverConstraints.CorotatedConstraints.BlendedZeta
			, SolverForces.bEnableGravity
			, SolverConstraints.CorotatedConstraints.bEnableCorotatedConstraint
			, SolverConstraints.bEnablePositionTargets
			, SolverConstraints.GaussSeidelConstraints.bUseGaussSeidelConstraints
			, SolverConstraints.GaussSeidelConstraints.bUseSOR
			, SolverConstraints.GaussSeidelConstraints.OmegaSOR
			, SolverConstraints.GaussSeidelConstraints.bUseGSNeohookean
			, SolverConstraints.GaussSeidelConstraints.SpringCollision.bDoSpringCollision
			, SolverConstraints.GaussSeidelConstraints.SpringCollision.InComponentSpringCollision.bDoInComponentSpringCollision
			, SolverConstraints.GaussSeidelConstraints.SpringCollision.InComponentSpringCollision.NRingExcluded
			, SolverConstraints.GaussSeidelConstraints.SpringCollision.CollisionSearchRadius
			, SolverConstraints.GaussSeidelConstraints.SpringCollision.SpringCollisionStiffness
			, SolverConstraints.GaussSeidelConstraints.SpringCollision.bAllowSliding
			, SolverConstraints.GaussSeidelConstraints.SphereRepulsion.bDoSphereRepulsion
			, SolverConstraints.GaussSeidelConstraints.SphereRepulsion.SphereRepulsionRadius
			, SolverConstraints.GaussSeidelConstraints.SphereRepulsion.SphereRepulsionStiffness
			, SolverMuscleActivation.GetDoLengthBasedMuscleActivation()
			, SolverMuscleActivation.bOverrideMuscleActivationWithAnimatedCurves
			, SolverConstraints.GaussSeidelConstraints.SpringCollision.bCollideWithFullmesh
			, SolverConstraints.GaussSeidelConstraints.bEnableDynamicSprings
		));
	}
}

void UDeformableSolverComponent::BuildSimulationProxy()
{
	SCOPE_CYCLE_COUNTER(STAT_ChaosDeformable_UDeformableSolverComponent_Reset);
	TRACE_CPUPROFILER_EVENT_SCOPE(ChaosDeformable_UDeformableSolverComponent_Reset);

	if (GChaosEngineDeformableCVarParams.bEnableDeformableSolver)
	{
		FleshSolverProxy.Solver = MakeUnique<Chaos::Softs::FDeformableSolver>(Chaos::Softs::FDeformableSolverProperties(
			SolverTiming.NumSubSteps
			, SolverTiming.NumSolverIterations
			, SolverTiming.FixTimeStep
			, SolverTiming.TimeStepSize
			, SolverDebugging.CacheToFile
			, SolverConstraints.bEnableKinematics
			, SolverCollisions.bUseFloor
			, false /*SolverCollisions.SolverGridBasedCollisions.bUseGridBasedConstraints*/
			, 25. /*SolverCollisions.SolverGridBasedCollisions.GridDx*/
			, SolverEvolution.SolverQuasistatics.bDoQuasistatics
			, SolverConstraints.CorotatedConstraints.bDoBlended
			, SolverConstraints.CorotatedConstraints.BlendedZeta
			, SolverForces.bEnableGravity
			, SolverConstraints.CorotatedConstraints.bEnableCorotatedConstraint
			, SolverConstraints.bEnablePositionTargets
			, SolverConstraints.GaussSeidelConstraints.bUseGaussSeidelConstraints
			, SolverConstraints.GaussSeidelConstraints.bUseSOR
			, SolverConstraints.GaussSeidelConstraints.OmegaSOR
			, SolverConstraints.GaussSeidelConstraints.bUseGSNeohookean
			, SolverConstraints.GaussSeidelConstraints.SpringCollision.bDoSpringCollision
			, SolverConstraints.GaussSeidelConstraints.SpringCollision.InComponentSpringCollision.bDoInComponentSpringCollision
			, SolverConstraints.GaussSeidelConstraints.SpringCollision.InComponentSpringCollision.NRingExcluded
			, SolverConstraints.GaussSeidelConstraints.SpringCollision.CollisionSearchRadius
			, SolverConstraints.GaussSeidelConstraints.SpringCollision.SpringCollisionStiffness
			, SolverConstraints.GaussSeidelConstraints.SpringCollision.bAllowSliding
			, SolverConstraints.GaussSeidelConstraints.SphereRepulsion.bDoSphereRepulsion
			, SolverConstraints.GaussSeidelConstraints.SphereRepulsion.SphereRepulsionRadius
			, SolverConstraints.GaussSeidelConstraints.SphereRepulsion.SphereRepulsionStiffness
			, SolverMuscleActivation.GetDoLengthBasedMuscleActivation()
			, SolverMuscleActivation.bOverrideMuscleActivationWithAnimatedCurves
			, SolverConstraints.GaussSeidelConstraints.SpringCollision.bCollideWithFullmesh
			, SolverConstraints.GaussSeidelConstraints.bEnableDynamicSprings
		));

		for (TObjectPtr<UDeformablePhysicsComponent>& DeformableComponent : ConnectedObjects.DeformableComponents)
		{
			if( DeformableComponent )
			{
				if (IsSimulating(DeformableComponent))
				{
					AddDeformableProxy(DeformableComponent);
				}
			}
		}
	}
}

void UDeformableSolverComponent::ResetSimulationProxy()
{
	for (TObjectPtr<UDeformablePhysicsComponent>& DeformableComponent : ConnectedObjects.DeformableComponents)
	{
		if( DeformableComponent )
		{
			if (IsSimulating(DeformableComponent))
			{
				RemoveDeformableProxy(DeformableComponent);
			}
		}
	}
	FleshSolverProxy.Solver.Reset();
}

void UDeformableSolverComponent::RemoveDeformableProxy(UDeformablePhysicsComponent* InComponent)
{
	SCOPE_CYCLE_COUNTER(STAT_ChaosDeformable_UDeformableSolverComponent_RemoveDeformableProxy);
	TRACE_CPUPROFILER_EVENT_SCOPE(ChaosDeformable_UDeformableSolverComponent_RemoveDeformableProxy);

	if (FleshSolverProxy.Solver && IsSimulating(InComponent))
	{
		FDeformableSolver::FGameThreadAccess GameThreadSolver = GameThreadAccess();
		if (!GameThreadSolver.HasObject(InComponent))
		{
			InComponent->RemoveProxy(GameThreadSolver);
		}
	}
}

void UDeformableSolverComponent::AddDeformableProxy(UDeformablePhysicsComponent* InComponent)
{
	SCOPE_CYCLE_COUNTER(STAT_ChaosDeformable_UDeformableSolverComponent_AddDeformableProxy);
	TRACE_CPUPROFILER_EVENT_SCOPE(ChaosDeformable_UDeformableSolverComponent_AddDeformableProxy);

	if (FleshSolverProxy.Solver && IsSimulating(InComponent))
	{
		FDeformableSolver::FGameThreadAccess GameThreadSolver = GameThreadAccess();
		if (!GameThreadSolver.HasObject(InComponent))
		{
			InComponent->AddProxy(GameThreadSolver);
		}
	}
}

void UDeformableSolverComponent::Simulate(float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_ChaosDeformable_UDeformableSolverComponent_Simulate);
	TRACE_CPUPROFILER_EVENT_SCOPE(ChaosDeformable_UDeformableSolverComponent_Simulate);

	if (FleshSolverProxy.Solver)
	{
		// @todo(accessor) : Should be coming from the threading class. 
		FDeformableSolver::FPhysicsThreadAccess PhysicsThreadSolver = PhysicsThreadAccess();
		if(!SimulationAsset.DataflowAsset)
		{
			PhysicsThreadSolver.Simulate(DeltaTime);
		}
	}
}

void UDeformableSolverComponent::WriteToSimulation(float DeltaTime, const bool bAsyncTask)
{
	SCOPE_CYCLE_COUNTER(STAT_ChaosDeformable_UDeformableSolverComponent_UpdateFromGameThread);
	TRACE_CPUPROFILER_EVENT_SCOPE(ChaosDeformable_UDeformableSolverComponent_UpdateFromGameThread);

	if (FleshSolverProxy.Solver)
	{
		FDeformableSolver::FGameThreadAccess GameThreadSolver = GameThreadAccess();

		Chaos::Softs::FDeformableDataMap DataMap;
		for (TObjectPtr<UDeformablePhysicsComponent>& DeformableComponent : ConnectedObjects.DeformableComponents)
		{
			if (DeformableComponent)
			{
				if (IsSimulating(DeformableComponent))
				{
					DeformableComponent->PreSolverUpdate();
					if (FDataMapValue Value = DeformableComponent->NewDeformableData())
					{
						DataMap.Add(DeformableComponent, Value);
					}
				}
			}
		}

		GameThreadSolver.PushInputPackage(GameThreadSolver.GetFrame(), MoveTemp(DataMap));
	}
}

void UDeformableSolverComponent::ReadFromSimulation(float DeltaTime, const bool bAsyncTask)
{
	SCOPE_CYCLE_COUNTER(STAT_ChaosDeformable_UDeformableSolverComponent_UpdateFromSimulation);
	TRACE_CPUPROFILER_EVENT_SCOPE(ChaosDeformable_UDeformableSolverComponent_UpdateFromSimulation);

	if (FleshSolverProxy.Solver)
	{
		FDeformableSolver::FGameThreadAccess GameThreadSolver = GameThreadAccess();
		TUniquePtr<FDeformablePackage> Output(nullptr);
		while (TUniquePtr<FDeformablePackage> SolverOutput = GameThreadSolver.PullOutputPackage())
		{
			Output.Reset(SolverOutput.Release());
		}

		if (Output)
		{
			for (TObjectPtr<UDeformablePhysicsComponent>& DeformableComponent : ConnectedObjects.DeformableComponents)
			{
				if (DeformableComponent)
				{
					if (IsSimulating(DeformableComponent))
					{
						if (const FDataMapValue* Buffer = Output->ObjectMap.Find(DeformableComponent))
						{
							DeformableComponent->UpdateFromSimulation(Buffer);
						}
					}
				}
			}
		}
	}
}

void UDeformableSolverComponent::ReadRestartData()
{
	if (FleshSolverProxy.Solver)
	{
		FDeformableSolver::FGameThreadAccess GameThreadSolver = GameThreadAccess();

		Chaos::Softs::FDeformableDataMap DataMap;
		for (TObjectPtr<UDeformablePhysicsComponent>& DeformableComponent : ConnectedObjects.DeformableComponents)
		{
			if (DeformableComponent)
			{
				if (IsSimulating(DeformableComponent))
				{
					DeformableComponent->PreSolverUpdate();
					if (FDataMapValue Value = DeformableComponent->WriteRestartDataToSimulation())
					{
						DataMap.Add(DeformableComponent, Value);
					}
				}
			}
		}

		GameThreadSolver.PushRestartPackage(GameThreadSolver.GetFrame(), MoveTemp(DataMap));
	}
}

#if WITH_EDITOR

bool UDeformableSolverComponent::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	const FName& Name = InProperty->GetFName();

	if (Name == GET_MEMBER_NAME_CHECKED(ThisClass, SimulationAsset))
	{
		static const auto CVarEnableSimulationDataflow = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Dataflow.EnableSimulation"));
		return CVarEnableSimulationDataflow->GetBool();
	}

	return true;
}

#endif









