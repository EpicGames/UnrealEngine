// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFlesh/ChaosDeformableSolverThreading.h"

#include "ChaosFlesh/ChaosDeformableSolverComponent.h"
#include "ChaosFlesh/ChaosDeformableSolverComponent.h"

DEFINE_LOG_CATEGORY_STATIC(LogChaosDeformableSolverThreading, Log, All);

FAutoConsoleTaskPriority CChaos_FParallelDeformableTaskPriority(
	TEXT("TaskGraph.TaskPriorities.ParallelDeformableTask"),
	TEXT("Task and thread priority for parallel deformable."),
	ENamedThreads::HighThreadPriority, // if we have high priority task threads, then use them...
	ENamedThreads::NormalTaskPriority, // .. at normal task priority
	ENamedThreads::HighTaskPriority // if we don't have hi pri threads, then use normal priority threads at high task priority instead
);

void FParallelDeformableTask::DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	UE_LOGF(LogChaosDeformableSolverThreading, Verbose, "FDeformableTickFunction::ExecuteTick");
	if (DeformableSolverComponent)
	{
		DeformableSolverComponent->Simulate(DeltaTime);
	}
}

ENamedThreads::Type FParallelDeformableTask::GetDesiredThread()
{
	if (CChaos_FParallelDeformableTaskPriority.Get() != 0)
	{
		return CChaos_FParallelDeformableTaskPriority.Get();
	}
	return ENamedThreads::GameThread;
}

ESubsequentsMode::Type FParallelDeformableTask::GetSubsequentsMode()
{
	return ESubsequentsMode::TrackSubsequents;
}


void FDeformableEndTickFunction::ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	UE_LOGF(LogChaosDeformableSolverThreading, Verbose, "FDeformableTickFunction::ExecuteTick");

	if (DeformableSolverComponent)
	{
		DeformableSolverComponent->ReadFromSimulation(DeltaTime,false);
	}
}

FString FDeformableEndTickFunction::DiagnosticMessage()
{
	if (DeformableSolverComponent)
	{
		return DeformableSolverComponent->GetFullName() + TEXT("[DeformableSolverComponent]");
	}
	return TEXT("<NULL>[DeformableSolverComponent]");
}

FName FDeformableEndTickFunction::DiagnosticContext(bool bDetailed)
{
	return FName(TEXT("FDeformableTickFunction"));
}
