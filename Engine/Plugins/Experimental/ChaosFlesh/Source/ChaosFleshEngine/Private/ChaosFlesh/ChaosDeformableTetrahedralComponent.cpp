// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFlesh/ChaosDeformableTetrahedralComponent.h"

#include "Animation/SkeletalMeshActor.h"
#include "Animation/Skeleton.h"
#include "ChaosStats.h"
#include "Chaos/DebugDrawQueue.h"
#include "Chaos/Tetrahedron.h"
#include "ChaosFlesh/FleshCollection.h"
#include "ChaosFlesh/ChaosDeformableSolverActor.h"
#include "ChaosFlesh/ChaosDeformableSolverComponent.h"
#include "ChaosFlesh/ChaosDeformableTypes.h"
#include "ChaosFlesh/ChaosFleshCollectionFacade.h"
#include "ChaosFlesh/FleshCollectionEngineUtility.h"
#include "ChaosFlesh/FleshDynamicAsset.h"
#include "Components/SkeletalMeshComponent.h"
#include "Dataflow/DataflowEngineUtil.h"
#include "Engine/SkeletalMesh.h"
#include "GeometryCollection/Facades/CollectionMuscleActivationFacade.h"
#include "GeometryCollection/Facades/CollectionTransformSourceFacade.h"
#include "GeometryCollection/Facades/CollectionTetrahedralSkeletalBindingsFacade.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/TransformCollection.h"
#include "ProceduralMeshComponent.h"
#include "Rendering/SkeletalMeshRenderData.h"

#if WITH_EDITOR
#include "Rendering/SkeletalMeshModel.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosDeformableTetrahedralComponent)

FChaosEngineDeformableCVarParams CVarParams;
FAutoConsoleVariableRef CVarDeforambleDoDrawSimulationMesh(TEXT("p.Chaos.DebugDraw.Deformable.SimulationMesh"), CVarParams.bDoDrawSimulationMesh, TEXT("Debug draw the deformable simulation resutls on the game thread. [def: true]"));
FAutoConsoleVariableRef CVarDeforambleDoDrawSkeletalMeshBindingPositions(TEXT("p.Chaos.DebugDraw.Deformable.SkeletalMeshBindingPositions"), CVarParams.bDoDrawSkeletalMeshBindingPositions, TEXT("Debug draw the deformable simulation's SkeletalMeshBindingPositions on the game thread. [def: false]"));
FAutoConsoleVariableRef CVarDeforambleDoDrawSkeletalMeshBindingPositionsSimulationBlendWeight(TEXT("p.Chaos.DebugDraw.Deformable.SkeletalMeshBindingPositions.SimulationBlendWeight"), CVarParams.DrawSkeletalMeshBindingPositionsSimulationBlendWeight, TEXT("Set the simulation blend weight of the skeletal mesh debug draw.[def: 1.]"));
FAutoConsoleVariableRef CVarDeformableFleshDeformerUpdateGPUBuffersOnTick(TEXT("p.Chaos.Deformable.FleshDeformer.UpdateGPUBuffersOnTick"), CVarParams.bUpdateGPUBuffersOnTick, TEXT("Enable/disable time varying updates of GPU buffer data."));

#define PERF_SCOPE(X) SCOPE_CYCLE_COUNTER(X); TRACE_CPUPROFILER_EVENT_SCOPE(X);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.UDeformableTetrahedralComponent.TickComponent"), STAT_ChaosDeformable_UDeformableTetrahedralComponent_TickComponent, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.UDeformableTetrahedralComponent.NewProxy"), STAT_ChaosDeformable_UDeformableTetrahedralComponent_NewProxy, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.UDeformableTetrahedralComponent.NewDeformableData"), STAT_ChaosDeformable_UDeformableTetrahedralComponent_NewDeformableData, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.UDeformableTetrahedralComponent.UpdateFromSimualtion"), STAT_ChaosDeformable_UDeformableTetrahedralComponent_UpdateFromSimualtion, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.UDeformableTetrahedralComponent.RenderProceduralMesh"), STAT_ChaosDeformable_UDeformableTetrahedralComponent_RenderProceduralMesh, STATGROUP_Chaos);

DEFINE_LOG_CATEGORY_STATIC(LogDeformableTetrahedralComponentInternal, Log, All);


UDeformableTetrahedralComponent::UDeformableTetrahedralComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, GPUBufferManager(this)
{
	Mesh = ObjectInitializer.CreateDefaultSubobject<UProceduralMeshComponent>(this, TEXT("Flesh Visualization Component"));
	PrimaryComponentTick.TickGroup = TG_LastDemotable;
	PrimaryComponentTick.bCanEverTick = CVarParams.bDoDrawSimulationMesh;
	bTickInEditor = CVarParams.bDoDrawSimulationMesh;

	DynamicCollection = ObjectInitializer.CreateDefaultSubobject<UFleshDynamicAsset>(this, TEXT("Flesh Dynamic Asset"));
	SimulationCollection = ObjectInitializer.CreateDefaultSubobject<USimulationAsset>(this, TEXT("Flesh Simulation Asset"));
}

UDeformableTetrahedralComponent::~UDeformableTetrahedralComponent()
{
	if (RenderMesh) delete RenderMesh;
}

void UDeformableTetrahedralComponent::Invalidate()
{
	bBoundsNeedsUpdate = true;
}

void UDeformableTetrahedralComponent::OnRegister()
{
	if (bBoundsNeedsUpdate)
	{
		UpdateLocalBounds();
	}
	Super::OnRegister();
}

void UDeformableTetrahedralComponent::EndPlay(const EEndPlayReason::Type ReasonEnd)
{
	if (GetDynamicCollection())
	{
		GetDynamicCollection()->Reset();
	}
	if (GetSimulationCollection())
	{
		GetSimulationCollection()->Reset();
	}

	Super::EndPlay(ReasonEnd);
}

void UDeformableTetrahedralComponent::SetRestCollection(const UFleshAsset* InRestCollection)
{
	RestCollection = InRestCollection;
	ResetDynamicCollection();
	// Rebuild the physics state since the dynamic collection has changed
	RecreatePhysicsState();
	Invalidate();
	UpdateLocalBounds();
	ResetProceduralMesh();
}


UDeformablePhysicsComponent::FThreadingProxy* UDeformableTetrahedralComponent::NewProxy()
{
	PERF_SCOPE(STAT_ChaosDeformable_UDeformableTetrahedralComponent_NewProxy);

	UpdateSimSpaceTransformIndex();
	if (const UFleshAsset* RestAsset = GetRestCollection())
	{
		if (TSharedPtr<const FFleshCollection> Rest = RestAsset->GetFleshCollection())
		{
			if (Rest->NumElements(FGeometryCollection::VerticesGroup))
			{
				if (!GetDynamicCollection())
				{
					DynamicCollection = NewObject<UFleshDynamicAsset>(this, TEXT("Flesh Dynamic Asset"));
				}
				if (!GetSimulationCollection())
				{
					SimulationCollection = NewObject<USimulationAsset>(this, TEXT("Flesh Simulation Asset"));
				}

				GetDynamicCollection()->Reset(Rest.Get());
				if (const FManagedArrayCollection* Dynamic = GetDynamicCollection()->GetCollection())
				{
					// Mesh points are in component space, such that the exterior hull aligns with the
					// surface of the skeletal mesh, which is subject to the transform hierarchy.
					const FTransform& ComponentToWorldXf = GetComponentTransform();
					const FTransform ComponentToSimXf = GetSimSpaceRestTransform();
					return new FFleshThreadingProxy(
						this,
						ComponentToWorldXf,
						ComponentToSimXf,
						SimulationSpace.SimSpace,
						Rest.ToSharedRef(),
						*Dynamic);
				}
			}
		}
	}
	return nullptr;
}

UDeformablePhysicsComponent::FDataMapValue UDeformableTetrahedralComponent::NewDeformableData()
{
	PERF_SCOPE(STAT_ChaosDeformable_UDeformableTetrahedralComponent_NewDeformableData);

	using namespace GeometryCollection::Facades;
	if (GetOwner())
	{
		if (const UFleshAsset* FleshAsset = GetRestCollection())
		{
			if (TSharedPtr<const FFleshCollection> Rest = FleshAsset->GetFleshCollection())
			{
				FTransformSource TransformSource(*Rest);
				FMuscleActivationFacade MuscleActivationFacade(*Rest);

				if (TransformSource.IsValid())
				{
					TArray<USkeletalMeshComponent*> SkeletalMeshComponents;
					GetOwner()->GetComponents<USkeletalMeshComponent>(SkeletalMeshComponents);

					if (const TManagedArray<FTransform3f>* RestTransforms = Rest->FindAttribute<FTransform3f>(FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup))
					{
						auto ToDoubleLocal = [](const TArray<FTransform3f>& Src, TArray<FTransform>& Tar) {
							// @todo : Push floats through to the solver and avoid the copy. 
							Tar.AddUninitialized(Src.Num());
							for (int i = 0; i < Src.Num(); i++) Tar[i] = FTransform(Src[i]);
						};

						TArray<FTransform3f> FloatRestTransform = RestTransforms->GetConstArray();
						TArray<FTransform> AnimationTransforms; ToDoubleLocal(FloatRestTransform, AnimationTransforms);
						TArray<FTransform> ComponentPose; ToDoubleLocal(FloatRestTransform, ComponentPose);
						TArray<int32> OverrideMuscleActivationIndices;
						TArray<float> OverrideMuscleActivationValues;

						// Extract animated transforms from all skeletal meshes.
						for (const USkeletalMeshComponent* SkeletalMeshComponent : SkeletalMeshComponents)
						{
							if (const USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset())
							{
								if (const USkeleton* Skeleton = SkeletalMesh->GetSkeleton())
								{
									TSet<int32> Roots = TransformSource.GetTransformSource(Skeleton->GetName(), Skeleton->GetGuid().ToString(), SkeletalMesh->GetName());
									if (!Roots.IsEmpty() && ensureMsgf(Roots.Num() == 1, TEXT("Error: Only supports a single root per skeleton.(%s)"), *Skeleton->GetName()))
									{
										TArray<FTransform> ComponentLocalPose;
										UE::Dataflow::Animation::GlobalTransforms(SkeletalMesh->GetRefSkeleton(), ComponentLocalPose);

										const TArray<FTransform>& ComponentTransforms = SkeletalMeshComponent->GetComponentSpaceTransforms();
										if (ComponentLocalPose.Num() == ComponentTransforms.Num())
										{
											for (int Adx = Roots.Array()[0], Cdx = 0; Adx < AnimationTransforms.Num() && Cdx < ComponentTransforms.Num(); Adx++, Cdx++)
											{
												// @todo(flesh) : Can we just use one array?
												AnimationTransforms[Adx] = ComponentTransforms[Cdx];
												ComponentPose[Adx] = ComponentLocalPose[Cdx]; 

												if (SimulationSpace.SimSpaceTransformGlobalIndex == INDEX_NONE &&
													SimulationSpace.SimSpaceTransformIndex == Cdx &&
													SkeletalMesh == SimulationSpace.SimSpaceSkeletalMesh)
												{
													SimulationSpace.SimSpaceTransformGlobalIndex = Adx;
												}
											}


										}
									}
								}
							}
							
							if (MuscleActivationFacade.IsValid())
							{
#if WITH_EDITOR
								const FBlendedHeapCurve& AnimCurve = SkeletalMeshComponent->GetAnimationCurves();
								OverrideMuscleActivationIndices.Reserve(AnimCurve.Num());
								OverrideMuscleActivationValues.Reserve(AnimCurve.Num());
								AnimCurve.ForEachElement(
									[&MuscleActivationFacade, &OverrideMuscleActivationIndices, &OverrideMuscleActivationValues]
									(const UE::Anim::FCurveElement& Element)
									{
										for (int32 MuscleIdx : MuscleActivationFacade.FindMuscleIndexByCurveName(Element.Name.ToString()))
										{
											OverrideMuscleActivationIndices.Add(MuscleIdx);
											OverrideMuscleActivationValues.Add(Element.Value);
										}
									});
#endif
							}
						}

						FTransform BoneSpaceXf;
						if (AnimationTransforms.IsValidIndex(SimulationSpace.SimSpaceTransformGlobalIndex))
						{
							BoneSpaceXf = FTransform(AnimationTransforms[SimulationSpace.SimSpaceTransformGlobalIndex]);
						}
						else
						{
							BoneSpaceXf = FTransform::Identity;
						}

						return FDataMapValue(
							new Chaos::Softs::FFleshThreadingProxy::FFleshInputBuffer(
								MoveTemp(*GetSimulationCollection()->GetCollection()),
								this->GetComponentTransform(),
								BoneSpaceXf, 
								SimulationSpace.SimSpaceTransformGlobalIndex,
								MoveTemp(AnimationTransforms), 
								MoveTemp(ComponentPose), 
								BodyForces.bApplyGravity,
								BodyForces.StiffnessMultiplier,
								BodyForces.DampingMultiplier,
								MassMultiplier,
								BodyForces.IncompressibilityMultiplier,
								BodyForces.InflationMultiplier,
								MoveTemp(OverrideMuscleActivationIndices),
								MoveTemp(OverrideMuscleActivationValues),
								this));
					}
				}
			}
		}
	}
	return FDataMapValue(
		new Chaos::Softs::FFleshThreadingProxy::FFleshInputBuffer(
			MoveTemp(*GetSimulationCollection()->GetCollection()),
			this->GetComponentTransform(),
			GetSimSpaceRestTransform(),
			SimulationSpace.SimSpaceTransformGlobalIndex,
			BodyForces.bApplyGravity,
			BodyForces.StiffnessMultiplier,
			BodyForces.DampingMultiplier,
			MassMultiplier,
			BodyForces.IncompressibilityMultiplier,
			BodyForces.InflationMultiplier,
			this));
}

TArray<FString> UDeformableTetrahedralComponent::GetSimSpaceBoneNameOptions() const
{
	TArray<FString> Names;
	if (RestCollection)
	{
		if (RestCollection->SkeletalMesh)
		{
			const FReferenceSkeleton& RefSkeleton = RestCollection->SkeletalMesh->GetSkeleton()->GetReferenceSkeleton();
			Names.SetNum(RefSkeleton.GetNum());
			for (int32 i = 0; i < RefSkeleton.GetNum(); i++)
			{
				Names[i] = RefSkeleton.GetBoneName(i).ToString();
			}
		}
	}
	return Names;
}

bool UDeformableTetrahedralComponent::UpdateSimSpaceTransformIndex()
{
	SimulationSpace.SimSpaceTransformIndex = INDEX_NONE;
	SimulationSpace.SimSpaceSkeletalMesh = nullptr;

	if (SimulationSpace.SimSpace != ChaosDeformableSimSpace::Bone)
	{
		return false;
	}

	if (RestCollection && RestCollection->SkeletalMesh)
	{
		const FReferenceSkeleton& RefSkeleton = RestCollection->SkeletalMesh->GetSkeleton()->GetReferenceSkeleton();
		for (int32 i = 0; i < RefSkeleton.GetNum(); i++)
		{
			if (RefSkeleton.GetBoneName(i).ToString() == SimulationSpace.SimSpaceBoneName.ToString())
			{
				SimulationSpace.SimSpaceSkeletalMesh = RestCollection->SkeletalMesh;
				SimulationSpace.SimSpaceTransformIndex = i;
				return true;
			}
		}
	}
	return false;
}

FTransform UDeformableTetrahedralComponent::GetSimSpaceRestTransform() const
{
	if (SimulationSpace.SimSpaceSkeletalMesh == nullptr)
	{
		return FTransform::Identity;
	}
	
	TArray<FTransform> ComponentTransforms;
	ComponentTransforms.SetNum(SimulationSpace.SimSpaceSkeletalMesh->GetRefSkeleton().GetNum());

	SimulationSpace.SimSpaceSkeletalMesh->FillComponentSpaceTransforms(
		SimulationSpace.SimSpaceSkeletalMesh->GetRefSkeleton().GetRefBonePose(),
		SimulationSpace.SimSpaceSkeletalMesh->GetResourceForRendering()->LODRenderData[0].RequiredBones,
		ComponentTransforms);

	if (!ComponentTransforms.IsValidIndex(SimulationSpace.SimSpaceTransformIndex))
	{
		return FTransform::Identity;
	}
	const FTransform& ComponentToBone = ComponentTransforms[SimulationSpace.SimSpaceTransformIndex];
	return ComponentToBone;
}

void UDeformableTetrahedralComponent::UpdateFromSimulation(const FDataMapValue* SimulationBuffer)
{
	PERF_SCOPE(STAT_ChaosDeformable_UDeformableTetrahedralComponent_UpdateFromSimualtion);
	if (const FFleshThreadingProxy::FFleshOutputBuffer* FleshBuffer = (*SimulationBuffer)->As<FFleshThreadingProxy::FFleshOutputBuffer>())
	{
		if (GetDynamicCollection())
		{
			// @todo(flesh) : reduce conversions
			auto UEVertd = [](FVector3f V) { return FVector3d(V.X, V.Y, V.Z); };
			auto UEVertf = [](FVector3d V) { return FVector3f(V.X, V.Y, V.Z); };

			TManagedArray<FVector3f>& DynamicVertex = GetDynamicCollection()->GetPositions();
			const TManagedArray<FVector3f>& SimulationVertex = FleshBuffer->Dynamic.GetAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);

			TArray<FTransform3f> Transforms;
			const TManagedArray<int32>& Parent = GetRestCollection()->GetFleshCollection()->GetAttribute<int32>(FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup);
			const TManagedArray<FTransform3f>& Transform = GetRestCollection()->GetFleshCollection()->GetAttribute<FTransform3f>(FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup);
			const TManagedArray<int32>& BoneIndex = GetRestCollection()->GetFleshCollection()->GetAttribute<int32>("BoneMap", FGeometryCollection::VerticesGroup);
			GeometryCollectionAlgo::GlobalMatrices(Transform, Parent, Transforms);
			// Simulator produces results in component space.
			for (int i = DynamicVertex.Num() - 1; i >= 0; i--)
			{
				if (0 < BoneIndex[i] && BoneIndex[i] < Transforms.Num())
				{
					DynamicVertex[i] = Transforms[BoneIndex[i]].TransformPosition(SimulationVertex[i]);
				}
				else
				{
					DynamicVertex[i] = SimulationVertex[i];
				}
			}
			
			// p.Chaos.Deformable.FleshDeformer.UpdateGPUBuffersOnTick 1 (default) or 0
			if (CVarParams.bUpdateGPUBuffersOnTick)
			{
				// Update time varying GPU buffers (but only if a consumer has been registered).
				GPUBufferManager.UpdateGPUBuffers();
			}

			// p.Chaos.DebugDraw.Enabled 1
			// p.Chaos.DebugDraw.Deformable.SkeletalMeshBindingPositions 1
			if (CVarParams.bDoDrawSkeletalMeshBindingPositions)
			{
				DebugDrawSkeletalMeshBindingPositions();
			}
		}
	}
}

UDeformablePhysicsComponent::FDataMapValue UDeformableTetrahedralComponent::WriteRestartDataToSimulation()
{
	if (!GetDynamicCollection())
	{
		ResetDynamicCollection();
	}
	return FDataMapValue(
		new Chaos::Softs::FFleshThreadingProxy::FFleshRestartBuffer(*GetDynamicCollection()->GetCollection(), this));
}

void UDeformableTetrahedralComponent::UpdateLocalBounds()
{
	if (bBoundsNeedsUpdate && RestCollection)
	{
		{
			FFleshAssetEdit EditObject = RestCollection->EditCollection();
			if (FFleshCollection* Collection = EditObject.GetFleshCollection())
			{
				Collection->UpdateBoundingBox();
			}
		}
		BoundingBox = RestCollection->GetFleshCollection()->GetBoundingBox();
		bBoundsNeedsUpdate = false;
	}

}

FBoxSphereBounds UDeformableTetrahedralComponent::CalcBounds(const FTransform& LocalToWorldIn) const
{
	return BoundingBox.TransformBy(GetComponentTransform()); // todo(chaos:flesh) use LocalToWorldIn
}

void UDeformableTetrahedralComponent::ResetDynamicCollection()
{
	if (const UFleshAsset* RestAsset = GetRestCollection())
	{
		if (!GetDynamicCollection())
		{
			DynamicCollection = NewObject<UFleshDynamicAsset>(this, TEXT("Flesh Dynamic Asset"));
		}
		GetDynamicCollection()->Reset(RestAsset->GetFleshCollection().Get()); //always reset from a new copy to avoid shrinking group size

		if (!GetDynamicCollection()->GetCollection() || 
			!GetDynamicCollection()->GetCollection()->NumElements(FGeometryCollection::VerticesGroup))
		{
			ResetProceduralMesh();
		}
	}
}

//
// Rendering Support
//

void UDeformableTetrahedralComponent::BeginPlay()
{
	Super::BeginPlay();

	if (ASkeletalMeshActor* SkeletalMeshActor = Cast<ASkeletalMeshActor>(GetOwner()))
	{
		if (USkeletalMeshComponent* SkeletalMeshComponent = SkeletalMeshActor->GetSkeletalMeshComponent())
		{
			PrimaryComponentTick.AddPrerequisite(SkeletalMeshComponent, SkeletalMeshComponent->PrimaryComponentTick);
		}
	}
	if (PrimarySolverComponent)
	{
		PrimaryComponentTick.AddPrerequisite(PrimarySolverComponent.Get(), PrimarySolverComponent->PrimaryComponentTick);
	}
}

void UDeformableTetrahedralComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	PERF_SCOPE(STAT_ChaosDeformable_UDeformableTetrahedralComponent_TickComponent);
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if (IsVisible())
	{
		RenderProceduralMesh();
	}
	else
	{
		ResetProceduralMesh();
	}
}

void UDeformableTetrahedralComponent::ResetProceduralMesh()
{
	if (Mesh && RenderMesh)
	{
		Mesh->ClearAllMeshSections();
	}
	if (RenderMesh)
	{
		delete RenderMesh;
		RenderMesh = nullptr;
	}
}

void UDeformableTetrahedralComponent::RenderProceduralMesh()
{
	bool bCanRender = false;
	if (const UFleshAsset* FleshAsset = GetRestCollection())
	{
		if (IsVisible())
		{
#if WITH_EDITORONLY_DATA
			if (FleshAsset->bRenderInEditor)
#endif
			{
				if (Mesh && CVarParams.bDoDrawSimulationMesh)
				{
					PERF_SCOPE(STAT_ChaosDeformable_UDeformableTetrahedralComponent_RenderProceduralMesh);

					if (TSharedPtr<const FFleshCollection> FleshCollection = FleshAsset->GetFleshCollection())
					{
						const Chaos::FFleshCollectionFacade Flesh(*FleshCollection);

						int32 NumVertices = Flesh.NumVertices();
						int32 NumFaces = Flesh.NumFaces();
						if (NumFaces && NumVertices)
						{
							if (RenderMesh && RenderMesh->Vertices.Num() != NumFaces * 3)
							{
								ResetProceduralMesh();
							}

							if (!RenderMesh)
							{
								RenderMesh = new FFleshRenderMesh;
								TArray<FVector3f> Vertex;
								Flesh.ComponentSpaceVertices(Vertex);
								const TManagedArray<FLinearColor>* Color = FleshCollection->FindAttributeTyped<FLinearColor>(FGeometryCollection::ColorAttribute, FGeometryCollection::VerticesGroup);
								for (int i = 0; i < NumFaces; ++i)
								{
									const auto& P1 = Vertex[Flesh.Indices[i][0]];
									const auto& P2 = Vertex[Flesh.Indices[i][1]];
									const auto& P3 = Vertex[Flesh.Indices[i][2]];

									RenderMesh->Vertices.Add(FVector(P1));
									RenderMesh->Vertices.Add(FVector(P2));
									RenderMesh->Vertices.Add(FVector(P3));
									if (Color)
									{
										RenderMesh->Colors.Add((*Color)[Flesh.Indices[i][0]]);
										RenderMesh->Colors.Add((*Color)[Flesh.Indices[i][1]]);
										RenderMesh->Colors.Add((*Color)[Flesh.Indices[i][2]]);
									}
									else
									{
										RenderMesh->Colors.Add(FLinearColor::White);
										RenderMesh->Colors.Add(FLinearColor::White);
										RenderMesh->Colors.Add(FLinearColor::White);
									}

									RenderMesh->UVs.Add(FVector2D(0, 0));
									RenderMesh->UVs.Add(FVector2D(0, 0));
									RenderMesh->UVs.Add(FVector2D(0, 0));

									RenderMesh->Triangles.Add(3 * i);
									RenderMesh->Triangles.Add(3 * i + 1);
									RenderMesh->Triangles.Add(3 * i + 2);

									auto Normal = -Chaos::FVec3::CrossProduct(P3 - P1, P2 - P1);
									RenderMesh->Normals.Add(Normal);
									RenderMesh->Normals.Add(Normal);
									RenderMesh->Normals.Add(Normal);

									auto Tangent = (P2 - P1).GetSafeNormal();
									RenderMesh->Tangents.Add(FProcMeshTangent(Tangent[0], Tangent[1], Tangent[2]));
									Tangent = (P3 - P2).GetSafeNormal();
									RenderMesh->Tangents.Add(FProcMeshTangent(Tangent[0], Tangent[1], Tangent[2]));
									Tangent = (P1 - P3).GetSafeNormal();
									RenderMesh->Tangents.Add(FProcMeshTangent(Tangent[0], Tangent[1], Tangent[2]));
								}
								if (Material)
								{
									Mesh->SetMaterial(0, Material);
								}
								Mesh->SetRelativeTransform(GetComponentTransform());
								Mesh->CreateMeshSection_LinearColor(0, RenderMesh->Vertices, RenderMesh->Triangles, RenderMesh->Normals, RenderMesh->UVs, RenderMesh->Colors, RenderMesh->Tangents, false);
							}
							else
							{
								TArray<FVector3f> RenderVertex;
								Flesh.ComponentSpaceVertices(RenderVertex);
								const UFleshDynamicAsset* ConstDynamicCollection = GetDynamicCollection();
								if (ConstDynamicCollection)
								{
									const TManagedArray<FVector3f>* DynamicVertex = ConstDynamicCollection->FindPositions();
									if ((DynamicVertex->Num() > 0) && (DynamicVertex->Num() == RenderVertex.Num())) 
										RenderVertex = DynamicVertex->GetConstArray();
									auto ActivationToColor = [](float Activation) { return FLinearColor(Activation, 0.f, 1.f - Activation); };
									//update muscle activation color
									const TManagedArray<float>* VertexActivations = ConstDynamicCollection->FindActivations();
									if (VertexActivations && RenderMesh->Colors.Num() == 3 * NumFaces && VertexActivations->Num() == NumVertices)
									{
										for (int32 i = 0; i < NumFaces; ++i)
										{
											for (int j = 0; j < 3; ++j)
											{
												float Activation = (*VertexActivations)[Flesh.Indices[i][j]];
												if (Activation >= 0)
												{
													RenderMesh->Colors[3 * i + j] = ActivationToColor(Activation);
												}
												else
												{
													RenderMesh->Colors[3 * i + j] = FLinearColor::White;
												}
											}
										}
									}
								}
								auto InRange = [](int32 Size, int32 Val) { return 0 <= Val && Val < Size; };

								// Display only
								for (int i = 0; i < NumFaces; ++i)
								{
									const auto& P1 = RenderVertex[Flesh.Indices[i][0]];
									const auto& P2 = RenderVertex[Flesh.Indices[i][1]];
									const auto& P3 = RenderVertex[Flesh.Indices[i][2]];

									RenderMesh->Vertices[3 * i] = FVector(P1);
									RenderMesh->Vertices[3 * i + 1] = FVector(P2);
									RenderMesh->Vertices[3 * i + 2] = FVector(P3);

									auto Normal = Chaos::FVec3::CrossProduct(P3 - P1, P2 - P1);
									RenderMesh->Normals[3 * i] = Normal;
									RenderMesh->Normals[3 * i + 1] = Normal;
									RenderMesh->Normals[3 * i + 2] = Normal;

									auto Tangent = (P2 - P1).GetSafeNormal();
									RenderMesh->Tangents[3 * i] = FProcMeshTangent(Tangent[0], Tangent[1], Tangent[2]);
									Tangent = (P3 - P2).GetSafeNormal();
									RenderMesh->Tangents[3 * i + 1] = FProcMeshTangent(Tangent[0], Tangent[1], Tangent[2]);
									Tangent = (P1 - P3).GetSafeNormal();
									RenderMesh->Tangents[3 * i + 2] = FProcMeshTangent(Tangent[0], Tangent[1], Tangent[2]);
								}

								if (!Mesh->GetComponentTransform().Equals(GetComponentTransform())) {
									Mesh->SetRelativeTransform(GetComponentTransform());
								}
								Mesh->UpdateMeshSection_LinearColor(0, RenderMesh->Vertices, RenderMesh->Normals, RenderMesh->UVs, RenderMesh->Colors, RenderMesh->Tangents, false);
							}

							bCanRender = true;
						}
					}
				}
			}
		}
	}
	if (!bCanRender)
	{
		ResetProceduralMesh();
	}
}


void UDeformableTetrahedralComponent::DebugDrawSkeletalMeshBindingPositions() const
{
#if WITH_EDITOR
	auto UEVertd = [](FVector3f V) { return FVector3d(V.X, V.Y, V.Z); };
	float SimulationBlendWeight = CVarParams.DrawSkeletalMeshBindingPositionsSimulationBlendWeight;

	if (const UFleshAsset* RestAsset = GetRestCollection())
	{
		const USkeletalMesh* SkeletalMesh = 
			GetRestCollection() && GetRestCollection()->TargetDeformationSkeleton ? 
			GetRestCollection()->TargetDeformationSkeleton : RestAsset->SkeletalMesh;
		if (SkeletalMesh)
		{
			TArray<bool> Influenced;
			TArray<FVector> PosArray = GetSkeletalMeshEmbeddedPositionsInternal(
				ChaosDeformableBindingOption::ComponentPos, FTransform::Identity, "", SimulationBlendWeight, &Influenced);

			for (int i = 0; i < PosArray.Num(); i++)
			{
				const FVector& Pos = PosArray[i];
				if (Influenced[i])
				{
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugPoint(
						GetComponentTransform().TransformPosition(Pos), FColor::Yellow, true, 2.0f, SDPG_Foreground, 10);
				}
			}
		}
	}
#endif
}

TArray<FVector> UDeformableTetrahedralComponent::GetSkeletalMeshBindingPositions(const USkeletalMesh* InSkeletalMesh) const
{
	return GetSkeletalMeshBindingPositionsInternal(InSkeletalMesh, nullptr);
}

TArray<FVector3f> UDeformableTetrahedralComponent::GetGeometryCachePositions(const USkeletalMesh* SkeletalMesh) const
{
	TArray<FVector3f> Positions;
	if (RestCollection && DynamicCollection && SkeletalMesh)
	{
		TSharedPtr<const FFleshCollection> FleshCollection = RestCollection->GetFleshCollection();
		const TManagedArray<FVector3f>* RestVertices = RestCollection->FindPositions();
		const TManagedArray<FVector3f>* SimulatedVertices = DynamicCollection->FindPositions();
		if (FleshCollection && RestVertices && SimulatedVertices)
		{
			ChaosFlesh::BoundSurfacePositions(SkeletalMesh, FleshCollection.Get(), RestVertices, SimulatedVertices, Positions);
		}
	}
	return Positions;
}

TArray<FVector3f> UDeformableTetrahedralComponent::GetGeometryCachePositions(const UStaticMesh* StaticMesh) const
{
	TArray<FVector3f> Positions;
	if (RestCollection && DynamicCollection && StaticMesh)
	{
		TSharedPtr<const FFleshCollection> FleshCollection = RestCollection->GetFleshCollection();
		const TManagedArray<FVector3f>* RestVertices = RestCollection->FindPositions();
		const TManagedArray<FVector3f>* SimulatedVertices = DynamicCollection->FindPositions();
		if (FleshCollection && RestVertices && SimulatedVertices)
		{
			ChaosFlesh::BoundSurfacePositions(StaticMesh, FleshCollection.Get(), RestVertices, SimulatedVertices, Positions);
		}
	}
	return Positions;
}

TOptional<TArray<int32>> UDeformableTetrahedralComponent::GetMeshImportVertexMap(const USkinnedAsset& SkinnedMeshAsset) const
{
	constexpr int32 LODIndex = 0;
	const TOptional<TArray<int32>> None;
#if WITH_EDITOR
	const FSkeletalMeshModel* const MLDModel = SkinnedMeshAsset.GetImportedModel();
	if (!MLDModel)
	{
		UE_LOGF(LogDeformableTetrahedralComponentInternal, Warning, "MLDModel is empty.");
		return None;
	}
	if (!MLDModel->LODModels.IsValidIndex(LODIndex))
	{
		UE_LOGF(LogDeformableTetrahedralComponentInternal, Warning, "LODIndex %d is not a valid index.", LODIndex);
		return None;
	}
	const FSkeletalMeshLODModel& MLDLOD = MLDModel->LODModels[LODIndex];
	const TArray<int32>& Map = MLDLOD.MeshToImportVertexMap;
	if (Map.IsEmpty())
	{
		UE_LOGF(LogDeformableTetrahedralComponentInternal, Warning, "MeshToImportVertexMap is empty. MLDeformer Asset should be an imported SkeletalMesh (e.g. from fbx).");
		return None;
	}

	//
	// @todo(flesh LOD) : Add support for managing vertex mappings between skeletal LOD.
	//		The cloth/flesh asset will extract the LOD from the ManagedArrayCollection.

	TArray<FVector3f> Positions;

	const USkeletalMesh* SkeletalMeshAsset = Cast<USkeletalMesh>(&SkinnedMeshAsset);
	const TManagedArray<FVector3f>* RestVertices = RestCollection->FindPositions();
	if (SkeletalMeshAsset && RestCollection && RestVertices)
	{
		ChaosFlesh::BoundSurfacePositions(SkeletalMeshAsset, RestCollection->GetFleshCollection().Get(), RestVertices, RestVertices, Positions);

		int32 NumSections = MLDLOD.Sections.Num();
		int32 VertexOffset = 0;
		int32 MLDVertexCount = 0;
		for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
		{
			const FSkelMeshSection& MLDSection = MLDLOD.Sections[SectionIndex];
			MLDVertexCount += MLDSection.NumVertices;
		}
		int32 NumOffVertices = 0;
		for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
		{
			const FSkelMeshSection& MLDSection = MLDLOD.Sections[SectionIndex];
			if (MLDSection.NumVertices > Positions.Num()
				|| (SectionIndex == NumSections - 1 && VertexOffset + MLDSection.NumVertices != Positions.Num()))
			{
				UE_LOGF(LogDeformableTetrahedralComponentInternal, Warning, "SkeletalMeshAsset and FleshAsset have different number of vertices in section %d. Check if the assets have the same mesh.", SectionIndex);
				return None;
			}

			for (int32 VertexIndex = 0; VertexIndex < MLDSection.NumVertices; ++VertexIndex)
			{
				const FVector3f& MLDPosition = MLDSection.SoftVertices[VertexIndex].Position;
				const FVector3f& FleshPosition = Positions[VertexIndex + VertexOffset];
				if (!MLDPosition.Equals(FleshPosition, UE_KINDA_SMALL_NUMBER))
				{
					NumOffVertices++;
				}
			}
			VertexOffset += MLDSection.NumVertices;
		}
		if (NumOffVertices > 0)
		{
			UE_LOGF(LogDeformableTetrahedralComponentInternal, Warning, 
				"%d vertices bound in FleshAsset [%ls] have different vertex positions than SkeletalMeshAsset [%ls]. Check if the assets have the same vertex order or some points are not embedded."
				, NumOffVertices, *RestCollection->GetName(), *SkeletalMeshAsset->GetName());
		}
	}

	return Map;
#else
	return None;
#endif
}

bool UDeformableTetrahedralComponent::IsSkeletalMeshAnimationCompatible(const USkeletalMeshComponent* SkeletalMeshComponent) const
{
	if (SkeletalMeshComponent)
	{
		if (const UFleshAsset* FleshAsset = GetRestCollection())
		{
			if (TSharedPtr<const FFleshCollection> Rest = FleshAsset->GetFleshCollection())
			{
				GeometryCollection::Facades::FTransformSource TransformSource(*Rest);
				if (TransformSource.IsValid())
				{
					if (const USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset())
					{
						if (const USkeleton* Skeleton = SkeletalMesh->GetSkeleton())
						{
							TSet<int32> Roots = TransformSource.GetTransformSource(Skeleton->GetName(), Skeleton->GetGuid().ToString(), SkeletalMesh->GetName());
							if (!Roots.IsEmpty() && ensureMsgf(Roots.Num() == 1, TEXT("Error: Only supports a single root per skeleton.(%s)"), *Skeleton->GetName()))
							{
								return true;
							}
						}
					}
				}
			}
		}
	}
	return false;
}

TArray<FVector> UDeformableTetrahedralComponent::GetSkeletalMeshEmbeddedPositions(
	const ChaosDeformableBindingOption Format,
	const FTransform TargetDeformationSkeletonOffset,
	const FName TargetBone,
	const float SimulationBlendWeight) const
{
	return GetSkeletalMeshEmbeddedPositionsInternal(Format, TargetDeformationSkeletonOffset, TargetBone, SimulationBlendWeight, nullptr);
}

TArray<FVector> UDeformableTetrahedralComponent::GetSkeletalMeshEmbeddedPositionsInternal(
	const ChaosDeformableBindingOption Format,
	const FTransform TargetDeformationSkeletonOffset, 
	const FName TargetBone,
	const float SimulationBlendWeight,
	TArray<bool>* OutInfluence) const
{
	if (!GetRestCollection())
	{
		UE_LOGF(LogDeformableTetrahedralComponentInternal, Warning,
			"'%ls' - GetSkeletalMeshEmbeddedPositionsInternal - RestCollection is not set.",
			*GetName());
		return TArray<FVector>();
	}
	if(!GetRestCollection()->TargetDeformationSkeleton)
	{
		UE_LOGF(LogDeformableTetrahedralComponentInternal, Warning,
			"'%ls' - GetSkeletalMeshEmbeddedPositionsInternal - TargetDeformationSkeleton is not set on the flesh asset.",
			*GetName());
		return TArray<FVector>();
	}

	TArray<FVector> EmbeddedPosComp;

	// Get sample points in the skel mesh's component space.  This code assumes that the
	// skeletal mesh and the flesh asset are aligned in their respective local spaces.  If
	// they're not aligned, then the TargetDeformationSkeletonOffset should be provided to
	// put the skeletal mesh in the same place as the flesh mesh.
	TArray<FVector> TransformPositions;
	if (Format == ChaosDeformableBindingOption::WorldDelta || 
		Format == ChaosDeformableBindingOption::ComponentDelta) // BoneDelta handled below
	{
		TArray<FTransform> ComponentPose;
		UE::Dataflow::Animation::GlobalTransforms(GetRestCollection()->TargetDeformationSkeleton->GetRefSkeleton(), ComponentPose);
		TransformPositions.SetNumUninitialized(ComponentPose.Num());
		if (TargetDeformationSkeletonOffset.Equals(FTransform::Identity))
		{
			for (int32 i = 0; i < ComponentPose.Num(); i++)
			{
				TransformPositions[i] = ComponentPose[i].GetTranslation();
			}
		}
		else
		{
			for (int32 i = 0; i < ComponentPose.Num(); i++)
			{
				TransformPositions[i] =
					TargetDeformationSkeletonOffset.TransformPosition(
						ComponentPose[i].GetTranslation());
			}
		}
	}
	else
	{
		// If we aren't computing deltas, then we don't need the bone positions.  We
		// only need to size the array to however many bones we have.
		TransformPositions.SetNum(GetRestCollection()->TargetDeformationSkeleton->GetRefSkeleton().GetNum());
	}



	// World space
	if (Format == ChaosDeformableBindingOption::WorldPos)
	{
		// Calculate their current embedded positions
		EmbeddedPosComp =
			GetEmbeddedPositionsInternal(
			TransformPositions,
			FName(GetRestCollection()->TargetDeformationSkeleton->GetName()), // for identifying the binding group
			SimulationBlendWeight,
			OutInfluence);

		// Put component space points into world space
		const FTransform ComponentXf = GetComponentTransform();
		for (int32 i = 0; i < EmbeddedPosComp.Num(); i++)
		{
			EmbeddedPosComp[i] = ComponentXf.TransformPosition(EmbeddedPosComp[i]);
		}
	}
	else if (Format == ChaosDeformableBindingOption::WorldDelta)
	{
		// Calculate their current embedded positions
		EmbeddedPosComp =
			GetEmbeddedPositionsInternal(
			TransformPositions,
			FName(GetRestCollection()->TargetDeformationSkeleton->GetName()), // for identifying the binding group
			1.f,
			OutInfluence);

		const FTransform ComponentXf = GetComponentTransform();
		for (int32 i = 0; i < EmbeddedPosComp.Num(); i++)
		{
			EmbeddedPosComp[i] = EmbeddedPosComp[i] - TransformPositions[i];
			EmbeddedPosComp[i] = ComponentXf.TransformVector(EmbeddedPosComp[i]);
		}
	}

	// Component space
	else if (Format == ChaosDeformableBindingOption::ComponentPos)
	{
		EmbeddedPosComp =
			GetEmbeddedPositionsInternal(
			TransformPositions,
			FName(GetRestCollection()->TargetDeformationSkeleton->GetName()), // for identifying the binding group
			1.f,
			OutInfluence);
	}
	else if (Format == ChaosDeformableBindingOption::ComponentDelta)
	{
		// Calculate their current embedded positions
		EmbeddedPosComp =
			GetEmbeddedPositionsInternal(
			TransformPositions,
			FName(GetRestCollection()->TargetDeformationSkeleton->GetName()), // for identifying the binding group
			1.f,
			OutInfluence);

		for (int32 i = 0; i < EmbeddedPosComp.Num(); i++)
		{
			EmbeddedPosComp[i] = (EmbeddedPosComp[i] - TransformPositions[i]) * SimulationBlendWeight;
		}
	}

	// Bone space
	else
	{
		EmbeddedPosComp =
			GetEmbeddedPositionsInternal(
			TransformPositions,
			FName(GetRestCollection()->TargetDeformationSkeleton->GetName()), // for identifying the binding group
			SimulationBlendWeight,
			OutInfluence);


		// Find the component that owns TargetDeformationSkeleton, so we can pull the 
		// current animated bone transforms out of it.
		//
		// It's possible that TargetDeformationSkeleton is owned by a component on another 
		// actor, in which case, we can't (easily) find it. If that becomes a desired use
		// case, then we'll need to have a handle to that component, not just the asset.
		TArray<USkeletalMeshComponent*> SkeletalMeshComponents;
		GetOwner()->GetComponents<USkeletalMeshComponent>(SkeletalMeshComponents);
		USkeletalMeshComponent* TargetDeformationSkeletalMeshComponent = nullptr;
		for (int32 i = 0; i < SkeletalMeshComponents.Num(); i++)
		{
			if (SkeletalMeshComponents[i]->GetSkeletalMeshAsset() == GetRestCollection()->TargetDeformationSkeleton)
			{
				TargetDeformationSkeletalMeshComponent = SkeletalMeshComponents[i];
				break;
			}
		}

		if (TargetDeformationSkeletalMeshComponent)
		{
			// Get the current animated bone transforms and then their positions.
			const TArray<FTransform>& TargetDeformationComponentTransforms = 
				TargetDeformationSkeletalMeshComponent->GetComponentSpaceTransforms();
			TArray<FVector> AnimTransformPositions;
			AnimTransformPositions.SetNumUninitialized(TargetDeformationComponentTransforms.Num());

			// Get their positions, and apply the offset if necessary.
			if (!TargetDeformationSkeletonOffset.Equals(FTransform::Identity) && 
				Format == ChaosDeformableBindingOption::BoneDelta)
			{
				for (int32 i = 0; i < TargetDeformationComponentTransforms.Num(); i++)
				{
					AnimTransformPositions[i] =
						TargetDeformationSkeletonOffset.TransformPosition(
							TargetDeformationComponentTransforms[i].GetTranslation());
				}
			}
			else
			{
				for (int32 i = 0; i < TargetDeformationComponentTransforms.Num(); i++)
				{
					AnimTransformPositions[i] = TargetDeformationComponentTransforms[i].GetTranslation();
				}
			}

			// Find the transform index of 'TargetBone'
			FTransform BoneToComponentXf = FTransform::Identity;
			const int32 BoneIndex = GetRestCollection()->TargetDeformationSkeleton->GetRefSkeleton().FindBoneIndex(TargetBone);
			if (TargetDeformationComponentTransforms.IsValidIndex(BoneIndex))
			{
				BoneToComponentXf = FTransform(TargetDeformationComponentTransforms[BoneIndex].ToMatrixWithScale().Inverse());
			}
			else
			{
				UE_LOGF(LogDeformableTetrahedralComponentInternal, Warning, 
					"'%ls' - Failed to find a valid bone index (got %d) for bone name '%ls' in TargetDeformationSkeleton '%ls' "
					"corresponding to SkeletalMeshComponent '%ls', which has %d bones.", 
					*GetName(),
					BoneIndex, *TargetBone.ToString(), *GetRestCollection()->TargetDeformationSkeleton.GetName(),
					*TargetDeformationSkeletalMeshComponent->GetName(), TargetDeformationComponentTransforms.Num());
				return TArray<FVector>();
			}

			// Compute the return values
			if (Format == ChaosDeformableBindingOption::BonePos)
			{
				if (!BoneToComponentXf.Equals(FTransform::Identity))
				{
					for (int32 i = 0; i < EmbeddedPosComp.Num(); i++)
					{
						EmbeddedPosComp[i] = BoneToComponentXf.TransformPosition(EmbeddedPosComp[i]);
					}
				}
			}
			else if (Format == ChaosDeformableBindingOption::BoneDelta)
			{
				if (!BoneToComponentXf.Equals(FTransform::Identity))
				{
					for (int32 i = 0; i < EmbeddedPosComp.Num(); i++)
					{
						EmbeddedPosComp[i] = EmbeddedPosComp[i] - AnimTransformPositions[i];
						EmbeddedPosComp[i] = BoneToComponentXf.TransformVector(EmbeddedPosComp[i]);
					}
				}
				else
				{
					for (int32 i = 0; i < EmbeddedPosComp.Num(); i++)
					{
						EmbeddedPosComp[i] = EmbeddedPosComp[i] - AnimTransformPositions[i];
					}
				}
			}
		}
		else
		{
			UE_LOGF(LogDeformableTetrahedralComponentInternal, Warning, 
				"'%ls' - Failed to find SkeletalMeshComponent for TargetDeformationSkeleton '%ls'.",
				*GetName(),
				*GetRestCollection()->TargetDeformationSkeleton->GetName());
			return TArray<FVector>();
		}
	}
	return EmbeddedPosComp;
}

TArray<FVector> UDeformableTetrahedralComponent::GetEmbeddedPositionsInternal(
	const TArray<FVector>& InPositions, 
	const FName SkeletalMeshName,
	const float SimulationBlendWeight,
	TArray<bool>* OutInfluence) const
{
	auto UEVert3d = [](FVector3f V) { return FVector3d(V.X, V.Y, V.Z); };
	auto UEVert4d = [](FVector4f V) { return FVector4d(V.X, V.Y, V.Z, V.W); };

	TArray<FVector> OutPositions;
	OutPositions.Init(FVector(0.f), InPositions.Num());

	if (const UFleshAsset* RestAsset = GetRestCollection())
	{
		if (TSharedPtr<const FFleshCollection> Rest = RestAsset->GetFleshCollection())
		{
			GeometryCollection::Facades::FTetrahedralSkeletalBindings TetBindings(*Rest);

			const TManagedArray<int32>* TetrahedronStart = Rest->FindAttribute<int32>(FTetrahedralCollection::TetrahedronStartAttribute, FGeometryCollection::GeometryGroup);
			const TManagedArray<FVector3f>* RestVerts = Rest->FindAttributeTyped<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
			const TManagedArray<FVector3f>* Verts = GetDynamicCollection() ? GetDynamicCollection()->FindPositions() : RestVerts;

			if (ensure(Verts != nullptr) && TetrahedronStart)
			{
				auto CalculateBindings = [this, TetBindings, TetrahedronStart, SkeletalMeshName, OutInfluence](const TManagedArray<FVector3f>* Verts, TArray<FVector>& OutPositions)
				{
					if (OutInfluence != nullptr)
						OutInfluence->Init(false, OutPositions.Num());
					for (int32 TetMeshIdx = 0; TetMeshIdx < TetrahedronStart->Num(); TetMeshIdx++)
					{
						FString MeshBindingsName =
							GeometryCollection::Facades::FTetrahedralSkeletalBindings::GenerateMeshGroupName(TetMeshIdx, SkeletalMeshName);
						if (!TetBindings.CalculateBindings(MeshBindingsName, Verts->GetConstArray(), OutPositions, OutInfluence))
						{
							UE_LOGF(LogDeformableTetrahedralComponentInternal, Warning,
								   "'%ls' - GetEmbeddedPositionsInternal - Failed to find mesh bindings for skeletal mesh '%ls'",
								   *GetName(),
								   *SkeletalMeshName.ToString());
						}
					}
				};
				CalculateBindings(Verts, OutPositions);

				// blend between the aniamtion position and the 
				// simulated position. 
				if (!FMath::IsNearlyEqual(SimulationBlendWeight, 1.0) && GetDynamicCollection())
				{
					TArray<FVector> RestPositions = InPositions;
					CalculateBindings(RestVerts, RestPositions);

					float ClampedWeight = FMath::Clamp(SimulationBlendWeight, 0.f, 1.f);
					for (int i = 0; i < OutPositions.Num(); i++)
					{
						FVector V = RestPositions[i] - OutPositions[i];
						OutPositions[i] += V * (1.0-ClampedWeight);
					}
				}
			}
		}
	}
	return OutPositions;
}

TArray<FVector> UDeformableTetrahedralComponent::GetSkeletalMeshBindingPositionsInternal(const USkeletalMesh* InSkeletalMesh, TArray<bool>* OutInfluence) const
{
	auto UEVert3d = [](FVector3f V) { return FVector3d(V.X, V.Y, V.Z); };
	auto UEVert4d = [](FVector4f V) { return FVector4d(V.X, V.Y, V.Z, V.W); };

	TArray<FVector> TransformPositions;
	if (InSkeletalMesh)
	{
		FName SkeletalMeshName(InSkeletalMesh->GetName());
		if (const UFleshAsset* RestAsset = GetRestCollection())
		{
			if (TSharedPtr<const FFleshCollection> Rest = RestAsset->GetFleshCollection())
			{
				GeometryCollection::Facades::FTetrahedralSkeletalBindings TetBindings(*Rest);

				const TManagedArray<int32>* TetrahedronStart = Rest->FindAttribute<int32>(FTetrahedralCollection::TetrahedronStartAttribute, FGeometryCollection::GeometryGroup);
				const TManagedArray<FVector3f>* Verts = GetDynamicCollection() ? GetDynamicCollection()->FindPositions() : Rest->FindAttributeTyped<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);

				if (ensure(Verts != nullptr) && TetrahedronStart)
				{
					// Component relative transforms, not world.
					TArray<FTransform> ComponentPose;
					UE::Dataflow::Animation::GlobalTransforms(InSkeletalMesh->GetRefSkeleton(), ComponentPose);

					TransformPositions.SetNumUninitialized(ComponentPose.Num());
					for (int32 i = 0; i < ComponentPose.Num(); i++)
					{
						TransformPositions[i] = ComponentPose[i].GetTranslation();
					}

					if (OutInfluence != nullptr) OutInfluence->Init(false, TransformPositions.Num());
					for (int32 TetMeshIdx = 0; TetMeshIdx < TetrahedronStart->Num(); TetMeshIdx++)
					{
						FString MeshBindingsName = GeometryCollection::Facades::FTetrahedralSkeletalBindings::GenerateMeshGroupName(TetMeshIdx, SkeletalMeshName);
						TetBindings.CalculateBindings(MeshBindingsName, Verts->GetConstArray(), TransformPositions, OutInfluence);
					}
				}
			}
		}
	}
	return TransformPositions;
}
