// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFlesh/ChaosFleshDeformerBufferManager.h"
#include "ChaosFlesh/FleshCollection.h"
#include "ChaosFlesh/ChaosDeformableTetrahedralComponent.h"
#include "ChaosFlesh/ChaosDeformableTypes.h"
#include "ChaosFlesh/FleshDynamicAsset.h"
#include "ChaosFlesh/SimulationAsset.h"
#include "GeometryCollection/Facades/CollectionTetrahedralBindingsFacade.h"

#include "RenderResource.h"

DEFINE_LOG_CATEGORY_STATIC(LogFleshDeformerBufferManager, Log, All);

namespace Local 
{
	FChaosEngineDeformableCVarParams CVarParams;
	FAutoConsoleVariableRef CVarTestSparseMappings(
		TEXT("p.Chaos.Deformable.FleshDeformer.TestSparseMappings"),
		CVarParams.bTestSparseMappings,
		TEXT("Test flesh deformer sparsification."));
	FAutoConsoleVariableRef CVarTestCompressedIndexing(
		TEXT("p.Chaos.Deformable.FleshDeformer.TestCompressedIndexing"),
		CVarParams.bTestCompressedIndexing,
		TEXT("Test flesh deformer compressed index encodings."));
} // namespace Local

using namespace Chaos::Softs;

//=============================================================================
// Util
//=============================================================================

namespace Chaos::Softs::Private
{
	TArray<float> NullFloatBuffer;
	TArray<FFloat16> NullHalfBuffer;
	TArray<int32> NullIndexBuffer;

	template<class T_Buffer, class T_StoreData>
	void InitBuffer(
		T_Buffer& Buffer,
		const TArray<FIntVector4>* Array,
		const TArray<T_StoreData>& NullBuffer)
	{
		if (Buffer.IsInitialized())
		{
			BeginReleaseResource(&Buffer);
		}
		if (Array && Array->Num())
		{
			Buffer.Init(Array->GetData(), Array->Num());
		}
		else
		{
			Buffer.Init(NullBuffer);
		}
		BeginInitResource(&Buffer);
	}

	template<class T_Buffer, class T_StoreData>
	void InitBuffer(
		T_Buffer& Buffer,
		const TArray<uint32>* Array,
		const TArray<T_StoreData>& NullBuffer)
	{
		if (Buffer.IsInitialized())
		{
			BeginReleaseResource(&Buffer);
		}
		if (Array && Array->Num())
		{
			const uint32* Data = Array->GetData();
			Buffer.Init(Data, Array->Num());
		}
		else
		{
			Buffer.Init(NullBuffer);
		}
		BeginInitResource(&Buffer);
	}

	template<class T_Buffer, class T_StoreData>
	void InitBuffer(
		T_Buffer& Buffer,
		const TArray<float>* Array,
		const TArray<T_StoreData>& NullBuffer)
	{
		if (Buffer.IsInitialized())
		{
			BeginReleaseResource(&Buffer);
		}
		if (Array && Array->Num())
		{
			const float* Data = Array->GetData();
			Buffer.Init(Data, Array->Num());
		}
		else
		{
			Buffer.Init(NullBuffer);
		}
		BeginInitResource(&Buffer);
	}

	template<class T_Buffer, class T_StoreData>
	void InitBuffer(
		T_Buffer& Buffer,
		const TArray<FVector3f>* Array,
		const TArray<T_StoreData>& NullBuffer)
	{
		if (Buffer.IsInitialized())
		{
			BeginReleaseResource(&Buffer);
		}
		if (Array && Array->Num())
		{
			FVector3f* Data = const_cast<FVector3f*>(Array->GetData());
			Buffer.Init(&(*Data)[0], Array->Num() * 3);
		}
		else
		{
			Buffer.Init(NullBuffer);
		}
		BeginInitResource(&Buffer);
	}

	template<class T_Buffer, class T_StoreData>
	void InitBuffer(
		T_Buffer& Buffer,
		const TArray<FVector4f>* Array,
		const TArray<T_StoreData>& NullBuffer)
	{
		if (Buffer.IsInitialized())
		{
			BeginReleaseResource(&Buffer);
		}
		if (Array && Array->Num())
		{
			FVector4f* Data = const_cast<FVector4f*>(Array->GetData());
			Buffer.Init(&(*Data)[0], Array->Num() * 4);
		}
		else
		{
			Buffer.Init(NullBuffer);
		}
		BeginInitResource(&Buffer);
	}

	template <class T_Buffer, class T_Data, class T_StoreData>
	void InitBuffer(
		T_Buffer& Buffer,
		const TManagedArrayAccessor<T_Data>* ManagedArray,
		const TArray<T_StoreData>& NullBuffer)
	{
		const TArray<T_Data>* Array = ManagedArray ? &ManagedArray->Get().GetConstArray() : nullptr;
		return InitBuffer(Buffer, Array, NullBuffer);
	}

	template<class T_Buffer, class T_StoreData>
	void UpdateBuffer(
		T_Buffer& Buffer,
		const TArray<FVector3f>* Array,
		const TArray<T_StoreData>& NullBuffer)
	{
		check(Buffer.IsInitialized());
		if (Array && Array->Num())
		{
			FVector3f* Data3f = const_cast<FVector3f*>(Array->GetData());
			Buffer.Init(&(*Data3f)[0], Array->Num() * 3);
		}
		else
		{
			Buffer.Init(NullBuffer);
		}
		BeginUpdateResourceRHI(&Buffer);
	}

} // namespace Chaos::Softs::Private


//=============================================================================
// FChaosFleshDeformableGPUManager::BindingsBuffer
//=============================================================================

FChaosFleshDeformableGPUManager::FBindingsBuffer::FBindingsBuffer()
{
	VerticesBuffer.SetBufferName(FString("FleshDeformer.SparseTetVertices"));
	RestVerticesBuffer.SetBufferName(FString("FleshDeformer.RestVertices"));

	ParentsBuffer.SetBufferName(FString("FleshDeformer.TetParents"));
	VertsToParentsBuffer.SetBufferName(FString("FleshDeformer.VertsToParents"));

	WeightsBuffer.SetBufferName(FString("FleshDeformer.TetWeights"));
	OffsetsBuffer.SetBufferName(FString("FleshDeformer.SurfaceOffsets"));
	MaskBuffer.SetBufferName(FString("FleshDeformer.Mask"));

	// The vertices buffer should retain its local memory, as it'll be updated on tick.
	VerticesBuffer.RetainLocalMemory();
}

FChaosFleshDeformableGPUManager::FBindingsBuffer::~FBindingsBuffer()
{
	DestroyGPUBuffers();
}

void FChaosFleshDeformableGPUManager::FBindingsBuffer::DestroyGPUBuffers()
{
	//StartBatchedRelease();
	if (VerticesBuffer.IsInitialized())
	{
		BeginReleaseResource(&VerticesBuffer);
	}

	if (RestVerticesBuffer.IsInitialized())
	{
		BeginReleaseResource(&RestVerticesBuffer);
	}
	if (ParentsBuffer.IsInitialized())
	{
		BeginReleaseResource(&ParentsBuffer);
	}
	if (VertsToParentsBuffer.IsInitialized())
	{
		BeginReleaseResource(&VertsToParentsBuffer);
	}
	if (WeightsBuffer.IsInitialized())
	{
		BeginReleaseResource(&WeightsBuffer);
	}
	if (OffsetsBuffer.IsInitialized())
	{
		BeginReleaseResource(&OffsetsBuffer);
	}
	if (MaskBuffer.IsInitialized())
	{
		BeginReleaseResource(&MaskBuffer);
	}
	//EndBatchedRelease();
	RenderResourceDestroyFence.BeginFence();
	RenderResourceDestroyFence.Wait();
}

void FChaosFleshDeformableGPUManager::FBindingsBuffer::SetOwnerName(const FName& OwnerName)
{
	VerticesBuffer.SetOwnerName(OwnerName);
	RestVerticesBuffer.SetOwnerName(OwnerName);

	ParentsBuffer.SetOwnerName(OwnerName);
	VertsToParentsBuffer.SetOwnerName(OwnerName);
	WeightsBuffer.SetOwnerName(OwnerName);
	OffsetsBuffer.SetOwnerName(OwnerName);
	MaskBuffer.SetOwnerName(OwnerName);
}

bool FChaosFleshDeformableGPUManager::FBindingsBuffer::InitData(
	const TManagedArrayAccessor<FIntVector4>* ParentsArray,
	const TManagedArrayAccessor<FVector4f>* WeightsArray,
	const TManagedArrayAccessor<FVector3f>* OffsetArray,
	const TManagedArrayAccessor<float>* MaskArray,
	const TManagedArray<FVector3f>* RestVerticesArray,
	const TManagedArray<FVector3f>* VerticesArray)
{
	InitCondensationMapping(*ParentsArray);

	// Bindings
	Chaos::Softs::Private::InitBuffer(VertsToParentsBuffer, &VertsToSparseParents, Chaos::Softs::Private::NullIndexBuffer);
	Chaos::Softs::Private::InitBuffer(ParentsBuffer, &SparseParents, Chaos::Softs::Private::NullIndexBuffer);
	Chaos::Softs::Private::InitBuffer(WeightsBuffer, WeightsArray, Chaos::Softs::Private::NullHalfBuffer);
	Chaos::Softs::Private::InitBuffer(OffsetsBuffer, OffsetArray, Chaos::Softs::Private::NullHalfBuffer);
	Chaos::Softs::Private::InitBuffer(MaskBuffer, MaskArray, Chaos::Softs::Private::NullHalfBuffer);

	// Rest vertices
	const TArray<FVector3f>& DenseRestVertex = RestVerticesArray->GetConstArray();
	UpdateSparseVertices(DenseRestVertex, SparseRestVertices);
	Chaos::Softs::Private::InitBuffer(RestVerticesBuffer, &SparseRestVertices, Chaos::Softs::Private::NullFloatBuffer);

	// Dynamic vertices
	const TArray<FVector3f>& DenseVertex = VerticesArray->GetConstArray();
	if (DenseVertex.Num() == DenseRestVertex.Num())
	{
		UpdateSparseVertices(DenseVertex, SparseVertices);
	}
	else
	{
		UpdateSparseVertices(DenseRestVertex, SparseVertices);
	}
	Chaos::Softs::Private::InitBuffer(VerticesBuffer, &SparseVertices, Chaos::Softs::Private::NullFloatBuffer);

#if WITH_EDITOR
	if (Local::CVarParams.bTestSparseMappings)
	{
		TMap<int32, int32> DenseToSparseIndices;
		DenseToSparseIndices.Reserve(SparseToDenseIndices.Num());
		for (int32 i = 0; i < SparseToDenseIndices.Num(); i++)
		{
			DenseToSparseIndices.Add(TTuple<int32, int32>(SparseToDenseIndices[i], i));
		}
		check(DenseToSparseIndices.Num() == SparseToDenseIndices.Num());

		const TArray<FIntVector4>& DenseParents = ParentsArray->Get().GetConstArray();
		for (int32 i = 0; i < VertsToSparseParents.Num(); i++)
		{
			// Sparse parent index for ender index i
			const uint32 SparseParentIdx = VertsToSparseParents[i];
			check(SparseParents.IsValidIndex(SparseParentIdx));
			const FIntVector4& SparseParent = SparseParents[SparseParentIdx];

			// Remap sparse parents back to dense indices
			FIntVector4 Parents;
			for (int32 j = 0; j < 4; j++)
			{
				const int32 SparseIndex = SparseParent[j];
				if (SparseIndex == INDEX_NONE)
					Parents[j] = INDEX_NONE;
				else
				{
					check(SparseToDenseIndices.IsValidIndex(SparseIndex));
					Parents[j] = SparseToDenseIndices[SparseIndex];
				}
			}

			// Check that the unmapped sparse indices match the dense indices.
			const FIntVector4& DenseParent = DenseParents[i];
			check(Parents == DenseParent);

			// Check that the sparse vertices match the dense vertices.
			if (DenseVertex.Num() == DenseRestVertex.Num())
			{
				for (int32 j = 0; j < 4; j++)
					if (SparseParent[j] == INDEX_NONE || DenseParent[j] == INDEX_NONE)
					{
						check(SparseParent[j] == DenseParent[j]);
					}
					else
					{
						check(SparseVertices[SparseParent[j]] == DenseVertex[DenseParent[j]]);
					}
			}
			else
			{
				for (int32 j = 0; j < 4; j++)
					if (SparseParent[j] == INDEX_NONE || DenseParent[j] == INDEX_NONE)
					{
						check(SparseParent[j] == DenseParent[j]);
					}
					else
					{
						check(SparseVertices[SparseParent[j]] == DenseRestVertex[DenseParent[j]]);
					}
			}
		}
		UE_LOGF(LogFleshDeformerBufferManager, Log,
			"FChaosFleshDeformableGPUManager::FBindingsBuffer::InitData() - TEST_SPARSE_MAPPINGS succeeded.");
	}
	if(Local::CVarParams.bTestCompressedIndexing)
	{
		FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

		TArray<int32> DataRange32;
		TArray<int32> DataRange16;
		TArray<int32> DataRange8;
		DataRange32.SetNum(1024);
		DataRange16.SetNum(1024);
		DataRange8.SetNum(1024);
		for (int i = 0; i < 1024; i++)
		{
			DataRange32[i] = FMath::RandRange(INDEX_NONE, TNumericLimits<uint16>::Max()*2);
			DataRange16[i] = FMath::RandRange(INDEX_NONE, TNumericLimits<uint16>::Max()-1);
			DataRange8[i] = FMath::RandRange(INDEX_NONE, TNumericLimits<uint8>::Max()-1);
		}
		{
			UE::ChaosDeformable::FIndexArrayBufferWithSRV Buffer32; 
			Buffer32.SetBufferName("Test_int32"); 
			Buffer32.Init(DataRange32.GetData(), DataRange32.Num()); 
			Buffer32.InitRHI(RHICmdList);
		}
		{
			UE::ChaosDeformable::FIndexArrayBufferWithSRV Buffer16; 
			Buffer16.SetBufferName("Test_int16"); 
			Buffer16.Init(DataRange16.GetData(), DataRange16.Num()); 
			Buffer16.InitRHI(RHICmdList);
		}
		{
			UE::ChaosDeformable::FIndexArrayBufferWithSRV Buffer8; 
			Buffer8.SetBufferName("Test_int8"); 
			Buffer8.Init(DataRange8.GetData(), DataRange8.Num()); 
			Buffer8.InitRHI(RHICmdList);
		}
	}
#endif

	UE_LOGF(LogFleshDeformerBufferManager, Log,
		"FChaosFleshDeformableGPUManager::FBindingsBuffer::InitData() - Data size stats:\n"
			"    VertsToParentsBuffer: %d\n"
			"    ParentsBuffer: %d\n"
			"    WeightsBuffer: %d\n"
			"    OffsetsBuffer: %d\n"
			"    MaskBuffer:    %d\n"
			"    RestVertex:    %d\n"
			"    VertexBuffer:  %d\n"
			"    TOTAL:         %d",
		VertsToParentsBuffer.GetBufferSize(),
		ParentsBuffer.GetBufferSize(),
		WeightsBuffer.GetBufferSize(),
		OffsetsBuffer.GetBufferSize(),
		MaskBuffer.GetBufferSize(),
		RestVerticesBuffer.GetBufferSize(),
		VerticesBuffer.GetBufferSize(),

		VertsToParentsBuffer.GetBufferSize() +
		ParentsBuffer.GetBufferSize() +
		WeightsBuffer.GetBufferSize() +
		OffsetsBuffer.GetBufferSize() +
		MaskBuffer.GetBufferSize() +
		RestVerticesBuffer.GetBufferSize() +
		VerticesBuffer.GetBufferSize());

	return true;
}

bool FChaosFleshDeformableGPUManager::FBindingsBuffer::UpdateVertices(const TArray<FVector3f>& DenseVertex)
{
	if (UpdateSparseVertices(DenseVertex, SparseVertices))
	{
		Chaos::Softs::Private::UpdateBuffer(VerticesBuffer, &SparseVertices, Chaos::Softs::Private::NullFloatBuffer);
		return true;
	}
	return false;
}

bool FChaosFleshDeformableGPUManager::FBindingsBuffer::UpdateSparseVertices(
	const TArray<FVector3f>& DenseVertexIn, 
	TArray<FVector3f>& SparseVerticesIn) const
{
	SparseVerticesIn.SetNumUninitialized(SparseToDenseIndices.Num());
	bool bChanged = false;
	for (int32 i = 0; i < SparseToDenseIndices.Num(); i++)
	{
		if (!DenseVertexIn.IsValidIndex(SparseToDenseIndices[i]))
		{
			return bChanged;
		}
		bChanged |= SparseVerticesIn[i] != DenseVertexIn[SparseToDenseIndices[i]];
		SparseVerticesIn[i] = DenseVertexIn[SparseToDenseIndices[i]];
	}
	return bChanged;
}

void FChaosFleshDeformableGPUManager::FBindingsBuffer::InitCondensationMapping(
	const TManagedArrayAccessor<FIntVector4>& ParentsArray)
{
	// We want to skip vertices not involved in the deformation.  The Parents array contains all referenced
	// nodes.  Construct a condensation mapping from that.
	TSet<int32> Visited;
	const TArray<FIntVector4>& DenseParents = ParentsArray.Get().GetConstArray();
	for (int32 i = 0; i < DenseParents.Num(); i++)
	{
		const FIntVector4& Tet = DenseParents[i];
		for (int32 j = 0; j < 4; j++)
		{
			if (Tet[j] >= -1)
			{
				Visited.Add(Tet[j]);
			}
		}
	}
	// Flatten Visited set into a sorted array.
	TArray<int32> VisitedArray;
	VisitedArray.Reserve(Visited.Num());
	for (TSet<int32>::TConstIterator It = Visited.CreateConstIterator(); It; ++It)
	{
		const int32 Idx = *It;
		ensure(Idx >= -1);
		VisitedArray.Add(Idx);
	}
	VisitedArray.Sort();

	// CondensationMapping remaps sparse vertex indices to dense indices.  Every vertex that was visited
	// gets added to the list, the index of which is the new "SparseIndex".  DenseToSparseIndices is the
	// inverse of CondensationMapping.
	SparseToDenseIndices.Empty();
	SparseToDenseIndices.Reserve(VisitedArray.Num());
	TMap<int32, int32> DenseToSparseIndices;
	DenseToSparseIndices.Reserve(VisitedArray.Num());
	DenseToSparseIndices.Add(INDEX_NONE, INDEX_NONE);
	for (int32 i = 0; i < VisitedArray.Num(); i++)
	{
		if (VisitedArray[i] >= 0) // skip -1, which exists in surface bindings
		{
			const int32 SparseIndex = SparseToDenseIndices.Add(VisitedArray[i]);
			DenseToSparseIndices.Add(VisitedArray[i], SparseIndex);
		}
	}

	// The Parents array likely has a lot of duplicate entries, so we consolidate Parents into SparseParents,
	// and add a level of indirection with VertsToSparseParents.  That way we store 1 index per render vertex
	// rather than 4.
	VertsToSparseParents.SetNum(DenseParents.Num());
	SparseParents.Reserve(DenseParents.Num() / 4);
	for (int32 i = 0; i < DenseParents.Num(); i++)
	{
		FIntVector4 Tet = DenseParents[i];
		// Remap non-sparse parent vertex indices to sparse vertex indices.
		for (int32 j = 0; j < 4; j++)
		{
			Tet[j] = DenseToSparseIndices[Tet[j]];
			ensure(Tet[j] >= -1);
		}
		int32 Idx = SparseParents.AddUnique(Tet);
		VertsToSparseParents[i] = Idx;
	}

	{
		float SparseToDenseRatio = static_cast<float>(SparseParents.Num() * 4 + VertsToSparseParents.Num()) / (DenseParents.Num() * 4);
		float SparseToDensePct = (1.0 - SparseToDenseRatio) * 100.0;

		UE_LOGF(LogFleshDeformerBufferManager, Log,
			"BindingsBuffer::InitCondensationMapping() - DenseParents: %d (%d); SparseParents: %d (%d) + VertsToSparseParents: %d = %d; %g pct reduction.",
			DenseParents.Num(), (DenseParents.Num() * 4),
			SparseParents.Num(), (SparseParents.Num() * 4), VertsToSparseParents.Num(),
			(SparseParents.Num() * 4 + VertsToSparseParents.Num()),
			SparseToDensePct);
	}

}

//=============================================================================
// FChaosFleshDeformableGPUManager
//=============================================================================

FChaosFleshDeformableGPUManager::FChaosFleshDeformableGPUManager()
	: Component(nullptr)
{}

FChaosFleshDeformableGPUManager::FChaosFleshDeformableGPUManager(UDeformableTetrahedralComponent* InComponent)
	: Component(InComponent)
{}

FChaosFleshDeformableGPUManager::~FChaosFleshDeformableGPUManager()
{
	GPUBufferConsumers.Empty();
	BufferTable.Empty();
}

void FChaosFleshDeformableGPUManager::SetOwner(UDeformableTetrahedralComponent* InComponent)
{
	Component = InComponent;
}

void FChaosFleshDeformableGPUManager::UnRegisterGPUBufferConsumer(const void* ID) const 
{ 
	if (GPUBufferConsumers.Num())
	{
		GPUBufferConsumers.Remove(ID); // mutable

		FChaosFleshDeformableGPUManager* NCThis = const_cast<FChaosFleshDeformableGPUManager*>(this);
		for (IdToBuffers::TIterator It = NCThis->BufferTable.CreateIterator(); It;)
		{
			if (It->Key.Get<0>() == ID)
			{
				IdToBuffers::TIterator TargetIt = It;
				++It;
				NCThis->BufferTable.Remove(TargetIt->Key);
			}
			else
			{
				++It;
			}
		}
	}
}

bool FChaosFleshDeformableGPUManager::HasGPUBindingsBuffer(
	const void* ID,
	const FName MeshName,
	const int32 LodIndex) const 
{
	TTuple<void*, FName, int32> Key(const_cast<void*>(ID), MeshName, LodIndex);
	return BufferTable.Contains(Key);
}

bool FChaosFleshDeformableGPUManager::InitGPUBindingsBuffer(
	const void* ID, 
	const FName MeshName, 
	const int32 LodIndex,
	const bool ForceInit)
{
	if (!ForceInit && HasGPUBindingsBuffer(ID, MeshName, LodIndex))
	{
		return true;
	}

	if (!Component)
	{
		return false;
	}

	const FFleshCollection* RestCollection = GetRestCollection();
	if (!RestCollection)
	{
		return false;
	}

	GeometryCollection::Facades::FTetrahedralBindings TetBindings(*RestCollection);
	const int32 TetIndex = TetBindings.GetTetMeshIndex(MeshName, LodIndex);
	if (TetIndex != INDEX_NONE)
	{
		if (!TetBindings.ReadBindingsGroup(TetIndex, MeshName, LodIndex))
		{
			UE_LOGF(LogFleshDeformerBufferManager, Error,
				"LogFleshDeformerBufferManager: Failed to read bindings group associated with mesh '%ls' LOD: %d",
				*MeshName.ToString(), LodIndex);
			return false;
		}
		if (!TetBindings.IsValid())
		{
			UE_LOGF(LogFleshDeformerBufferManager, Error,
				"LogFleshDeformerBufferManager: Bindings group associated with mesh '%ls' LOD: %d are invalid.",
				*MeshName.ToString(), LodIndex);
			return false;
		}
	}
	else 
	{
		UE_LOGF(LogFleshDeformerBufferManager, Error,
			"LogFleshDeformerBufferManager: Failed to find tet mesh index associated to mesh '%ls' LOD: %d",
			*MeshName.ToString(), LodIndex);
		return false;
	}

	const TManagedArrayAccessor<FIntVector4>* ParentsArray = TetBindings.GetParentsRO();
	const TManagedArrayAccessor<FVector4f>* WeightsArray = TetBindings.GetWeightsRO();
	const TManagedArrayAccessor<FVector3f>* OffsetArray = TetBindings.GetOffsetsRO();
	const TManagedArrayAccessor<float>* MaskArray = TetBindings.GetMaskRO();
	if (!ParentsArray || !WeightsArray || !OffsetArray || !MaskArray)
	{
		return false;
	}

	const TManagedArray<FVector3f>* RestVertexArray = GetRestVertexArray();
	const TManagedArray<FVector3f>* VertexArray = GetVertexArray();
	if (!RestVertexArray || !VertexArray)
	{
		return false;
	}
	
	TSharedPtr<FBindingsBuffer> Bindings(new FBindingsBuffer());
	Bindings->SetOwnerName(FName(Component->GetName()));
	if (Bindings->InitData(
		ParentsArray, WeightsArray, OffsetArray, MaskArray,
		RestVertexArray, VertexArray))
	{
		TTuple<void*, FName, int32> Key(const_cast<void*>(ID), MeshName, LodIndex);
		BufferTable.Add(MoveTemp(Key), MoveTemp(Bindings));
		RegisterGPUBufferConsumer(ID);
		return true;
	}
	return false;
}

bool FChaosFleshDeformableGPUManager::UpdateGPUBuffers()
{
	if (!Component || !HasRegisteredGPUBufferConsumer())
	{
		return false;
	}

	const TManagedArray<FVector3f>* DenseVertexArray = GetVertexArray();
	if (!DenseVertexArray)
	{
		return false;
	}
	const TArray<FVector3f>& DenseVertex = DenseVertexArray->GetConstArray();

	for(IdToBuffers::TIterator It = BufferTable.CreateIterator(); It; ++It)
	{
		if (TSharedPtr<FBindingsBuffer>& Buffer = It.Value())
		{
			Buffer->UpdateVertices(DenseVertex);
		}
	}
	return !BufferTable.IsEmpty();
}

const FChaosFleshDeformableGPUManager::FBindingsBuffer* FChaosFleshDeformableGPUManager::GetBindingsBuffer(
	const void* ID, 
	const FName MeshName, 
	const int32 LodIndex) const
{
	TTuple<void*, FName, int32> Key(const_cast<void*>(ID), MeshName, LodIndex);
	const TSharedPtr<FBindingsBuffer>* Bindings = BufferTable.Find(Key);
	return Bindings ? Bindings->Get() : nullptr;
}

FChaosFleshDeformableGPUManager::FBindingsBuffer* FChaosFleshDeformableGPUManager::GetBindingsBuffer(
	const void* ID,
	const FName MeshName,
	const int32 LodIndex)
{
	TTuple<void*, FName, int32> Key(const_cast<void*>(ID), MeshName, LodIndex);
	TSharedPtr<FBindingsBuffer>* Bindings = BufferTable.Find(Key);
	return Bindings ? Bindings->Get() : nullptr;
}

const FFleshCollection* FChaosFleshDeformableGPUManager::GetRestCollection() const
{
	if (!Component)
	{
		return nullptr;
	}

	const UFleshAsset* RestCollectionAsset = Component->GetRestCollection();
	if (!RestCollectionAsset)
	{
		return nullptr;
	}

	return RestCollectionAsset->GetFleshCollection().Get(); // can be null
}

const TManagedArray<FVector3f>* FChaosFleshDeformableGPUManager::GetRestVertexArray() const
{
	if (!Component)
	{
		return nullptr;
	}

	if (const UFleshAsset* RestCollection = Component->GetRestCollection())
	{
		return RestCollection->FindPositions();
	}
	return nullptr;
}

const TManagedArray<FVector3f>* FChaosFleshDeformableGPUManager::GetVertexArray() const
{
	if (!Component)
	{
		return nullptr;
	}
	if (UFleshDynamicAsset* DynamicCollection = Component->GetDynamicCollection())
	{
		if (const UFleshAsset* RestCollection = Component->GetRestCollection())
		{
			const int32 NumRestVertices = DynamicCollection->GetCollection()->NumElements(FGeometryCollection::VerticesGroup);
			const int32 NumDynVertices = DynamicCollection->GetCollection()->NumElements(FGeometryCollection::VerticesGroup);
			if (NumRestVertices == NumDynVertices)
			{
				const TManagedArray<FVector3f>* RetVal = DynamicCollection->FindPositions();
				return RetVal;
			}
		}
	}

	return GetRestVertexArray();
}

