// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFlesh/ChaosFleshEnginePlugin.h"

#include "ChaosCache/FleshComponentCacheAdapter.h"
#include "DataInterfaces/DIFleshDeformer.h"
#include "Interfaces/IPluginManager.h"
#include "IOptimusCoreModule.h"
#include "ShaderCore.h"

#if USE_USD_SDK && DO_USD_CACHING
#include "USDMemory.h"
#endif

class FChaosFleshEnginePlugin : public IChaosFleshEnginePlugin
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	TUniquePtr<Chaos::FFleshCacheAdapter> FleshCacheAdapter;
};


void FChaosFleshEnginePlugin::StartupModule()
{
	// Make sure our shaders can be found via the virtual shader paths.
	const FString PluginShaderDir = 
		FPaths::Combine(
			IPluginManager::Get().FindPlugin(TEXT("ChaosFlesh"))->GetBaseDir(), 
			TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/ChaosFlesh"), PluginShaderDir);

	// Self-register for now.
	// FIXME: Should add a method to IOptimusModule to register interfaces.
	if (IOptimusCoreModule* OptimusCoreModule = FModuleManager::GetModulePtr<IOptimusCoreModule>("OptimusCore"))
	{
		OptimusCoreModule->RegisterDataInterfaceClass<UDIFleshDeformer>();
	}

	FleshCacheAdapter = MakeUnique<Chaos::FFleshCacheAdapter>();
	Chaos::RegisterAdapter(FleshCacheAdapter.Get());
}


void FChaosFleshEnginePlugin::ShutdownModule()
{	
}


#if USE_USD_SDK && DO_USD_CACHING
IMPLEMENT_MODULE_USD(FChaosFleshEnginePlugin, ChaosFleshEngine)
#else
IMPLEMENT_MODULE(FChaosFleshEnginePlugin, ChaosFleshEngine)
#endif
