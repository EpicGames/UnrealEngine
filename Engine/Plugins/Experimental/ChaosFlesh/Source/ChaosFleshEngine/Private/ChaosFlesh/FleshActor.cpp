// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFlesh/FleshActor.h"

#include "ChaosFlesh/ChaosDeformableSolverComponent.h"
#include "ChaosFlesh/ChaosDeformableSolverActor.h"
#include "ChaosFlesh/ChaosDeformableTetrahedralComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FleshActor)

DEFINE_LOG_CATEGORY_STATIC(AFleshLogging, Log, All);

AFleshActor::AFleshActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	//UE_LOGF(AFleshLogging, Verbose, "AFleshActor::AFleshActor()");
	FleshComponent = CreateDefaultSubobject<UFleshComponent>(TEXT("FleshComponent0"));
	RootComponent = FleshComponent;
	PrimaryActorTick.bCanEverTick = true;
}

void AFleshActor::EnableSimulation(ADeformableSolverActor* InActor)
{
	if (InActor)
	{
		if (FleshComponent && FleshComponent->GetRestCollection())
		{
			FleshComponent->EnableSimulation(InActor->GetDeformableSolverComponent());
		}
	}
}


#if WITH_EDITOR
void AFleshActor::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);
	if (PropertyThatWillChange && PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(AFleshActor, PrimarySolver))
	{
		PreEditChangePrimarySolver = PrimarySolver;
	}
}


void AFleshActor::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	//
	// The UDeformablePhysicsComponent and the UDeformableSolverComponent hold references to each other. 
	// If one of the attributes change, then the attribute on the other component needs to be updated. 
	//
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(AFleshActor, PrimarySolver))
	{
		if (PrimarySolver)
		{
			if (UDeformableSolverComponent* SolverComponent = PrimarySolver->GetDeformableSolverComponent())
			{
				if (FleshComponent)
				{
					FleshComponent->PrimarySolverComponent = SolverComponent;
					if (!SolverComponent->ConnectedObjects.DeformableComponents.Contains(FleshComponent))
					{
						SolverComponent->ConnectedObjects.DeformableComponents.Add(TObjectPtr<UDeformablePhysicsComponent>(FleshComponent));
					}
				}
			}
		}
		else if (PreEditChangePrimarySolver)
		{
			if (UDeformableSolverComponent* SolverComponent = PreEditChangePrimarySolver->GetDeformableSolverComponent())
			{
				if (FleshComponent)
				{
					FleshComponent->PrimarySolverComponent = nullptr;
					if (SolverComponent->ConnectedObjects.DeformableComponents.Contains(FleshComponent))
					{
						SolverComponent->ConnectedObjects.DeformableComponents.Remove(FleshComponent);
					}
				}
			}
		}
	}
}


bool AFleshActor::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	Super::GetReferencedContentObjects(Objects);

	if (FleshComponent && FleshComponent->GetRestCollection())
	{
		Objects.Add(const_cast<UFleshAsset*>(FleshComponent->GetRestCollection()));
	}
	return true;
}

void AFleshActor::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedPtr<IPropertyHandle> bReplicatePhysicsProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(AFleshActor, bAsyncPhysicsTickEnabled), AActor::StaticClass());
	bReplicatePhysicsProperty->MarkHiddenByCustomization();
}

#endif


