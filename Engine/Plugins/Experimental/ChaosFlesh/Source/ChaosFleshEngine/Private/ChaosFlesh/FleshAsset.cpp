// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FleshAsset.cpp: UFleshAsset methods.
=============================================================================*/
#include "ChaosFlesh/FleshAsset.h"
#include "ChaosFlesh/FleshCollection.h"
#include "Components/SkeletalMeshComponent.h"
#include "Dataflow/DataflowContent.h"
#include "Engine/SkeletalMesh.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"
#include "GeometryCollection/TransformCollection.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkeletalMeshModel.h"
#include "ChaosFlesh/ChaosFleshCollectionFacade.h"
#include "ChaosFlesh/FleshCollectionUtility.h"
#include "Materials/Material.h"
#include "MaterialDomain.h"

#if WITH_EDITOR
#include "BoneWeights.h"
#include "ChaosFlesh/FleshComponent.h"
#include "MeshUtilities.h"
#include "UObject/UObjectIterator.h"
#include "Utils/ClothingMeshUtils.h"
#endif
#include UE_INLINE_GENERATED_CPP_BY_NAME(FleshAsset)

DEFINE_LOG_CATEGORY_STATIC(LogFleshAssetInternal, Log, All);

namespace UE::FleshAsset::Private
{
	static const FName DataflowTerminalNodeName("FleshAssetTerminal");
	static FReferenceSkeleton MakeDefaultRefSkeleton()
	{
		FReferenceSkeleton DefaultRefSkeleton;
		FReferenceSkeletonModifier SkeletonModifier(DefaultRefSkeleton, nullptr);
		SkeletonModifier.Add(FMeshBoneInfo(FName("Root"), FString("Root"), INDEX_NONE), FTransform::Identity);
		return DefaultRefSkeleton;
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////

FFleshAssetEdit::FFleshAssetEdit(UFleshAsset* InAsset, FPostEditFunctionCallback InCallback)
	: PostEditCallback(InCallback)
	, Asset(InAsset)
{
}

FFleshAssetEdit::~FFleshAssetEdit()
{
	PostEditCallback();
}

FFleshCollection* FFleshAssetEdit::GetFleshCollection()
{
	if (Asset)
	{
		return Asset->FleshCollection.Get();
	}
	return nullptr;
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////

UFleshAsset::UFleshAsset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, FleshCollection(new FFleshCollection())
	, DataflowInstance(this)
{
	// set the default name for the terminal node 
	DataflowInstance.SetDataflowTerminal(UE::FleshAsset::Private::DataflowTerminalNodeName);

}

void UFleshAsset::SetCollection(FFleshCollection* InCollection)
{
	FleshCollection = TSharedPtr<FFleshCollection, ESPMode::ThreadSafe>(InCollection);
	Modify();
}

void UFleshAsset::SetFleshCollection(TUniquePtr<FFleshCollection>&& InCollection)
{
	FleshCollection = TSharedPtr<FFleshCollection, ESPMode::ThreadSafe>(InCollection.Release());
	Modify();
}

#if WITH_EDITOR
static void BuildLod(FSkeletalMeshLODModel& LODModel, const UFleshAsset& FleshAsset, int32 LodIndex)
{
	using namespace ::Chaos::Softs;

	// Start from an empty LODModel
	LODModel.Empty();

	// Clear the mesh infos, none are stored on this asset
	LODModel.MaxImportVertex = 0;

	// Set 1 texture coordinate
	LODModel.NumTexCoords = 1;

	// Init the size of the vertex buffer
	LODModel.NumVertices = 0;

	// Offset to remap the LOD materials to the asset materials
	const int32 MaterialOffset = 0;

	const TSharedPtr<const FFleshCollection> FleshCollection = FleshAsset.GetFleshCollection();
	// Load the mesh utilities module used to optimized the index buffer
	IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");

	// Build the sim mesh descriptor for creation of the sections' mesh to mesh mapping data
	Chaos::FFleshCollectionFacade FleshFacade(*FleshCollection);
	const int32 NumLodSimVertices = FleshFacade.NumVertices();

	TArray<FVector3f> SimPositions;
	for (int32 VertexIndex = 0; VertexIndex < NumLodSimVertices; ++VertexIndex)
	{
		SimPositions.Add(FleshFacade.Vertex[VertexIndex]);
	}
	TArray<uint32> SimIndices;
	TArray<FIntVector3> SimElements; // for correct data formatting
	for (int32 TriIndex = 0; TriIndex < FleshFacade.NumFaces(); ++TriIndex)
	{
		SimIndices.Add(uint32(FleshFacade.Indices[TriIndex][0]));
		SimIndices.Add(uint32(FleshFacade.Indices[TriIndex][1]));
		SimIndices.Add(uint32(FleshFacade.Indices[TriIndex][2]));
		SimElements.Add(FleshFacade.Indices[TriIndex]);
	}
	const ClothingMeshUtils::ClothMeshDesc SourceMesh(
		SimPositions,
		SimIndices);  // Let it calculate the averaged normals as to match the simulation data output
	const bool bIsValidSourceMesh = (SourceMesh.GetPositions().Num() > 0);

	TArray<FVector3f> RenderPositions;
	TArray<FIntVector3> RenderElements;
	TArray<int32> NewToOldVertexMap = ChaosFlesh::CompactSurfaceVertices(SimPositions, SimElements, RenderPositions, RenderElements);
	const int32 NumRenderVertices = RenderPositions.Num();
	const int32 NumRenderFaces = RenderElements.Num();
	const int32 NumRenderIndices = NumRenderFaces * 3;

	LODModel.MeshToImportVertexMap.Reserve(NumRenderVertices);

	// Populate this LOD's sections and the LOD index buffer
	const int32 NumSections = 1;  // Cloth Render Patterns == Skeletal Mesh Sections
	LODModel.Sections.SetNum(NumSections);

	TArray<UE::Tasks::FTask> PendingTasks;
	PendingTasks.Reserve(NumSections);
	int32 BaseIndex = 0;

	TArray<TSet<FBoneIndexType>> ActiveBoneIndicesArray;
	// Require at least one active bone indices set to merge all batches set and include the exta bones indices at the end.
	ActiveBoneIndicesArray.SetNum(FMath::Max(NumSections, 1));

	// Struct to store FSkeletalMeshLODModel data by thread
	struct FSkeletalMeshLODModelData
	{
		TArray<uint32> IndexBuffer;
		TArray<int32> MeshToImportVertexMap;
		int32 MaxImportVertex;
		uint32 NumTexCoords;
	};

	TArray <FSkeletalMeshLODModelData> LODModelDatas;
	LODModelDatas.SetNum(NumSections);

	// Mutex to protect the source mesh which has mutable attribute which are modified
	UE::FMutex SourceMeshMutex;
	for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
	{
		// Keep track of the active bone indices for this LOD model
		TSet<FBoneIndexType>& ActiveBoneIndices = ActiveBoneIndicesArray[SectionIndex];
		ActiveBoneIndices.Reserve(FleshAsset.GetRefSkeleton().GetNum());

		FSkelMeshSection& Section = LODModel.Sections[SectionIndex];
		Section.OriginalDataSectionIndex = SectionIndex;

		Section.BaseIndex = (uint32)BaseIndex;
		BaseIndex += NumRenderIndices;
		Section.BaseVertexIndex = LODModel.NumVertices;
		LODModel.NumVertices += (uint32)NumRenderVertices;

		FSkeletalMeshLODModelData& LODModelData = LODModelDatas[SectionIndex];

		UE::Tasks::FTask PendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION,
			// Instance data per thread, data that are not shared through different threads
			[SectionIndex, &Section, NumRenderFaces, NumRenderIndices, &LODModelData, &ActiveBoneIndices, &RenderPositions, &NumRenderVertices, 
			&RenderElements, &NewToOldVertexMap,
			// Const data that should not be modified or protected data by mutex.
			MaterialOffset, &FleshAsset, &MeshUtilities, bIsValidSourceMesh, &SourceMesh, &SourceMeshMutex, LodIndex]()
			{
				const int32 MaterialIndex = MaterialOffset + SectionIndex;

				// Build the section face data (indices)
				Section.MaterialIndex = (uint16)MaterialIndex;
				Section.NumTriangles = (uint32)NumRenderFaces;

				TArray<uint32> Indices;
				Indices.SetNumUninitialized(NumRenderIndices);

				for (int32 FaceIndex = 0; FaceIndex < NumRenderFaces; ++FaceIndex)
				{
					const FIntVector3& RenderIndices = RenderElements[FaceIndex];
					for (int32 VertexIndex = 0; VertexIndex < 3; ++VertexIndex)
					{
						const int32 RenderIndex = RenderIndices[VertexIndex];
						Indices[FaceIndex * 3 + VertexIndex] = (uint32)RenderIndex;
					}
				}

				MeshUtilities.CacheOptimizeIndexBuffer(Indices);

				LODModelData.IndexBuffer.Append(MoveTemp(Indices));

				// Build the section vertex data 
				Section.SoftVertices.SetNumUninitialized(NumRenderVertices);
				Section.NumVertices = NumRenderVertices;

				// Map reference skeleton bone index to the index in the section's bone map
				TMap<FBoneIndexType, FBoneIndexType> ReferenceToSectionBoneMap;

				// Track how many bones we added to the section's bone map so far
				int CurSectionBoneMapNum = 0;

				GeometryCollection::Facades::FVertexBoneWeightsFacade VertexBoneWeightsFacade(*FleshAsset.GetFleshCollection());
				const TManagedArray<TArray<int32>>* VertexBoneIndices = VertexBoneWeightsFacade.FindBoneIndices();
				const TManagedArray<TArray<float>>* VertexBoneWeights = VertexBoneWeightsFacade.FindBoneWeights();
				GeometryCollection::Facades::FCollectionMeshFacade CollectionMeshFacade(*FleshAsset.GetFleshCollection());
				
				LODModelData.NumTexCoords = 1;
				for (int32 VertexIndex = 0; VertexIndex < NumRenderVertices; ++VertexIndex)
				{
					const int32 CollectionVertexIndex = NewToOldVertexMap[VertexIndex];
					// Save the original indices for the newly added vertices
					LODModelData.MeshToImportVertexMap.Add(VertexIndex);
					LODModelData.MaxImportVertex = VertexIndex;

					FSoftSkinVertex& SoftVertex = Section.SoftVertices[VertexIndex];

					SoftVertex.Position = RenderPositions[VertexIndex];

					SoftVertex.TangentX = CollectionMeshFacade.TangentUAttribute.IsValid() && 
						CollectionMeshFacade.TangentUAttribute.IsValidIndex(CollectionVertexIndex) ?
							CollectionMeshFacade.TangentUAttribute[CollectionVertexIndex] :
							FVector3f(1, 0, 0);
					SoftVertex.TangentY = CollectionMeshFacade.TangentVAttribute.IsValid() &&
						CollectionMeshFacade.TangentVAttribute.IsValidIndex(CollectionVertexIndex) ?
							CollectionMeshFacade.TangentVAttribute[CollectionVertexIndex] :
							FVector3f(0, 1, 0);
					SoftVertex.TangentZ = CollectionMeshFacade.NormalAttribute.IsValid() &&
						CollectionMeshFacade.NormalAttribute.IsValidIndex(CollectionVertexIndex) ?
							CollectionMeshFacade.NormalAttribute[CollectionVertexIndex] :
							FVector3f(0, 0, 1); 

					constexpr bool bSRGB = false; // Avoid linear to srgb conversion
					SoftVertex.Color = CollectionMeshFacade.ColorAttribute.IsValid() &&
						CollectionMeshFacade.ColorAttribute.IsValidIndex(CollectionVertexIndex) ?
							CollectionMeshFacade.ColorAttribute[CollectionVertexIndex].ToFColor(bSRGB) :
							FColor::Red;

					const TArray<FVector2f> RenderUVs = TArray<FVector2f>({ FVector2f() });
					FMemory::Memzero(SoftVertex.UVs, sizeof(SoftVertex.UVs));
					const int32 NumTexCoords = FMath::Min(RenderUVs.Num(), (int32)MAX_TEXCOORDS);
					for (int32 TexCoord = 0; TexCoord < NumTexCoords; ++TexCoord)
					{
						SoftVertex.UVs[TexCoord] = RenderUVs[TexCoord];
					}
					LODModelData.NumTexCoords = FMath::Max(LODModelData.NumTexCoords, (uint32)NumTexCoords);

					bool HasWeights = VertexBoneIndices && VertexBoneWeights && (*VertexBoneIndices)[CollectionVertexIndex].Num() && (*VertexBoneWeights)[CollectionVertexIndex].Num();
					const TArray<int32> BoneIndices = HasWeights ? (*VertexBoneIndices)[CollectionVertexIndex] : TArray<int32>({ 0 });
					const TArray<float> BoneWeights = HasWeights ? (*VertexBoneWeights)[CollectionVertexIndex] : TArray<float>({ 1.f });;
					const int32 NumInfluences = BoneIndices.Num();
					// Add all of the bones that have non-zero influence to the section's bone map and keep track of the order
					// that we added the reference bone via CurSectionBoneMapNum
					for (int32 Influence = 0; Influence < NumInfluences; ++Influence)
					{
						const FBoneIndexType InfluenceBone = (FBoneIndexType)BoneIndices[Influence];

						if (ReferenceToSectionBoneMap.Contains(InfluenceBone) == false)
						{
							ReferenceToSectionBoneMap.Add(InfluenceBone, CurSectionBoneMapNum);
							++CurSectionBoneMapNum;
						}
					}

					int32 Influence = 0;
					for (; Influence < NumInfluences; ++Influence)
					{
						const FBoneIndexType InfluenceBone = (FBoneIndexType)BoneIndices[Influence];
						const float InWeight = BoneWeights[Influence];
						const uint16 InfluenceWeight = static_cast<uint16>(InWeight * static_cast<float>(UE::AnimationCore::MaxRawBoneWeight) + 0.5f);

						// FSoftSkinVertex::InfluenceBones contain indices into the section's bone map and not the reference
						// skeleton, so we need to remap
						const FBoneIndexType* const MappedIndexPtr = ReferenceToSectionBoneMap.Find(InfluenceBone);

						// ReferenceToSectionBoneMap should always contain InfluenceBone since it was added above
						checkSlow(MappedIndexPtr);
						if (MappedIndexPtr != nullptr)
						{
							SoftVertex.InfluenceBones[Influence] = *MappedIndexPtr;
							SoftVertex.InfluenceWeights[Influence] = InfluenceWeight;
						}
					}

					for (; Influence < MAX_TOTAL_INFLUENCES; ++Influence)
					{
						SoftVertex.InfluenceBones[Influence] = 0;
						SoftVertex.InfluenceWeights[Influence] = 0;
					}
				}

				// Initialize the section bone map
				Section.BoneMap.SetNumUninitialized(ReferenceToSectionBoneMap.Num());
				for (const TPair<FBoneIndexType, FBoneIndexType>& Pair : ReferenceToSectionBoneMap)
				{
					Section.BoneMap[Pair.Value] = Pair.Key;
				}

				ActiveBoneIndices.Append(Section.BoneMap);

				// Update max bone influences
				Section.CalcMaxBoneInfluences();
				Section.CalcUse16BitBoneIndex();

				// Setup clothing data
				if (bIsValidSourceMesh)
				{
					Section.ClothMappingDataLODs.SetNum(1);  // TODO: LODBias maps for raytracing

					Section.ClothingData.AssetLodIndex = LodIndex;
					Section.ClothingData.AssetGuid = FleshAsset.GetAssetGuid();  // There is only one cloth asset,
					Section.CorrespondClothAssetIndex = 0;       // this one

					// Create mapping from scratch
					TArray<FVector3f> SectionRenderPositions;
					TArray<FVector3f> SectionRenderNormals;
					TArray<FVector3f> SectionRenderTangents;
					SectionRenderPositions.Reserve(NumRenderVertices);
					SectionRenderNormals.Reserve(NumRenderVertices);
					SectionRenderTangents.Reserve(NumRenderVertices);
					for (const FSoftSkinVertex& SoftVert : Section.SoftVertices)
					{
						SectionRenderPositions.Add(SoftVert.Position);
						SectionRenderNormals.Add(SoftVert.TangentZ);
						SectionRenderTangents.Add(SoftVert.TangentX);
					}
					TArray<uint32> SectionRenderIndices;
					SectionRenderIndices.Reserve(NumRenderIndices);
					const TArrayView<uint32> SectionIndexBuffer(LODModelData.IndexBuffer.GetData(), NumRenderIndices);
					for (const uint32 LodModelVertIndex : SectionIndexBuffer)
					{
						SectionRenderIndices.Add(LodModelVertIndex - Section.BaseVertexIndex);
					}

					const ClothingMeshUtils::ClothMeshDesc TargetMesh(
						SectionRenderPositions,
						SectionRenderNormals,
						SectionRenderTangents,
						SectionRenderIndices);

					// SourceMesh has mutable AAbbtree which is modified so this function cannot be multi threaded
					SourceMeshMutex.Lock();
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					ClothingMeshUtils::GenerateMeshToMeshVertData(
						Section.ClothMappingDataLODs[0],
						TargetMesh,
						SourceMesh,
						/*MaxDistances*/ nullptr,
						/*bSmoothTransition_DEPRECATED*/ true,
						/*bUseMultipleInfluences_DEPRECATED*/ false,
						/*SkinningKernelRadius_DEPRECATED*/ 30.f);
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
					SourceMeshMutex.Unlock();
				}

				// Compute the overlapping vertices map (inspired from MeshUtilities::BuildSkeletalMesh)
				const TArray<FSoftSkinVertex>& SoftVertices = Section.SoftVertices;

				typedef TPair<float, int32> FIndexAndZ;  // Acceleration structure, list of vertex Z / index pairs
				TArray<FIndexAndZ> IndexAndZs;
				IndexAndZs.Reserve(NumRenderVertices);
				for (int32 VertexIndex = 0; VertexIndex < NumRenderVertices; ++VertexIndex)
				{
					const FVector3f& Position = SoftVertices[VertexIndex].Position;

					const float Z = 0.30f * Position.X + 0.33f * Position.Y + 0.37f * Position.Z;
					IndexAndZs.Emplace(Z, VertexIndex);
				}
				IndexAndZs.Sort([](const FIndexAndZ& A, const FIndexAndZ& B) { return A.Key < B.Key; });

				for (int32 Index0 = 0; Index0 < IndexAndZs.Num(); ++Index0)
				{
					const float Z0 = IndexAndZs[Index0].Key;
					const uint32 VertexIndex0 = IndexAndZs[Index0].Value;
					const FVector3f& Position0 = SoftVertices[VertexIndex0].Position;

					// Only need to search forward, since we add pairs both ways
					for (int32 Index1 = Index0 + 1; Index1 < IndexAndZs.Num() && FMath::Abs(IndexAndZs[Index1].Key - Z0) <= THRESH_POINTS_ARE_SAME; ++Index1)
					{
						const uint32 VertexIndex1 = IndexAndZs[Index1].Value;
						const FVector3f& Position1 = SoftVertices[VertexIndex1].Position;

						if (PointsEqual(Position0, Position1))
						{
							// Add to the overlapping map
							TArray<int32>& SrcValueArray = Section.OverlappingVertices.FindOrAdd(VertexIndex0);
							SrcValueArray.Add(VertexIndex1);

							TArray<int32>& IterValueArray = Section.OverlappingVertices.FindOrAdd(VertexIndex1);
							IterValueArray.Add(VertexIndex0);
						}
					}
				}
			});

		PendingTasks.Add(PendingTask);
	}
	UE::Tasks::Wait(PendingTasks);

	for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
	{
		FSkeletalMeshLODModelData& LODModelData = LODModelDatas[SectionIndex];
		LODModel.IndexBuffer.Append(MoveTemp(LODModelData.IndexBuffer));
		LODModel.NumTexCoords = FMath::Max(LODModel.NumTexCoords, LODModelData.NumTexCoords);
		LODModel.MeshToImportVertexMap.Append(MoveTemp(LODModelData.MeshToImportVertexMap));
		LODModel.MaxImportVertex = LODModelData.MaxImportVertex;

		FSkelMeshSection& Section = LODModel.Sections[SectionIndex];
		// Must be in single thread
		// Copy to user section data, otherwise the section data set above would get lost when the user section gets synced
		FSkelMeshSourceSectionUserData::GetSourceSectionUserData(LODModel.UserSectionsData, Section);

	}

	// TODO: enable when NumSections > 1
	//for (int32 SectionIndex = 1; SectionIndex < NumSections; ++SectionIndex)
	//{
	//	ActiveBoneIndicesArray[0].Append(MoveTemp(ActiveBoneIndicesArray[SectionIndex]));
	//}

	// Update the active bone indices on the LOD model
	LODModel.ActiveBoneIndices = ActiveBoneIndicesArray[0].Array();

	// Ensure parent exists with incoming active bone indices, and the result should be sorted
	FleshAsset.GetRefSkeleton().EnsureParentsExistAndSort(LODModel.ActiveBoneIndices);
	LODModel.ActiveBoneIndices.Shrink();

	// Add the extra simulation bones to the list of required bones if any
	LODModel.RequiredBones = LODModel.ActiveBoneIndices;
}

void UFleshAsset::BuildResourceForRendering(FSkeletalMeshRenderData& OutSkeletalMeshRenderData) const
{
	// Allocate empty entries for each LOD level in source mesh
	const int32 NumLods = 1;

	FSkeletalMeshModel SkelMeshModel;
	SkelMeshModel.LODModels.Reset(NumLods);
	// Rebuild each LOD models
	for (int32 LodIndex = 0; LodIndex < NumLods; ++LodIndex)
	{
		SkelMeshModel.LODModels.Add(new FSkeletalMeshLODModel());

		BuildLod(SkelMeshModel.LODModels[LodIndex], *this, LodIndex);
		const FSkeletalMeshLODModel* LODModel = &(SkelMeshModel.LODModels[LodIndex]);

		FSkeletalMeshLODRenderData* LODData = new FSkeletalMeshLODRenderData();
		OutSkeletalMeshRenderData.LODRenderData.Add(LODData);
		
		LODData->BuildFromLODModel(LODModel);
	}
}
#endif  // #if WITH_EDITOR

TArray<FSkeletalMaterial> UFleshAsset::GetMaterials() const
{
	return TArray<FSkeletalMaterial>({ UMaterial::GetDefaultMaterial(EMaterialDomain::MD_Surface) });
}

const FReferenceSkeleton& UFleshAsset::GetRefSkeleton() const
{
	if (SkeletalMesh)
	{
		return SkeletalMesh->GetRefSkeleton();
	}
	else
	{
		static FReferenceSkeleton DefaultRefSkeleton = UE::FleshAsset::Private::MakeDefaultRefSkeleton();
		return DefaultRefSkeleton;
	}
}

FGuid UFleshAsset::GetAssetGuid() const
{
	return AssetGuid;
}

void UFleshAsset::PostEditCallback()
{
	//UE_LOGF(LogFleshAssetInternal, Log, "UFleshAsset::PostEditCallback()");
}

TManagedArray<FVector3f>& UFleshAsset::GetPositions()
{
	return FleshCollection->ModifyAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
}

const TManagedArray<FVector3f>* UFleshAsset::FindPositions() const
{
	return FleshCollection->FindAttributeTyped<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
}

void UFleshAsset::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	// Dataflow specific tags
	UE::Dataflow::InstanceUtils::GetAssetRegistryTags(this, Context);

	Super::GetAssetRegistryTags(Context);
}

/** Serialize */
void UFleshAsset::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	bool bCreateSimulationData = false;
	Chaos::FChaosArchive ChaosAr(Ar);
	FleshCollection->Serialize(ChaosAr);
}

TObjectPtr<UDataflowBaseContent> UFleshAsset::CreateDataflowContent()
{
	TObjectPtr<UDataflowFleshContent> SkeletalContent = NewObject<UDataflowFleshContent>(this, UDataflowFleshContent::StaticClass());
	SkeletalContent->SetIsSaved(false);

	SkeletalContent->SetDataflowOwner(this);
	SkeletalContent->SetTerminalAsset(this);

	WriteDataflowContent(SkeletalContent);
	
	return SkeletalContent;
}

const FDataflowInstance& UFleshAsset::GetDataflowInstance() const
{
	return DataflowInstance;
}

FDataflowInstance& UFleshAsset::GetDataflowInstance()
{
	return DataflowInstance;
}

void UFleshAsset::SetDataflowAsset(UDataflow* InDataflowAsset)
{
	DataflowInstance.SetDataflowAsset(InDataflowAsset);
}

UDataflow* UFleshAsset::GetDataflowAsset() const
{
	return DataflowInstance.GetDataflowAsset();
}

void UFleshAsset::SetDataflowTerminal(const FString& InDataflowTerminal)
{
	DataflowInstance.SetDataflowTerminal(FName(InDataflowTerminal));
}

FString UFleshAsset::GetDataflowTerminal() const
{
	return DataflowInstance.GetDataflowTerminal().ToString();
}

void UFleshAsset::WriteDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent) const
{
	if(const TObjectPtr<UDataflowFleshContent> SkeletalContent = Cast<UDataflowFleshContent>(DataflowContent))
	{
		SkeletalContent->SetDataflowAsset(DataflowInstance.GetDataflowAsset());
		SkeletalContent->SetDataflowTerminal(DataflowInstance.GetDataflowTerminal().ToString());
		
		SkeletalContent->SetSkeletalMesh(SkeletalMesh, true);

#if WITH_EDITORONLY_DATA
		SkeletalContent->SetAnimationAsset(PreviewAnimationAsset.LoadSynchronous());
		SkeletalContent->SolverTiming = PreviewSolverTiming;
		SkeletalContent->SolverEvolution = PreviewSolverEvolution;
		SkeletalContent->SolverCollisions = PreviewSolverCollisions;
		SkeletalContent->SolverConstraints = PreviewSolverConstraints;
		SkeletalContent->SolverForces = PreviewSolverForces;
		SkeletalContent->SolverDebugging = PreviewSolverDebugging;
		SkeletalContent->SolverMuscleActivation = PreviewSolverMuscleActivation;
#endif
	}
}

void UFleshAsset::ReadDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent)
{
	if(const TObjectPtr<UDataflowFleshContent> SkeletalContent = Cast<UDataflowFleshContent>(DataflowContent))
	{
#if WITH_EDITORONLY_DATA
		PreviewAnimationAsset = SkeletalContent->GetAnimationAsset();
		PreviewSolverTiming = SkeletalContent->SolverTiming;
		PreviewSolverEvolution = SkeletalContent->SolverEvolution;
		PreviewSolverCollisions = SkeletalContent->SolverCollisions;
		PreviewSolverConstraints = SkeletalContent->SolverConstraints;
		PreviewSolverForces = SkeletalContent->SolverForces;
		PreviewSolverDebugging = SkeletalContent->SolverDebugging;
		PreviewSolverMuscleActivation = SkeletalContent->SolverMuscleActivation;
#endif
	}
}

void UFleshAsset::PostLoad()
{
	Super::PostLoad();

	DataflowInstance.PostLoad();

	// migrate deprecated data if necessary 
	MigrateDeprecatedDataflowData();
}

void UFleshAsset::MigrateDeprecatedDataflowData()
{
#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (DataflowAsset != nullptr)
	{
		DataflowInstance.SetDataflowAsset(DataflowAsset);
		DataflowInstance.SetDataflowTerminal(FName(DataflowTerminal));

		DataflowAsset = nullptr;
		DataflowTerminal.Empty();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
}

#if WITH_EDITOR
void UFleshAsset::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.Property->GetFName();
    
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UFleshAsset, SkeletalMesh))
	{
		if(SkeletalMesh && (SkeletalMesh->GetSkeleton() != Skeleton))
		{
			Skeleton = SkeletalMesh->GetSkeleton();
		}
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UFleshAsset, Skeleton))
	{
		if(SkeletalMesh && (SkeletalMesh->GetSkeleton() != Skeleton))
		{
			SkeletalMesh = nullptr;
		}
	}
	InvalidateDataflowContents();
}

void UFleshAsset::PostEditUndo()
{
	PropagateTransformUpdateToComponents();
	Super::PostEditUndo();
}

void UFleshAsset::PropagateTransformUpdateToComponents() const
{
	for (TObjectIterator<UFleshComponent> It(RF_ClassDefaultObject, false, EInternalObjectFlags::Garbage); It; ++It)
	{
		if (It->GetRestCollection() == this)
		{
			// make sure to reset the rest collection to make sure the internal state of the components is up to date 
			// but we do not apply asset default to avoid overriding the existing overrides
			It->SetRestCollection(this);
		}
	}
}
#endif //if WITH_EDITOR

UDataflowFleshContent::UDataflowFleshContent() : Super()
{
	bHideSkeletalMesh = false;
	bHideAnimationAsset = false;
}

void UDataflowFleshContent::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);
	UDataflowFleshContent* This = CastChecked<UDataflowFleshContent>(InThis);
	Super::AddReferencedObjects(InThis, Collector);
}

void UDataflowFleshContent::SetActorProperties(TObjectPtr<AActor>& PreviewActor) const
{
	Super::SetActorProperties(PreviewActor);
	OverrideStructProperty(PreviewActor, SolverTiming, TEXT("SolverTiming"));
	OverrideStructProperty(PreviewActor, SolverEvolution, TEXT("SolverEvolution"));
	OverrideStructProperty(PreviewActor, SolverCollisions, TEXT("SolverCollisions"));
	OverrideStructProperty(PreviewActor, SolverConstraints, TEXT("SolverConstraints"));
	OverrideStructProperty(PreviewActor, SolverForces, TEXT("SolverForces"));
	OverrideStructProperty(PreviewActor, SolverDebugging, TEXT("SolverDebugging"));
	OverrideStructProperty(PreviewActor, SolverMuscleActivation, TEXT("SolverMuscleActivation"));
}

#if WITH_EDITOR

void UDataflowFleshContent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	SetSimulationDirty(true);
}

#endif //if WITH_EDITOR



