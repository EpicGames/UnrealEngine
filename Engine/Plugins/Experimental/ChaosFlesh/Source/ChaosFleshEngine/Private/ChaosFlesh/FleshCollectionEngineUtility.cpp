// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ChaosDeformableTetrahedralComponent methods.
=============================================================================*/

#include "ChaosFlesh/FleshCollectionEngineUtility.h"

#include "ChaosFlesh/ChaosFlesh.h"
#include "ChaosFlesh/FleshCollection.h"
#include "GeometryCollection/Facades/CollectionTetrahedralBindingsFacade.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"

namespace ChaosFlesh
{
	FString GetMeshId(const USkeletalMesh* SkeletalMesh, const bool bUseImportModel)
	{
		FPrimaryAssetId Id = SkeletalMesh->GetPrimaryAssetId();
		FString MeshId = Id.IsValid() ? Id.ToString() : SkeletalMesh->GetName();
		if (bUseImportModel)
		{
			MeshId.Append(TEXT("_ImportModel"));
		}
		return MeshId;
	}

	FString GetMeshId(const UStaticMesh* StaticMesh)
	{
		FPrimaryAssetId Id = StaticMesh->GetPrimaryAssetId();
		FString MeshId = Id.IsValid() ? Id.ToString() : StaticMesh->GetName();
		return MeshId;
	}

	void BoundSurfacePositions(
		const FString& MeshId,
		const FFleshCollection* FleshCollection,
		const TManagedArray<FVector3f>* RestVertices,
		const TManagedArray<FVector3f>* SimulatedVertices,
		TArray<FVector3f>& OutPositions)
	{
		GeometryCollection::Facades::FTetrahedralBindings TetBindings(*FleshCollection);
		FName MeshIdName(MeshId);

		const int32 LODIndex = 0;
		const int32 TetIndex = TetBindings.GetTetMeshIndex(MeshIdName, LODIndex);
		if (TetIndex == INDEX_NONE)
		{
			UE_LOGF(LogChaosFlesh, Error,
				"CreateGeometryCache - No tet mesh index associated with mesh '%ls' LOD: %d",
				*MeshId, LODIndex);
			return;
		}
		if (!TetBindings.ReadBindingsGroup(TetIndex, MeshIdName, LODIndex))
		{
			UE_LOGF(LogChaosFlesh, Error,
				"CreateGeometryCache - Failed to read bindings group associated with mesh '%ls' LOD: %d",
				*MeshId, LODIndex);
			return;
		}

		TUniquePtr<GeometryCollection::Facades::FTetrahedralBindings::Evaluator> BindingsEvalPtr = TetBindings.InitEvaluator(RestVertices);
		const GeometryCollection::Facades::FTetrahedralBindings::Evaluator& BindingsEval = *BindingsEvalPtr.Get();
		if (!BindingsEval.IsValid())
		{
			UE_LOGF(LogChaosFlesh, Error,
				"CreateGeometryCache - Min parent index = %d at tet index %d",
				BindingsEval.MinIndex(), BindingsEval.MinIndexPosition());
			UE_LOGF(LogChaosFlesh, Error,
				"CreateGeometryCache - Invalid flesh bindings for mesh [%ls]",
				*MeshId);
			return;
		}

		TArray<Chaos::FVec3f> CurrVertices;
		CurrVertices.SetNum(SimulatedVertices->Num());
		for (int32 Index = 0; Index < CurrVertices.Num(); ++Index)
		{
			CurrVertices[Index] = Chaos::FVec3f((*SimulatedVertices)[Index]);
		}

		const int32 NumVertices = BindingsEval.NumVertices();
		OutPositions.SetNum(NumVertices);
		for (int32 Index = 0; Index < NumVertices; ++Index)
		{
			OutPositions[Index] = BindingsEval.GetEmbeddedPosition(Index, CurrVertices);
		}
	}
	void BoundSurfacePositions(
		const USkeletalMesh* SkeletalMesh,
		const FFleshCollection* FleshCollection,
		const TManagedArray<FVector3f>* RestVertices,
		const TManagedArray<FVector3f>* SimulatedVertices,
		TArray<FVector3f>& OutPositions)
	{
		FString MeshId = GetMeshId(SkeletalMesh, false);
		BoundSurfacePositions(MeshId, FleshCollection, RestVertices, SimulatedVertices, OutPositions);
	}

	void BoundSurfacePositions(
		const UStaticMesh* StaticMesh,
		const FFleshCollection* FleshCollection,
		const TManagedArray<FVector3f>* RestVertices,
		const TManagedArray<FVector3f>* SimulatedVertices,
		TArray<FVector3f>& OutPositions)
	{
		FString MeshId = GetMeshId(StaticMesh);
		BoundSurfacePositions(MeshId, FleshCollection, RestVertices, SimulatedVertices, OutPositions);
	}
}
