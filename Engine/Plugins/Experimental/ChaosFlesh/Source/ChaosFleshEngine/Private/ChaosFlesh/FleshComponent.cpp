// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFlesh/FleshComponent.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(FleshComponent)

DEFINE_LOG_CATEGORY_STATIC(LogFleshComponentInternal, Log, All);

UFleshComponent::UFleshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

