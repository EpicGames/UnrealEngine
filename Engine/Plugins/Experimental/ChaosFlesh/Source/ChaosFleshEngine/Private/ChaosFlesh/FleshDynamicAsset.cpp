// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FleshAsset.cpp: UFleshAsset methods.
=============================================================================*/
#include "ChaosFlesh/FleshDynamicAsset.h"
#include "GeometryCollection/GeometryCollection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FleshDynamicAsset)

DEFINE_LOG_CATEGORY_STATIC(LogFleshDynamicAssetInternal, Log, All);


UFleshDynamicAsset::UFleshDynamicAsset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, DynamicCollection(new FManagedArrayCollection())
{
	Init();
}

void UFleshDynamicAsset::Init()
{
	DynamicCollection->AddAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
	DynamicCollection->AddAttribute<int32>("ObjectState", FGeometryCollection::VerticesGroup);
	DynamicCollection->AddAttribute<float>("Activation", FGeometryCollection::VerticesGroup);
}

void UFleshDynamicAsset::Reset(const FManagedArrayCollection* InCopyFrom)
{
	DynamicCollection = TSharedPtr<FManagedArrayCollection, ESPMode::ThreadSafe>( new FManagedArrayCollection());
	Init();

	if (InCopyFrom)
	{
		GetObjectState().Fill((int32)Chaos::EObjectStateType::Dynamic);
		DynamicCollection->CopyMatchingAttributesFrom(*InCopyFrom);
	}
}

void UFleshDynamicAsset::ResetAttributesFrom(const FManagedArrayCollection* InCopyFrom)
{
	if (InCopyFrom)
	{
		DynamicCollection->CopyMatchingAttributesFrom(*InCopyFrom);
	}
}


TManagedArray<FVector3f>& UFleshDynamicAsset::GetPositions()
{
	return DynamicCollection->ModifyAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
}

const TManagedArray<FVector3f>* UFleshDynamicAsset::FindPositions() const
{
	return DynamicCollection->FindAttributeTyped<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
}

TManagedArray<int32>& UFleshDynamicAsset::GetObjectState()
{
	return DynamicCollection->ModifyAttribute<int32>("ObjectState", FGeometryCollection::VerticesGroup);
}

TManagedArray<float>& UFleshDynamicAsset::GetActivations()
{
	return DynamicCollection->ModifyAttribute<float>("Activation", FGeometryCollection::VerticesGroup);
}

const TManagedArray<float>* UFleshDynamicAsset::FindActivations() const
{
	return DynamicCollection->FindAttributeTyped<float>("Activation", FGeometryCollection::VerticesGroup);
}