// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FleshAsset.cpp: UFleshAsset methods.
=============================================================================*/
#include "ChaosFlesh/SimulationAsset.h"
#include "GeometryCollection/GeometryCollection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationAsset)

DEFINE_LOG_CATEGORY_STATIC(LogSimulationAssetInternal, Log, All);


USimulationAsset::USimulationAsset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, SimulationCollection(new FManagedArrayCollection())
{
	Init();
}

void USimulationAsset::Init()
{
	SimulationCollection->AddAttribute<int32>("ObjectState", FGeometryCollection::VerticesGroup);
}

void USimulationAsset::Reset(const FManagedArrayCollection* InCopyFrom)
{
	SimulationCollection = TSharedPtr<FManagedArrayCollection, ESPMode::ThreadSafe>( new FManagedArrayCollection());
	Init();

	if (InCopyFrom)
	{
		GetObjectState().Fill((int32)Chaos::EObjectStateType::Dynamic);
		SimulationCollection->CopyMatchingAttributesFrom(*InCopyFrom);
	}
}

void USimulationAsset::ResetAttributesFrom(const FManagedArrayCollection* InCopyFrom)
{
	if (InCopyFrom)
	{
		SimulationCollection->CopyMatchingAttributesFrom(*InCopyFrom);
	}
}

TManagedArray<int32>& USimulationAsset::GetObjectState()
{
	return SimulationCollection->ModifyAttribute<int32>("ObjectState", FGeometryCollection::VerticesGroup);
}

