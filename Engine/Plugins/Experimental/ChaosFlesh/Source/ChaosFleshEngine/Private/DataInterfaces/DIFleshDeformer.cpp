// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataInterfaces/DIFleshDeformer.h"

#include "ChaosFlesh/ChaosFleshDeformerBufferManager.h"
#include "Components/SkeletalMeshComponent.h"
#include "Containers/Set.h"
#include "ComputeFramework/ComputeKernelPermutationVector.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "GeometryCollection/Facades/CollectionTetrahedralBindingsFacade.h"
#include "OptimusDataDomain.h"
#include "ShaderParameterMetadataBuilder.h"
#include "RenderGraphBuilder.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "ShaderCompilerCore.h"
#include "SkeletalMeshDeformerHelpers.h"
#include "SkeletalRenderPublic.h"

DEFINE_LOG_CATEGORY(LogFleshDeformer);

//
// Interface
//

UDIFleshDeformer::~UDIFleshDeformer()
{
	if (UDeformableTetrahedralComponent* Component = ProducerComponent.Get())
	{
		Component->GetGPUBufferManager().UnRegisterGPUBufferConsumer(this);
	}
}

FString
UDIFleshDeformer::GetDisplayName() const
{
	return TEXT("Flesh Deformer");
}

TArray<FOptimusCDIPinDefinition>
UDIFleshDeformer::GetPinDefinitions() const
{
	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({ "Mask", "ReadMask", Optimus::DomainName::Vertex, "ReadNumVertices" });
	Defs.Add({ "Parents", "ReadParents", Optimus::DomainName::Vertex, "ReadNumVertices" });
	Defs.Add({ "Weights", "ReadWeights", Optimus::DomainName::Vertex, "ReadNumVertices"  });
	Defs.Add({ "Offset", "ReadOffset", Optimus::DomainName::Vertex, "ReadNumVertices" });
	Defs.Add({ "EmbeddedPos", "GetEmbeddedPos", Optimus::DomainName::Vertex, "ReadNumVertices" });
	return Defs;
}

TSubclassOf<UActorComponent> 
UDIFleshDeformer::GetRequiredComponentClass() const
{
	return USkinnedMeshComponent::StaticClass();
}

void
UDIFleshDeformer::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumVertices"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadMask"))
		.AddReturnType(EShaderFundamentalType::Float)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadParents"))
		.AddReturnType(EShaderFundamentalType::Int, 4) // int4
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadWeights"))
		.AddReturnType(EShaderFundamentalType::Float, 4) // float4
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadOffset"))
		.AddReturnType(EShaderFundamentalType::Float, 3) // float4
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("GetEmbeddedPos"))
		.AddReturnType(EShaderFundamentalType::Float, 3) // float3
		.AddParam(EShaderFundamentalType::Uint);

}

BEGIN_SHADER_PARAMETER_STRUCT(FFleshDeformerDataInterfaceParameters, )
	SHADER_PARAMETER(uint32, NumVertices)
	SHADER_PARAMETER(uint32, NumTetRestVertexBuffer)
	SHADER_PARAMETER(uint32, NumTetVertexBuffer)

	SHADER_PARAMETER(uint32, NumVertsToParentsBuffer)
	SHADER_PARAMETER(int32, VertsToParentsBufferOffset)
	SHADER_PARAMETER(uint32, VertsToParentsBufferStride)

	SHADER_PARAMETER(uint32, NumParentsBuffer)
	SHADER_PARAMETER(int32, ParentsBufferOffset)
	SHADER_PARAMETER(uint32, ParentsBufferStride)

	SHADER_PARAMETER(uint32, NumWeightsBuffer)
	SHADER_PARAMETER(uint32, NumOffsetBuffer)
	SHADER_PARAMETER(uint32, NumMaskBuffer)
	SHADER_PARAMETER_SRV(Buffer<float>, TetRestVertexBuffer)
	SHADER_PARAMETER_SRV(Buffer<float>, TetVertexBuffer)
	SHADER_PARAMETER_SRV(Buffer<uint32>, VertsToParentsBuffer)
	SHADER_PARAMETER_SRV(Buffer<uint32>, ParentsBuffer)
	SHADER_PARAMETER_SRV(Buffer<FFloat16>, WeightsBuffer)
	SHADER_PARAMETER_SRV(Buffer<FFloat16>, OffsetBuffer)
	SHADER_PARAMETER_SRV(Buffer<FFloat16>, MaskBuffer)
END_SHADER_PARAMETER_STRUCT()

void
UDIFleshDeformer::GetShaderParameters(
	TCHAR const* UID,
	FShaderParametersMetadataBuilder& InOutBuilder,
	FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FFleshDeformerDataInterfaceParameters>(UID);
}

void
UDIFleshDeformer::GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const
{
	OutPermutationVector.AddPermutation(TEXT("ENABLE_DEFORMER_FLESH"), 2);
}

void
UDIFleshDeformer::GetShaderHash(FString& InOutKey) const
{
	// Module registers "ChaosFlesh/Shaders/Private" as virtual shader path: "/Plugin/ChaosFlesh".
	GetShaderFileHash(
		TEXT("/Plugin/ChaosFlesh/Private/DIFleshDeformer.ush"),
		EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void
UDIFleshDeformer::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(
		TEXT("/Plugin/ChaosFlesh/Private/DIFleshDeformer.ush"),
		EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider*
UDIFleshDeformer::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UDIFleshDeformerDataProvider* Provider = NewObject<UDIFleshDeformerDataProvider>();

	if ((Provider->SkinnedMesh = Cast<USkinnedMeshComponent>(InBinding)))
	{
		AActor* Actor = Provider->SkinnedMesh->GetOwner();
		const TSet<UActorComponent*>& Components = Actor->GetComponents();

		// Count the number of flesh components we have.
		int32 NumFleshComponents = 0;
		for (TSet<UActorComponent*>::TConstIterator It = Components.CreateConstIterator(); It; ++It)
		{
			NumFleshComponents += const_cast<UDeformableTetrahedralComponent*>(Cast<UDeformableTetrahedralComponent>(*It)) != nullptr;
		}

		// Find the flesh component that is associated with this skeletal mesh.
		for (TSet<UActorComponent*>::TConstIterator It = Components.CreateConstIterator(); It; ++It)
		{
			if (const UActorComponent* Component = *It)
			{
				if (UDeformableTetrahedralComponent* FleshComponent = const_cast<UDeformableTetrahedralComponent*>(Cast<UDeformableTetrahedralComponent>(Component)))
				{
					if (const UFleshAsset* FleshAsset = FleshComponent->GetRestCollection())
					{
						if (FleshAsset->TargetDeformationSkeleton == Cast<USkeletalMesh>(Provider->SkinnedMesh->GetSkinnedAsset()))
						{
							Provider->FleshMesh = FleshComponent;
							ProducerComponent = FleshComponent;
							break;
						}
						else if (!FleshAsset->TargetDeformationSkeleton && NumFleshComponents == 1)
						{
							// The typical case is probably that we only have 1 skeletal mesh and 1 flesh
							// component.  So, in an effort to keep the easy case easy, if the TargetSkeletalMesh
							// isn't set, just do the obvious thing and use it.
							UE_LOGF(LogFleshDeformer, Display,
								"FleshAsset '%ls.%ls.RestCollection' has no value for 'TargetDeformationSkeleton', "
									"using it for skeletal mesh '%ls' anyway as it's the only UFleshComponent available.",
								*Actor->GetName(),
								*FleshAsset->GetName(),
								*Provider->SkinnedMesh->GetName());
							Provider->FleshMesh = FleshComponent;
							ProducerComponent = FleshComponent;
							break;
						}
						else if (!FleshAsset->TargetDeformationSkeleton)
						{
							UE_LOGF(LogFleshDeformer, Warning,
								"FleshAsset '%ls.%ls.RestCollection' has no value for 'TargetDeformationSkeleton'; "
									"not using it for skeletal mesh '%ls' as there are %d UFleshComponent candidates.",
								*Actor->GetName(),
								*FleshAsset->GetName(),
								*Provider->SkinnedMesh->GetName(),
								NumFleshComponents);
						}
					}
				}
			}
		}

		if (!Provider->FleshMesh)
		{
			if (NumFleshComponents)
			{
				UE_LOGF(LogFleshDeformer, Error,
					"Found %d UFleshComponents under actor '%ls', none of which were associated with skeletal mesh '%ls'. "
						"Disambiguate the association by setting 'FleshAsset.RestCollection.TargetDeformationSkeleton' to the "
						"skeletal mesh it should deform.",
					NumFleshComponents,
					*Actor->GetName(),
					*Provider->SkinnedMesh->GetName());
			}
			else
			{
				UE_LOGF(LogFleshDeformer, Error,
					"No UFleshComponent found under actor '%ls' for skeletal mesh '%ls'.",
					*Actor->GetName(),
					*Provider->SkinnedMesh->GetName());
			}
		}
	}
	// TODO: Someday, if/when Optimus supports deforming static meshes...
	//if (Provider->StaticMesh = Cast<UStaticMeshComponent>(InBinding))
	//{}

	Provider->FleshDeformerParameters = FleshDeformerParameters;
	return Provider;
}

//
// DataProvider
//

FComputeDataProviderRenderProxy*
UDIFleshDeformerDataProvider::GetRenderProxy()
{
	return new FDIFleshDeformerProviderProxy(SkinnedMesh, FleshMesh, FleshDeformerParameters);
}

//
// Proxy
//

FDIFleshDeformerProviderProxy::FDIFleshDeformerProviderProxy(
	USkinnedMeshComponent* SkinnedMeshComponentIn, 
	UDeformableTetrahedralComponent* FleshComponentIn,
	FFleshDeformerParameters const& FleshDeformerParametersIn)
	: SkinnedMeshComponent(SkinnedMeshComponentIn)
	, FleshComponent(FleshComponentIn)
	, FleshDeformerParameters(FleshDeformerParametersIn)
{
	if (SkinnedMeshComponentIn && SkinnedMeshComponentIn->IsRegistered())
	{
		SceneInterface = SkinnedMeshComponentIn->GetScene();
		ComponentId = SkinnedMeshComponentIn->GetPrimitiveSceneId();
	}

	if (SkinnedMeshComponent)
	{
		if (USkinnedAsset* SkinnedAsset = SkinnedMeshComponent->GetSkinnedAsset())
		{
			FPrimaryAssetId Id = SkinnedAsset->GetPrimaryAssetId();
			MeshId = Id.IsValid() ? Id.ToString() : SkinnedAsset->GetName();
		}
	}
}

bool 
FDIFleshDeformerProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}

	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	if (SkinnedMeshComponent == nullptr || !SkeletalMeshObject || FleshComponent == nullptr)
	{
		return false;
	}

	const UFleshAsset* RestCollectionAsset = FleshComponent->GetRestCollection();
	if (!RestCollectionAsset)
	{
		return false;
	}

	TSharedPtr<const FFleshCollection> FleshCollection = RestCollectionAsset->GetFleshCollection();
	if (!FleshCollection)
	{
		return false;
	}

	// This function is called every draw call.  It may be better to do this initialization
	// higher in the stack, but we do it here to support on the fly LOD switching.  I think...
	FName MeshName(*MeshId, MeshId.Len());
	if (!FleshComponent->GetGPUBufferManager().InitGPUBindingsBuffer(SkinnedMeshComponent, MeshName, SkeletalMeshObject->GetLOD()))
	{
		return false;
	}

	return true;
}

struct FFleshDeformerDataInterfacePermutationIds
{
	uint32 EnableDeformerFlesh = 0;
	uint32 EnableSurfaceOffsets = 0;

	FFleshDeformerDataInterfacePermutationIds(FComputeKernelPermutationVector const& PermutationVector)
	{
		{
			static FString Name(TEXT("ENABLE_DEFORMER_FLESH"));
			static uint32 Hash = GetTypeHash(Name);
			EnableDeformerFlesh = PermutationVector.GetPermutationBits(Name, Hash, 1);
		}
	}
};

void
FDIFleshDeformerProviderProxy::GatherPermutations(FPermutationData& InOutPermutationData) const
{
	FFleshDeformerDataInterfacePermutationIds PermutationIds(InOutPermutationData.PermutationVector);
	for (int32 InvocationIndex = 0; InvocationIndex < InOutPermutationData.NumInvocations; ++InvocationIndex)
	{
		InOutPermutationData.PermutationIds[InvocationIndex] |= PermutationIds.EnableDeformerFlesh;
	}
}

void
FDIFleshDeformerProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{}

void
FDIFleshDeformerProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[SkeletalMeshObject->GetLOD()];
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);

	FName MeshName(*MeshId, MeshId.Len());
	Chaos::Softs::FChaosFleshDeformableGPUManager& GPUManager = FleshComponent->GetGPUBufferManager();
	Chaos::Softs::FChaosFleshDeformableGPUManager::FBindingsBuffer* Bindings = GPUManager.GetBindingsBuffer(SkinnedMeshComponent, MeshName, SkeletalMeshObject->GetLOD());
	if (!Bindings)
	{
		return;
	}

	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];

		Parameters.NumVertices = LodRenderData->GetNumVertices();

		//
		// Time varying
		//

		// Sparsified tet vertices - only referenced parent vertices.
		// SHADER_PARAMETER(uint32, NumTetVertexBuffer)
		// SHADER_PARAMETER_SRV(Buffer<float>, TetVertexBuffer)
		Parameters.NumTetVertexBuffer = Bindings->GetVerticesBuffer().GetNumValues();
		Parameters.TetVertexBuffer = Bindings->GetVerticesBuffer().ShaderResourceViewRHI;

		//
		// Unvarying (typically)
		//

		// Sparsified rest tet vertices - only referenced parent vertices.
		// SHADER_PARAMETER(uint32, NumTetRestVertexBuffer)
		// SHADER_PARAMETER_SRV(Buffer<float>, TetRestVertexBuffer)
		Parameters.NumTetRestVertexBuffer = Bindings->GetRestVerticesBuffer().GetNumValues();
		Parameters.TetRestVertexBuffer = Bindings->GetRestVerticesBuffer().ShaderResourceViewRHI;

		// Per vertex packed (8, 16, or 32 bit) index to sparsified parents
		// SHADER_PARAMETER(uint32, NumVertsToParentsBuffer)
		// SHADER_PARAMETER(int32, VertsToParentsBufferOffset)
		// SHADER_PARAMETER(uint32, VertsToParentsBufferStride)
		// SHADER_PARAMETER_SRV(Buffer<uint32>, VertsToParentsBuffer)
		Parameters.NumVertsToParentsBuffer = Bindings->GetVertsToParentsBuffer().GetNumValues();
		Parameters.VertsToParentsBufferOffset = Bindings->GetVertsToParentsBuffer().GetOffset();
		Parameters.VertsToParentsBufferStride = Bindings->GetVertsToParentsBuffer().GetDataStride();
		Parameters.VertsToParentsBuffer = Bindings->GetVertsToParentsBuffer().ShaderResourceViewRHI;

		// Sparsified (only unique) and packed (8, 16, or 32 bit) parents.
		// SHADER_PARAMETER(uint32, NumParentsBuffer)
		// SHADER_PARAMETER(int32, ParentsBufferOffset)
		// SHADER_PARAMETER(uint32, ParentsBufferStride)
		// SHADER_PARAMETER_SRV(Buffer<uint32>, ParentsBuffer)
		Parameters.NumParentsBuffer = Bindings->GetParentsBuffer().GetNumValues();
		Parameters.ParentsBufferOffset = Bindings->GetParentsBuffer().GetOffset();
		Parameters.ParentsBufferStride = Bindings->GetParentsBuffer().GetDataStride();
		Parameters.ParentsBuffer = Bindings->GetParentsBuffer().ShaderResourceViewRHI;

		// Per vertex half precision floating point barycentric weights.
		// SHADER_PARAMETER(uint32, NumWeightsBuffer)
		//SHADER_PARAMETER_SRV(Buffer<FFloat16>, WeightsBuffer)
		Parameters.NumWeightsBuffer = Bindings->GetWeightsBuffer().GetNumValues();
		Parameters.WeightsBuffer = Bindings->GetWeightsBuffer().ShaderResourceViewRHI;

		// Per vertex half precision floating point offset vector.
		//SHADER_PARAMETER(uint32, NumOffsetBuffer)
		//SHADER_PARAMETER_SRV(Buffer<FFloat16>, OffsetBuffer)
		Parameters.NumOffsetBuffer = Bindings->GetOffsetsBuffer().GetNumValues();
		Parameters.OffsetBuffer = Bindings->GetOffsetsBuffer().ShaderResourceViewRHI;

		// Per vertex half precision floating point mask (amount).
		//SHADER_PARAMETER(uint32, NumMaskBuffer)
		//SHADER_PARAMETER_SRV(Buffer<FFloat16>, MaskBuffer)
		Parameters.NumMaskBuffer = Bindings->GetMaskBuffer().GetNumValues();
		Parameters.MaskBuffer = Bindings->GetMaskBuffer().ShaderResourceViewRHI;
	}
}
