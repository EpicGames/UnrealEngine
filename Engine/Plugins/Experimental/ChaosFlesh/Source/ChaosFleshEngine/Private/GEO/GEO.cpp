// Copyright Epic Games, Inc. All Rights Reserved.
#include "GEO/GEO.h"

#if WITH_EDITOR

#include "GEO/IFileStream.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"

#ifdef USE_ZLIB
#include "ChaosFlesh/ZIP.h"
#endif

#include "Containers/StringConv.h"

#include <algorithm>
#include <charconv>
#include <fstream>
#include <iostream>
#include <sstream>
#include <memory>

namespace ChaosFlesh {

using namespace std;

//! Reads "abc 123", consuming quotes but returns unquoted.
//! \p delim denotes the character to read to (and is discarded)
//! \p bracketed specifies if the string is deliminated at the beginning and the end.
bool
Read(std::istream &inref, std::string &buffer, const char delim='"', const bool bracketed=true)
{
    buffer.clear();
    bool open=!bracketed;
    while(inref.good())
    {
        char ch;
        inref.get(ch); // consume next char
        if(ch == delim)
        {
            if(!open) 
                open = true;
            else
                return true;
        }
        else
        {
            buffer.append(1, ch);
        }
    }
    return false;
}

//! Given "[...]..." returns "[...]".  Given "[...[...]...]..." returns "[...[...]...]".
bool
ReadMatching(
	std::istream &inref, 
	std::string &buffer, 
	const char openDelim='[', 
	const char closeDelim=']', 
	bool stripBrackets=false)
{
    buffer.clear();
    int open = 0;
    inref >> std::ws; // discard white space
    bool inQuote = false;
    const bool sameChar = openDelim == closeDelim;
    while(inref.good())
    {
        if(!inQuote)          // if not currently parsing a quoted string...
            inref >> std::ws; // discard white space

        char ch;
        inref.get(ch); // consume next char

        if(sameChar)
        {
            if(ch == openDelim)
            {
                if(!open)
                {
                    open++;
                    if(!stripBrackets) buffer.append(1, ch);
                }
                else
                {
                    open--;
                    if(!stripBrackets) buffer.append(1, ch);
                }
            }
            else
                buffer.append(1, ch);
        }
        else
        {
            if(ch == openDelim)
            {
                open++;
                if(!stripBrackets || (stripBrackets && open > 1)) buffer.append(1, ch);
            }
            else if(ch == closeDelim)
            {
                if(!stripBrackets || (stripBrackets && open > 1)) buffer.append(1, ch);
				open--;
			}
            else 
                buffer.append(1, ch);
        }

        if(ch == '"')
            inQuote=!inQuote;
        if(!open)
            return true;
    }
    return false;
}

//! Read numeric characters from \p inref storing them in \p buffer, until a 
//! non-numeric character is encountered.  Doesn't try to parse or validate.
bool
ReadNumeric(std::istream &inref, std::string &buffer)
{
    buffer.clear();
    inref >> std::ws; // discard white space
    while(inref.good())
    {
        inref >> std::ws; // discard white space
        const char nch = inref.peek();
        switch(nch)
        {
        case '0':
        case '1':
        case '2':
        case '3':
        case '4':
        case '5':
        case '6':
        case '7':
        case '8':
        case '9':

        case 'e':
        case 'E':

        case '.':
        case '-':
            buffer.append(1, nch);
            inref.ignore();
            break;
        default:
            return !buffer.empty();
        }
    }
    return !buffer.empty();
}

std::string
ParseString(const std::string& str)
{
	if (str.empty())
		return str;
	if (str.substr(0, 2) == "\\\"" && str.substr(str.size() - 2, 2) == "\\\"")
		return str.substr(2, str.size() - 4);
	else if (str.front() == '"' && str.back() == '"')
		return str.substr(1, str.size() - 2);
	return str;
}

bool
ParseGeoKVPairs(
    std::istream &inref, 
    TMap<FString, std::string> &kvp,
    const char *filename=nullptr, 
    std::ostream *errorStream=nullptr,
	TArray<std::string> *allValues=nullptr)
{
    std::string key; 
    key.reserve(128);
    std::string buffer;
    buffer.reserve(2048);

    bool globalOpenBracket = false;
    while(inref.good())
    {
        inref >> std::ws; // discard white space

        //
        // Read key
        // 

        if(key.empty())
        {
            const char nch = inref.peek();
            switch(nch)
            {
            case '[':   // document and section (nv pair list)
            case '{':   // dictionary parsing
                // Aside from the global opening bracket, we should not encounter brackets on the key side.
                if(globalOpenBracket) 
                    goto error;
                globalOpenBracket=true;
                inref.ignore(); // Skip this char
                continue;       // Try again for key
            case ']':
            case '}':
                // Closing bracket must match the opening bracket.
                if(!globalOpenBracket)
                    goto error;
                globalOpenBracket=false;
                inref.ignore(); // Skip this char
                break;          // Stop parsing
            case '"':
                // Keys should be quoted strings.
                if(!Read(inref, buffer))
                    goto error;
                key = buffer;
                break;
            }

            if(key.empty())
            {
                if(!globalOpenBracket)
                {
                    // done reading
                    break;
                }
                if(errorStream) 
                    *errorStream << "Failed to determine key at file position: " << inref.tellg() << "." << std::endl;
                goto error;
            }
            continue;
        }

        //
        // Read value
        //

        buffer.clear();
        const char nch = inref.peek();
        switch(nch)
        {
        case ',':       // "<key name>",
        case ':':       // "<key name>":
            inref.ignore(); // Skip this char
            continue;       // Try again for value
        case '"':       // "<key name>","
            if(!Read(inref, buffer))
                goto error;
            break;
        case '{':       // "<key name>",{
            if(!ReadMatching(inref, buffer, '{', '}'))
                goto error;
            break;
        case '[':       // "<key name>",[
            if(!ReadMatching(inref, buffer, '[', ']'))
                goto error;
            break;
        case 't':       // "<key name>",true
            buffer = "true";
            inref.ignore(4);
            if(!inref.good())
                goto error;
            break;
        case 'f':       // "<key name>",false
            buffer = "false";
            inref.ignore(5);
            if(!inref.good())
                goto error;
            break;
        default:       // "<key name>",<numeric value>
            if(!ReadNumeric(inref, buffer))
                goto error;
            break;
        }

        if(inref.peek()==',') // consume trailing ','
            inref.ignore(); 

		kvp.Add(FString(key.c_str()), buffer);
		if (allValues)
		{
			allValues->Add(key);
			allValues->Add(buffer);
		}
		key.clear();
        buffer.clear();
    } // end while(inref.good())

    return true;

error:
    if(errorStream && filename) 
        *errorStream << "ParseGeoKVPairs(): parse error in file '" << filename << "' at position: " << inref.tellg() << "." << std::endl;
    return false;
}

bool
ParseKVPairs(
    const std::string &str, 
    TMap<FString, std::string> &kvp,
    const char *filename=nullptr, 
    std::ostream *errorStream=nullptr,
	TArray<std::string>* allValues=nullptr)
{
    std::stringstream ss(str);
    return ParseGeoKVPairs(ss, kvp, filename, errorStream, allValues);
}

std::string
FormatStringArray(const TArray<std::string> &values, const char openBracket='[', const char closeBracket=']')
{
    std::string retval;
    retval += openBracket;
    for(size_t i=0; i < values.Num(); i++)
    {
        if(i > 0) retval += ",";
        retval += '"' + values[i] + '"';
    }
    retval += closeBracket;
    return retval;
}

/*
//! Prints \p kvp by sorted keys.  Values longer than \p maxLen are truncated in the middle: "abcd<...>wxyz".
void
DebugPrint(const TMap<FString,std::string> &kvp, const std::string &prefix="    ", const size_t maxLen=128)
{
    //std::vector<std::string> keys;
    //for(auto it=kvp.begin(), itEnd=kvp.end(); it != itEnd; ++it)
    //    keys.push_back(it->first);
    //std::sort(keys.begin(), keys.end());
	TArray<FString> keys;
	kvp.GetKeys(keys);
	keys.Sort();

    for(const auto &key : keys)
    {
        //const std::string &value = kvp.find(key)->second;
		const std::string& value = *kvp.Find(key);
        if(value.size() > maxLen)
        {
            std::cout << prefix
                << key << " = "
                << value.substr(0,maxLen/2) << "<...>" 
                << value.substr(value.size()-(maxLen/2)) << std::endl;
        }
        else
        {
            std::cout << prefix
                << key << " = " << value << std::endl;
        }
    }
}

void
DebugPrint(const TMap<FString,TMap<FString,std::string>> &kvp, const std::string &prefix="    ")
{
    for(auto it=kvp.begin(), itEnd=kvp.end(); it != itEnd; ++it)
        DebugPrint(it->second, prefix+'.'+it->first+'.');
}
*/

//! Array format:
//! [ [...],[...],... ]
bool
ParseArray(const std::string &strIn, TArray<std::string> &values, const char openBracket='[', const char closeBracket=']', const bool stripInternalBrackets=false)
{
    values.Empty();
    bool open = false;
    std::string buffer;

    std::stringstream ss(strIn);
    ss >> std::ws; // discard white space

    while(ss.good())
    {
        ss >> std::ws; // discard white space
        char ch = ss.peek();

        if(ch == openBracket)
        {
            if(!open)
            {
                ss.ignore(); // consume this char
                open = true;
                continue;
            }
            else
            {
                if(ReadMatching(ss, buffer, openBracket, closeBracket, stripInternalBrackets))
                    values.Add(buffer);
                else
                    return false;
            }
        }
        else if(ch == '"') // array of strings
        {
            if(ReadMatching(ss, buffer, '"', '"', stripInternalBrackets))
                values.Add(buffer);
            else
                return false;
        }
		else if (isdigit(ch))
		{
			buffer.resize(0);
			do {
				buffer.push_back(ch);
				ss.ignore(); // consume this char
				if (!ss.good())
					break;
				ch = ss.peek();
			} while (isdigit(ch));
			values.Add(buffer);
		}
        else if(ch == ',')
        {
            ss.ignore(); // consume this char
            continue;
        }
        else if(ch == closeBracket)
        {
            break;
        }
        else
        {
            std::cout << "ParseArray() - parse failure at position " << ss.tellg() 
                << ", expected '[],', got '" << ch << "', "
                << " input string: '" << strIn << "'" << std::endl;
            return false;
        }
    }
	return true;
}

#if defined(__GNUC__)
    // The gnu stdlib has issues with the floating point versions of std::from_chars() (their 
    // implementations allocate memory, which the standard forbids), and they hide them behind
    // a macro through v20.  Lame.  See: https://gcc.gnu.org/bugzilla/show_bug.cgi?id=100146
bool
ParseNum(const std::string &strIn, double &value)
{
    const char* str = strIn.c_str();
    char* strEnd = nullptr;
    value = std::strtod(str, &strEnd);
    if(value == 0. && strEnd == str)
        return false;
    return true;
}
bool
ParseNum(const std::string &strIn, float &value)
{
    const char* str = strIn.c_str();
    char* strEnd = nullptr;
    value = std::strtof(str, &strEnd);
    if(value == 0. && strEnd == str)
        return false;
    return true;
}
#endif

//! Parse a numeric value of type \p T from \p strIn.  Returns \c false on error.
template <class T>
#if defined(__GNUC__)
    // The gnu stdlib has issues with the floating point versions of std::from_chars() (their 
    // implementations allocate memory, which the standard forbids), and they hide them behind
    // a macro through v20.  Lame.  See: https://gcc.gnu.org/bugzilla/show_bug.cgi?id=100146
    //
    // Enable if T is integral on gnu.
typename std::enable_if<std::is_integral<T>::value, bool>::type
#else
bool
#endif
ParseNum(const std::string &strIn, T &value)
{

    //if(std::from_chars(&strIn[0], &strIn[0]+strIn.size(), value).ec != std::errc())
    if(std::from_chars(strIn.c_str(), strIn.c_str()+strIn.size(), value).ec != std::errc())
        return false;
    return true;
}

//! Parses numeric values of type \p T from \p strIn in array format demarked by 
//! \p openBracket and \p closeBracket, with values delinated by commas.  
//! <code>
//! TArray<int32> values; 
//! ParseNum("[1, 2, 3, 4, 5, 6, 7, 8, 9]", values);
//! <\code>
template <class T>
bool
ParseNumArray(const std::string &strIn, TArray<T> &values, const char openBracket='[', const char closeBracket=']', const bool debug=false)
{
    //values.clear();
	values.Empty();

    // We're parsing an ascii format, and so there may be line breaks within str for formatting reasons.
    // Also, std::from_chars() fails with leading white space. Just remove all ws.
    std::string str; str.reserve(strIn.size());
    size_t num = 0;
    for(char ch : strIn)
    {
        if(!std::isspace(ch))
            str.push_back(ch);
        if(ch == ',')
            num++;
    }
    values.Reserve(num+1);

    size_t i = str.find_first_of(openBracket);
    if(i == std::string::npos)
        i = 0; // Didn't find openBracket, start at 0.
    else 
        i++;   // Found openBracket, skip it.
    std::string delim(","); delim+=closeBracket; delim+=openBracket;
    while(i < str.size())
    {
        // advance 
        i = str.find_first_not_of(delim, i);
        if(i == std::string::npos)
            break;

        size_t j = i+1<str.size() ? str.find_first_of(delim, i+1) : str.size();
        if(j == std::string::npos) 
            j = str.size(); // Didn't find closeBracket, point 1 past last char.

        const size_t len = j-i;

        values.SetNum(values.Num()+1);
        if(!ParseNum(str.substr(i, len), values[values.Num()-1]))
        {
            if(debug)
                std::cout << "ParseNumArray() - std::from_chars() failed on '" << str.substr(i, len) 
                    << "' in context [" << std::max(((int)i)-64,0) << ", " << std::min((int)str.size(),((int)i)+128) << "]: " << std::endl
                    << "'" << str.substr(std::max(((int)i)-64,0), std::min((int)str.size(),((int)i)+128)) 
                    << "', i: " << i << " j: " << j << " len: " << len << " - delim: '" << delim << "'."
                    << std::endl;
            return false;
        }

        i = j+1;
    }
    if(debug)
    {
        std::cout << "ParseNumArray() - strIn: '" << strIn.substr(0, 128) << "'" << std::endl;
        for(int j=0; j < std::min((int)values.Num(),10); j++)
            std::cout << values[j] << ", ";
        std::cout << std::endl;
    }

    return true;
}

//! Returns \c true if \p strIn contains a array of valid numeric values.
bool
IsNumArray(const std::string &strIn, size_t &numValues, const char openBracket='[', const char closeBracket=']')
{
    numValues = 0;
    std::string str; str.reserve(128);
    for(char ch : strIn)
    {
        if(std::isspace(ch))
            continue;
        // [71500,25743,43157,43152,82894,...]
        // [[0,0,0],[-0.402781129,-0.911162317,0.0868939161],[-0.392075658,-0.91307807,0.112093881],...]
        if(ch == openBracket || ch == closeBracket || ch == ',')
        {
            if(!str.empty())
            {
                double num;
                if(!ParseNum(str, num)) // parse to double
                    return false;
                numValues++;
            }
            str.clear();
        }
        else if(ch == '"')
            return false;
        else
            str.push_back(ch);
    }
    return numValues != 0;
}

//! Parses numeric values of type \p T from \p strIn in array format demarked by 
//! \p openBracket and \p closeBracket, with vector values delinated by commas.  
//! <code>
//! TArray<int> values; 
//! ParseVector3Array("[[1, 2, 3], [4, 5, 6], [7, 8, 9]]", values);
//! <\code>
template <class T>
bool
ParseVector3Array(const std::string &strIn, TArray<T> &values, const char openBracket='[', const char closeBracket=']')
{
    values.clear();
    std::string substr; substr.reserve(1024);
	TArray<T> tmp; tmp.Reserve(3);

    size_t i = strIn.find(openBracket, 0);
    size_t iEnd = strIn.find_last_of(closeBracket, 0);
    if(i == std::string::npos || iEnd == std::string::npos)
        return false;
    while(i < iEnd)
    {
        i = strIn.find_first_of(openBracket, i+1);
        if(i == std::string::npos)
            break;
        size_t j = strIn.find_first_of(closeBracket, i+1);
        if(j == std::string::npos)
            break;

        substr = strIn.substr(i, j+1-i); // [i, j]
        if(!ParseNumArray(substr, tmp, openBracket, closeBracket) || tmp.Num()!=3)
            return false;

        size_t curr = values.size();
        values.SetNum(curr+tmp.Num());
        for(size_t k=0; k < tmp.Num(); k++)
            values[curr+k] = tmp[k];

        i = j;
    }
    return true;
}

bool
IsVector3(const std::string &str)
{ TArray<float> vec; if(ParseNumArray(str, vec) && vec.Num()==3) return true; return false; }

bool
ParsePrimitives(
	const std::string &str, 
	TMap<FString, std::string> &flattenedKvp)
{
	/*
	"primitives",

1	[
2		[
3			["type","Tetrahedron_run"],["startvertex",0,"nprimitives",8895]
-2		],
2		[
3			["type","Polygon_run"],["startvertex",35580,"nprimitives",162,"nvertices_rle",[3,162]]
-2		]
-1	]
	*/

    std::string l1str;
	{
		std::stringstream ss(str);
		if (!ReadMatching(ss, l1str, '[', ']', true)) // strip outer brackets 1
			exit(1);
	}

	// Get L2 string
	std::stringstream ss(l1str);
	while(ss.good())
	{
		ss >> std::ws; // discard white space
		if (!ss.good())
			break;
		char ch = ss.peek();
		std::string l2str;
		switch (ch)
		{
		case '[':
			ReadMatching(ss, l2str, '[', ']', true); // strip outer brackets 2, advances ss
			break;
		case ',':
			ss.get(ch); // consume char
			break;
		default:
			//exit(1);
			continue;
		}
		if (l2str.empty())
			continue;

		// Parse L3 arrays
		std::string prefix;
		std::stringstream l2ss(l2str);
		while (l2ss.good())
		{
			l2ss >> std::ws; // discard white space
			if (!l2ss.good())
				break;
			char l2ch = l2ss.peek();
			std::string l3str;
			switch (l2ch)
			{
			case '[':
				ReadMatching(l2ss, l3str, '[', ']', false); // keep brackets, advances l2ss
				break;
			case ',':
				l2ss.get(l2ch);
				break;
			default:
				//exit(1);
				continue;
			}
			if (l3str.empty())
				continue;

			TArray<std::string> values;
			if (ParseArray(l3str, values))
			{
				for (int i = 0; i < values.Num() - 1; i++)
				{
					const std::string key = ParseString(values[i]);
					const std::string value = ParseString(values[++i]);
					if (key == "type")
						prefix = value + ":";
					else
					{
						std::string k = prefix + key;
						flattenedKvp.Add(FString(k.c_str()), value);
					}
				}
			}
		}
	}
    return true;
}

bool
ParseTopology(
	const std::string &str, 
	TMap<FString, std::string> &flattenedKvp)
{
/*
        "topology",[
                "pointref",[
                        "indices",[71500,25743,43157,...,88117,89363,81072]
                ]
        ],
*/
    TMap<FString, std::string> level1kvp;
    if(!ParseKVPairs(str, level1kvp))
    {
        std::cout << "ParseTopology() - ParseKVPairs() failed at level 1 with str: '" << str << "'" << std::endl;
        exit(1);
    }
    for(const auto & [l1key, l1value] : level1kvp)
    {
        TMap<FString, std::string> level2kvp;
        if(!ParseKVPairs(l1value, level2kvp))
        {
            std::cout << "ParseTopology() - ParseKVPairs() failed at level 2 with str: '" << l1value  << "'" << std::endl;
            exit(1);
        }
        
        TMap<FString, std::string> level3kvp;
        if(!ParseKVPairs(l1value, level3kvp))
        {
            std::cout << "ParseTopology() - ParseKVPairs() failed at level 3 with str: '" << l1value  << "'" << std::endl;
            exit(1);
        }

		for (const auto& [l3key, l3value] : level3kvp)
		{
			std::string tmp = std::string(TCHAR_TO_UTF8(*l1key)) + "." + std::string(TCHAR_TO_UTF8(*l3key));
			flattenedKvp.Add(FString(tmp.c_str()), l3value);
		}
    }

    return true;
}

bool
ParseAttributes(
	const std::string &str, 
	TMap<FString, TMap<FString, std::string>> &categorizedKvp)
{
    // "vertexattributes",[...],
    // "pointattributes",[...],
    // "primitiveattributes",
    TMap<FString, std::string> attrCategories;
    ParseKVPairs(str, attrCategories);
	for(auto it = attrCategories.CreateConstIterator(); it; ++it)
    {
		const FString& category = it.Key();
		const std::string& catScope = it.Value();
        TMap<FString, std::string> &kvp = categorizedKvp.Add(category);

		TArray<std::string> attrNames;

        TArray<std::string> attrTuples;
        if(!ParseArray(catScope, attrTuples))
        {
            std::cout << "ParseAttributes() - failed to parse attr tuples." << std::endl;
            return false;
        }
        for(const std::string &attrStr : attrTuples)
        {
            TArray<std::string> attrBlocks;
            if(!ParseArray(attrStr, attrBlocks))
            {
                std::cout << "ParseAttributes() - failed to parse attr blocks from str: '" 
                    << (attrStr.size()>1024?attrStr.substr(0,1024):attrStr) << "'" << std::endl;
                break;
            }

            std::string attrName;
            for(const std::string &attrBlock : attrBlocks)
            {
                TMap<FString, std::string> attrVars;
                if(!ParseKVPairs(attrBlock, attrVars))
                {
                    std::cout << "ParseAttributes() - failed to parse attrVars from str: '" << attrStr << "'" << std::endl;
                    break;
                }
                // Attribute name should be in the first block
                if(attrName.empty())
                {
                    const auto nameIt=attrVars.Find("name");
                    if(nameIt == nullptr)
                    {
                        std::cout << "ParseAttributes() - failed to find attr name!" << std::endl;
                        break;
                    }
					attrName = *nameIt;
					attrNames.Add(attrName);
                }
                //std::string attrPath = "attributes." + category + "." + attrName + ".";
                std::string attrPath = attrName + ".";
                for(const auto & [k, v] : attrVars)
                {
                    if(k == "values" || k == "indices")
                    {
                        TMap<FString, std::string> valuesKvp;
                        if(!ParseKVPairs(v, valuesKvp))
                        {
                            std::cout << "ParseAttributes() - failed to parse values for attr: '" << std::string(TCHAR_TO_UTF8(*k))
                                << "' from str: '" << v << "'" << std::endl;
                            break;
                        }
                        for(const auto & [valuesK, valuesV] : valuesKvp)
                        {
                            if(valuesK == "arrays" && valuesV.substr(0,2) == "[[")
                            {
                                // Strip off extra brackets from "arrays" entries.
								std::string tmp = attrPath + std::string(TCHAR_TO_UTF8(*k)) + '.' + std::string(TCHAR_TO_UTF8(*valuesK));
                                kvp.Add(FString(tmp.c_str()), valuesV.substr(1, valuesV.size() - 2));
                            }
                            else
                            {
								std::string tmp = attrPath + std::string(TCHAR_TO_UTF8(*k)) + '.' + std::string(TCHAR_TO_UTF8(*valuesK));
                                kvp.Add(FString(tmp.c_str()), valuesV);
                            }
                        }
                    }
                    else
                    {
						std::string tmp = attrPath + std::string(TCHAR_TO_UTF8(*k));
                        kvp.Add(FString(tmp.c_str()), v);
                    }
                }
            } // end for attrBlocks
        } // end for attrTuples

        kvp.Add("entries", FormatStringArray(attrNames));
    } // end for categories
    return true;
}

bool
FlattenDictionaries(
    TMap<FString, std::string> &kvpIn,
    TMap<FString, std::string> &kvpOut)
{
    kvpOut.Empty();
	for(auto it=kvpIn.CreateIterator(); it; ++it)
    {
		const FString& key = it.Key();
		std::string &value = it.Value();
        if(key.IsEmpty() || value.empty())
            continue;

        if(value[0] == '{')
        {
            TMap<FString,std::string> kvp;
            if(!ParseKVPairs(value, kvp))
            {
                std::cout << "FlattenKVPairs() - ParseKVPairs returned false on key '" << std::string(TCHAR_TO_UTF8(*key)) << "' value: '" << value << "'" << std::endl;
                continue;
            }
			for (auto& [l2k, l2v] : kvp)
			{
				std::string tmp = std::string(TCHAR_TO_UTF8(*key)) + '.' + std::string(TCHAR_TO_UTF8(*l2k));
				kvpOut.Add(FString(tmp.c_str()), l2v);
			}
        }
        else
        {
            kvpOut.Add(key, value);
        }
    }
    kvpIn.Empty();
    return true;
}

void
ProcessInfo(
	TMap<FString, int32>& IntVars,
	const TMap<FString, std::string> &kvp)
{
    int count=0;
    auto it=kvp.Find("pointcount");
    if(it != nullptr)
    {
		const std::string &value = *it;
        if(ParseNum(value, count) && count > 0)
        {
			IntVars.Add("pointcount", count);
        }
    }
/*
    if(!count)
    {
        //it=kvp.find("pointcount");
        it=kvp.find("vertexcount");
        if(it != itEnd)
        {
            const std::string &value = it->second;
            if(ParseNum(value, count) && count > 0)
            {
                pdm->addParticles(count);
                kvp.erase(it);
            }
        }
    }
*/
/*
    for(it=kvp.begin(), itEnd=kvp.end(); it != itEnd; ++it)
    {
        const std::string &key = it->first;
        const std::string &value = it->second;

        FixedAttribute attr = pdm->addFixedAttribute(key.c_str(), ParticleAttributeType::INDEXEDSTR, 1);
        const int id = pdm->registerFixedIndexedStr(attr, value.c_str());
        pdm->setFixed(attr, &id);
    }
*/
}

/*
* IntVars: 
*	{ ["Tetrahedron_run:startvertex",0], ["Tetrahedron_run:nprimitives",8895],
*	  ["Polygon_run:startvertex",35580], ["Polygon_run:nprimitives", 162] }
*/
void
ProcessPrimitivesAndTopology(
	TMap<FString, int32>& IntVars,
	TMap<FString, TArray<int32>>& IntVectorVars,
	const TMap<FString, std::string> &prim, 
	const TMap<FString, std::string> &topo)
{
	// primitives:
	std::string type;
	for(auto it=prim.CreateConstIterator(); it; ++it)
    {
        const FString key=it.Key();
        const std::string value=it.Value();

	    int vi;
		if (ParseNum(value, vi))
		{
			std::string tmp = std::string(TCHAR_TO_UTF8(*key));
			IntVars.Add(FString(tmp.c_str()), vi);
		}
	}

    // topo:
    // pointref.indices = int array
	for(auto it=topo.CreateConstIterator(); it; ++it)
    {
        const FString key=it.Key();
        const std::string value=it.Value();

        TArray<int> iv;
        if(ParseNumArray(value, iv))
        {
			IntVectorVars.Add(key, iv);
		}
        else
        {
			//UE_LOGF(LogChaosFlesh, Display, "Unsupported topology attribute '%ls' = '%ls'.", 
			//	*key, 
			//	(value.size() > 128 ? value.substr(0, 64) + "<...>" + value.substr(value.size() - 64) : value).c_str());
			std::cerr << "Unsupported topology attribute '" << std::string(TCHAR_TO_UTF8(*key))
                << "' = '" << (value.size()>128?value.substr(0,64)+"<...>"+value.substr(value.size()-64):value) 
                << "'." << std::endl;
        }
    }
}

void
ProcessAttributes(
	TMap<FString, int32>& IntVars,
	TMap<FString, TArray<int32>>& IntVectorVars,
	TMap<FString, TArray<float>>& FloatVectorVars,
	TMap<FString, TPair<TArray<std::string>, TArray<int32>>>& IndexedStringVars,
	const TMap<FString, TMap<FString, std::string>> &catToAttrs)
{
	for(auto cIt=catToAttrs.CreateConstIterator(); cIt; ++cIt)
    {
        const FString &category = cIt.Key(); // vertexattributes, pointattributes, primitiveattributes
        const auto &attrs = cIt.Value();

        auto it = attrs.Find("entries");
        if(it == nullptr)
            continue;
        const std::string &attrNamesStr = *it;
        TArray<std::string> attrNames;
        ParseArray(attrNamesStr, attrNames, '[', ']', true);

        //for(auto nIt=attrNames.begin(), nItEnd=attrNames.end(); nIt != nItEnd; ++nIt)
		for(const std::string &name : attrNames)
        {
            //const std::string &name = *nIt;

            it = attrs.Find(FString(std::string(name+".type").c_str()));
            std::string type = it != nullptr ? *it : "";
            if(type == "numeric")
            {
                it = attrs.Find(FString(std::string(name+".values.size").c_str()));
                std::string sizeStr = it != nullptr ? *it : "";
                int size=0;
                if(!ParseNum(sizeStr, size))
                {
                    std::cout << "ProcessAttributes() - failed to parse size string '" << sizeStr 
                        << "' for attribute: '" << std::string(TCHAR_TO_UTF8(*category)) << "." << name << "'." << std::endl;
                    continue;
                }
                std::string dataAttrName = size==1 ? name+".values.arrays" : name+".values.tuples";
                it = attrs.Find(FString(dataAttrName.c_str()));
                const std::string &dataStr = it != nullptr ? *it : "";

                it = attrs.Find(FString(std::string(name+".values.storage").c_str()));
                std::string storage = it != nullptr ? *it : "";
                if(storage == "int32")
                {
                    TArray<int> values;
                    if(!ParseNumArray(dataStr, values))
                    {
                        std::cout << "ProcessAttributes() - failed to parse int array attribute '" << dataAttrName 
                            << "' from string: '" << (dataStr.size()>128 ? dataStr.substr(0,64)+"<...>"+dataStr.substr(dataStr.size()-64) : dataStr)
                            << "'." << std::endl;
                        continue;
                    }
					IntVectorVars.Add(FString(name.c_str()), values);
				}
                else if(storage == "fpreal32")
                {
                    TArray<float> values;
                    if(!ParseNumArray(dataStr, values))
                    {
                        std::cout << "ProcessAttributes() - failed to parse float array attribute '" << dataAttrName 
                            << "' from string: '" << (dataStr.size()>128 ? dataStr.substr(0,64)+"<...>"+dataStr.substr(dataStr.size()-64) : dataStr)
                            << "'." << std::endl;
                        continue;
                    }
					FloatVectorVars.Add(FString(name.c_str()), values);
				}
                else
                {
                    std::cout << "Unsupported storage format '" << storage << "' for attribute: '" << dataAttrName 
                        << "'." << std::endl;
                    continue;
                }
            }
            else if(type == "string")
            {
                it = attrs.Find(FString(std::string(name+".strings").c_str()));
                const std::string &stringsStr = it != nullptr ? *it : "";
                TArray<std::string> strings;
                if(!ParseArray(stringsStr, strings))
                {
                    std::cout << "ProcessAttributes() - failed to parse string array attribute: '" << name 
                        << ".strings'" << std::endl;
                    continue;
                }
                it = attrs.Find(FString(std::string(name+".indices.arrays").c_str()));
                const std::string &indicesStr = it != nullptr ? *it : "";
                TArray<int> indices;
                if(!ParseNumArray(indicesStr, indices))
                {
                    std::cout << "ProcessAttributes() - failed to parse int array attribute '" << name 
                        << ".indices.arrays' from string: '" 
                        << (indicesStr.size()>128 ? indicesStr.substr(0,64)+"<...>"+indicesStr.substr(indicesStr.size()-64) : indicesStr)
                        << "'." << std::endl;
                    continue;
                }
				IndexedStringVars.Add(
					FString(name.c_str()), 
					TPair<TArray<std::string>, TArray<int32>>(strings, indices));
			}
            else
            {
                std::cout << "ProcessAttributes() - unsupported attr type: '" << type << "'." << std::endl;
                continue;
            }
        }// end for attrNames
    }// and for catToAttrs
}

bool
ReadGEO_v19(
	const std::string &filename,
	TMap<FString, int32>& IntVars,
	TMap<FString, TArray<int32>>& IntVectorVars,
	TMap<FString, TArray<float>>& FloatVectorVars,
	TMap<FString, TPair<TArray<std::string>, TArray<int32>>>& IndexedStringVars,
	std::ostream *errorStream);

std::unique_ptr<std::istream> 
unzip(const std::string& filename, std::unique_ptr<IFileHandle>& infile, std::ios::openmode mode=std::ios::in)
{
	std::unique_ptr<std::istream> stream;
	FPlatformFileManager& FileManager = FPlatformFileManager::Get();
	IPlatformFile& PlatformFile = FileManager.GetPlatformFile();
	infile.reset(PlatformFile.OpenRead(*FString(filename.c_str()), false));

    FString errmsg;
#ifdef USE_ZLIB
	if(GZIP_FILE_HEADER header; header.Read(infile.get(), errmsg))
    {
        infile->Seek(0);
        stream.reset(new ZIP_FILE_ISTREAM(infile.get(), false));
    }
    else
#endif // USE_ZLIB
	{
        infile->Seek(0);
        stream.reset(new IFileStream(infile.get()));
    }
    if(stream)
        stream->imbue(std::locale::classic());
    return stream;
}

bool
ReadGEO(
	const std::string &filename,
	TMap<FString, int32>& IntVars,
	TMap<FString, TArray<int32>>& IntVectorVars,
	TMap<FString, TArray<float>>& FloatVectorVars,
	TMap<FString, TPair<TArray<std::string>, TArray<int32>>>& IndexedStringVars,
	std::ostream *errorStream)
{
	std::unique_ptr<IFileHandle> infile = nullptr;
    std::unique_ptr<std::istream> input(unzip(filename, infile));
    if(!*input)
    {
        if(errorStream) *errorStream<<"ReadGEO(): Can't open GEO file: '"<<filename<<"'"<<endl;
        return false;
    }
    std::istream &inref = *input;

    // Determine file version and forward to the proper read routine.

    inref >> std::ws;
    std::string buffer; buffer.reserve(1024);
    inref >> buffer;
    if(buffer == "[") // [ "fileversion","19.0.518",
    {
        inref >> std::ws;
        inref >> buffer;
        if(buffer.substr(0,13) == "\"fileversion\"")
        {
            input.reset(); // close file
            if(buffer.substr(15, 3) == "19.") // match major version
				return ReadGEO_v19(filename, IntVars, IntVectorVars, FloatVectorVars, IndexedStringVars, errorStream);
            else
            {
                if(errorStream) 
                    (*errorStream) << "Warning: using v19 GEO parser for file: '" << filename << "'." << std::endl;
				return ReadGEO_v19(filename, IntVars, IntVectorVars, FloatVectorVars, IndexedStringVars, errorStream);
			}
        }
    }
    else if(buffer == "PGEOMETRY") // PGEOMETRY V5
    {
#ifdef READ_GEO_LEGACY
        inref >> std::ws;
        inref >> buffer;
        input.reset(); // close file
        if(buffer == "V5")
            return readGEO_Legacy(filename, headersOnly, errorStream);
        else
        {
            if(errorStream) 
                (*errorStream) << "Warning: using legacy V5 GEO parser for file: '" << filename << "'." << std::endl;
            return readGEO_Legacy(filename, headersOnly, errorStream);
        }
#else
        return false;
#endif
    }

	infile.reset();
    input.reset(); // close file

#ifdef READ_GEO_LEGACY
	if(errorStream)
        (*errorStream) << "Warning: Failed to determine GEO file version; using legacy V5 GEO parser for file: '" << filename << "'." << std::endl;
    return readGEO_Legacy(filename, headersOnly, errorStream);
#else
    return false;
#endif
}

bool
ReadGEO_v19(
	const std::string& filename,
	TMap<FString, int32>& IntVars,
	TMap<FString, TArray<int32>>& IntVectorVars,
	TMap<FString, TArray<float>>& FloatVectorVars,
	TMap<FString, TPair<TArray<std::string>, TArray<int32>>>& IndexedStringVars,
	std::ostream* errorStream)
{
	std::unique_ptr<IFileHandle> infile = nullptr;
	std::unique_ptr<std::istream> input(unzip(filename, infile));
	if(!*input)
    {
        if(errorStream) *errorStream<<"Partio: Can't open GEO file: '"<<filename<<"'"<<endl;
        return false;
    }
    std::istream &inref = *input;

	TMap<FString, std::string> kvp;
	if(!ParseGeoKVPairs(inref, kvp, filename.c_str(), errorStream))
    {
        if(errorStream) *errorStream<<"Partio: Failed to parse GEO file: '" << filename << "'" << std::endl;
        return false;
    }

	TMap<FString, TMap<FString, std::string>> attributes;
	const auto *attributesIt = kvp.Find("attributes");
	if(attributesIt != nullptr)
    {
        if(!ParseAttributes(*attributesIt, attributes))
            return false;
		kvp.Remove("attributes");
	}

    TMap<FString, std::string> topology;
	const auto topologyIt = kvp.Find("topology");
	if(topologyIt != nullptr)
    {
        if(!ParseTopology(*topologyIt, topology))
            return false;
		kvp.Remove("topology");
    }

    TMap<FString, std::string> primitives;
    const auto primitivesIt = kvp.Find("primitives");
    if(primitivesIt != nullptr)
    {
        if(!ParsePrimitives(*primitivesIt, primitives))
            return false;
		kvp.Remove("primitives");
	}

    TMap<FString, std::string> info;
    if(!FlattenDictionaries(kvp, info))
    {
        if(errorStream) *errorStream<<"ReadGEO_v19(): Failed to flatten GEO file: '" << filename << "'" << std::endl;
        return false;
    }
/*
    std::cout << "readGEO() - flattened kvp dump:" << std::endl;
    DebugPrint(info);
    std::cout << std::endl;
    std::cout << "readGEO() - primitives:" << std::endl;
    DebugPrint(primitives);
    std::cout << std::endl;
    std::cout << "readGEO() - topology:" << std::endl;
    DebugPrint(topology);
    std::cout << std::endl;
    std::cout << "readGEO() - attributes:" << std::endl;
    DebugPrint(attributes, "    attributes");
    std::cout << std::endl;
*/
	ProcessInfo(IntVars, info);
	ProcessPrimitivesAndTopology(IntVars, IntVectorVars, primitives, topology);
	ProcessAttributes(IntVars, IntVectorVars, FloatVectorVars, IndexedStringVars, attributes);

	return true;
}


#ifdef READ_GEO_LEGACY
ParticlesDataMutable* readGEO_Legacy(const char* filename,const bool headersOnly,std::ostream* errorStream)
{
    unique_ptr<istream> input(io::unzip(filename));
    if(!*input){
        if(errorStream) *errorStream<<"Partio: Can't open particle data file: "<<filename<<endl;
        return 0;
    }
    int NPoints=0, NPointAttrib=0, NIndices;

    ParticlesDataMutable* simple=0;
    if(headersOnly) simple=new ParticleHeaders;
    else simple=create();

    // read NPoints and NPointAttrib
    string word;
    while(input->good()){
        *input>>word;
        if(word=="NPoints"||word=="pointcount") *input>>NPoints;
        else if(word=="vertexcount") *input>>NIndices;
        else if(word=="NPointAttrib")
        {
            *input>>NPointAttrib;
            break;
        }
    }
    // skip until PointAttrib
    while(input->good()){
        *input>>word;
        if(word=="PointAttrib") break;
    }
    // read attribute descriptions
    int attrInfoRead = 0;
    
    ParticleAttribute positionAttr=simple->addAttribute("position",VECTOR,3);
    ParticleAccessor positionAccessor(positionAttr);

    vector<ParticleAttribute> attrs;
    vector<ParticleAccessor> accessors;
    while (input->good() && attrInfoRead < NPointAttrib) {
        string attrName, attrType;
        int nvals = 0;
        *input >> attrName >> nvals >> attrType;
        if(attrType=="index"){
            if(errorStream) *errorStream<<"Partio: attr '"<<attrName<<"' of type index (string) found, treating as integer"<<endl;
            int nIndices=0;
            *input>>nIndices;
            ParticleAttribute attribute=simple->addAttribute(attrName.c_str(),INDEXEDSTR,1);
            attrs.push_back(attribute);
            for(int j=0;j<nIndices;j++){
                string indexName;
                // *input>>indexName;
                indexName=scanString(*input);
                if (!headersOnly) {
                    int id=simple->registerIndexedStr(attribute,indexName.c_str());
                    if(id != j){
                        if(errorStream) *errorStream<<"Partio: error on read, expected registerIndexStr to return index "<<j<<" but got "<<id<<" for string "<<indexName<<endl;
                    }
                }
            }
            accessors.push_back(ParticleAccessor(attrs.back()));
            attrInfoRead++;
            
        }else{
            for (int i=0;i<nvals;i++) {
                float defval;
                *input>>defval;
            }
            ParticleAttributeType type;
            // TODO: fix for other attribute types
            if(attrType=="float") type=FLOAT;
            else if(attrType=="vector") type=VECTOR;
            else if(attrType=="int") type=INT;
            else{
                if(errorStream) *errorStream<<"Partio: unknown attribute "<<attrType<<" type... aborting"<<endl;
                type=NONE;
            }
            attrs.push_back(simple->addAttribute(attrName.c_str(),type,nvals));
            accessors.push_back(ParticleAccessor(attrs.back()));
            attrInfoRead++;
        }
    }

    simple->addParticles(NPoints);

    ParticlesDataMutable::iterator iterator=simple->begin();
    iterator.addAccessor(positionAccessor);
    for(size_t i=0;i<accessors.size();i++) iterator.addAccessor(accessors[i]);

    if(headersOnly) return simple; // escape before we try to touch data
    
    
    float fval;
    // TODO: fix
    for(ParticlesDataMutable::iterator end=simple->end();iterator!=end  && input->good();++iterator){
        float* posInternal=positionAccessor.raw<float>(iterator); 
        for(int i=0;i<3;i++) *input>>posInternal[i];
        *input>>fval;
        //cout<<"saw "<<posInternal[0]<<" "<<posInternal[1]<<" "<<posInternal[2]<<endl;
        
        // skip open paren
        char paren = 0;
        *input>> paren;
        if (paren != '(') break;
        
        // read additional attribute values
        for (unsigned int i=0;i<attrs.size();i++){
            switch(attrs[i].type){
                case NONE: assert(false);break;
                case FLOAT: readGeoAttr<FLOAT>(*input,attrs[i],accessors[i],iterator);break;
                case VECTOR: readGeoAttr<VECTOR>(*input,attrs[i],accessors[i],iterator);break;
                case INT: readGeoAttr<INT>(*input,attrs[i],accessors[i],iterator);break;
                case INDEXEDSTR: readGeoAttr<INDEXEDSTR>(*input,attrs[i],accessors[i],iterator);break;
            }
        }
        // skip closing parenthes
        *input >> paren;
        if (paren != ')') break;
    }
    return simple;
}
#endif //READ_GEO_LEGACY

} // namespace ChaosFlesh

#endif // WITH_EDITOR
