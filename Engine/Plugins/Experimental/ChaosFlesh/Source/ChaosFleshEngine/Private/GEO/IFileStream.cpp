// Copyright Epic Games, Inc. All Rights Reserved.
#include "GEO/IFileStream.h"
#include "GenericPlatform/GenericPlatformFile.h"

#if WITH_EDITOR

IFileStreamBuf::IFileStreamBuf(IFileHandle* istream)
    : istream(istream)
    , total_read(0)
    , valid(true)
{
    setg((char*)in, (char*)in, (char*)in);
    setp(0, 0);
}

IFileStreamBuf::~IFileStreamBuf()
{}

int
IFileStreamBuf::process()
{
    if (!valid) return -1;
    int count = FGenericPlatformMath::Min<int>(buffer_size-4, istream->Size()-total_read);
    istream->Read((uint8*)(out + 4), count);
    total_read += count;
    return count;
}

int
IFileStreamBuf::underflow()
{
    if (gptr() && (gptr() < egptr())) return traits_type::to_int_type(*gptr()); // if we already have data just use it
    int put_back_count = (int)(gptr() - eback());
    if (put_back_count > 4) put_back_count = 4;
    std::memmove(out + (4 - put_back_count), gptr() - put_back_count, put_back_count);
    int num = process();
    setg((char*)(out + 4 - put_back_count), (char*)(out + 4), (char*)(out + 4 + num));
    if (num <= 0) return EOF;
    return traits_type::to_int_type(*gptr());
}

int
IFileStreamBuf::overflow(int c)
{
    return EOF;
}

IFileStream::IFileStream(IFileHandle* istream)
	: std::istream(&buf)
	, buf(istream)
{}

IFileStream::~IFileStream()
{}

#endif // WITH_EDITOR