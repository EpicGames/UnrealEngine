// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR

#include <cstdio>
#include <istream>
#include <streambuf>

class IFileHandle;

/**
 * Stream buffer that uses \c IFileHandle for file access.
 */
class IFileStreamBuf : public std::streambuf
{
	static const unsigned int buffer_size = 512;
	IFileHandle* istream;
	unsigned char in[buffer_size];
	unsigned char out[buffer_size];
	int total_read;
	bool valid;
public:
	IFileStreamBuf(IFileHandle* istream);
	virtual ~IFileStreamBuf();

	int process();
	virtual int underflow();
	virtual int overflow(int c = EOF);
};

/**
 * Input stream that uses \c IFileHandle for file access.
 */
class IFileStream : public std::istream
{
	IFileStreamBuf buf;
public:
	IFileStream(IFileHandle* istream);
	virtual ~IFileStream();
};

#endif
