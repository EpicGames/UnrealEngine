// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Adapters/CacheAdapter.h"
#include "Chaos/Deformable/ChaosDeformableSolver.h"
#include "Chaos/Deformable/ChaosDeformableSolverProxy.h"
#include "Chaos/Deformable/ChaosDeformableSolverTypes.h"
#include "Chaos/Range.h"
#if USE_USD_SDK && DO_USD_CACHING

#include "UsdWrappers/UsdStage.h"
#include "USDIncludesStart.h"
	#include "pxr/base/gf/vec3f.h"
	#include "pxr/base/vt/array.h"
#include "USDIncludesEnd.h"

#endif // USE_USD_SDK && DO_USD_CACHING

DECLARE_LOG_CATEGORY_EXTERN(LogChaosFleshCache, Verbose, All);

namespace Chaos::Softs
{
	class FDeformableSolver;
	class FPBDEvolution;
	class FSolverParticles;
}
class UDeformablePhysicsComponent;

namespace Chaos
{
	struct FFleshCacheAdapterCVarParams
	{
		// USD caching options
		bool bWriteBinary = true;
		bool bNoClobber = true;
		int32 SaveFrequency = 10;
	};

	/**
	 * Skeletal mesh cache adapter to be able to cache cloth simulation datas through the chaos cache system
	 */
	class FFleshCacheAdapter : public FComponentCacheAdapter
	{
	public:
		typedef Chaos::Softs::FDeformableSolver FDeformableSolver;
		typedef Chaos::Softs::FPBDEvolution FEvolution;
		typedef Chaos::Softs::FSolverParticles FParticles;

#if USE_USD_SDK && DO_USD_CACHING
		static CHAOSFLESHENGINE_API FString GetUSDCacheDirectory(const FObservedComponent& InObserved);
		static CHAOSFLESHENGINE_API FString GetUSDCacheFileName(const UDeformablePhysicsComponent* FleshComp);
		static CHAOSFLESHENGINE_API FString GetUSDCacheFilePathRO(const FObservedComponent& InObserved, const UDeformablePhysicsComponent* FleshComp);
		static CHAOSFLESHENGINE_API FString GetUSDCacheFilePathRW(const FObservedComponent& InObserved, const UDeformablePhysicsComponent* FleshComp);
#endif // USE_USD_SDK && DO_USD_CACHING

		virtual ~FFleshCacheAdapter();

		// ~Begin FComponentCacheAdapter interface
		virtual SupportType					SupportsComponentClass(UClass* InComponentClass) const override;
		virtual UClass*						GetDesiredClass() const override;
		virtual uint8						GetPriority() const override;
		virtual FGuid						GetGuid() const override;
		virtual bool						ValidForPlayback(UPrimitiveComponent* InComponent, UChaosCache* InCache) const override;
		virtual bool						SupportsPlayWhileRecording() const override { return true; };
		virtual Chaos::FPhysicsSolver*		GetComponentSolver(UPrimitiveComponent* InComponent) const override;
		virtual Chaos::FPhysicsSolverEvents* BuildEventsSolver(UPrimitiveComponent* InComponent) const override;
		virtual void						SetRestState(UPrimitiveComponent* InComponent, UChaosCache* InCache, const FTransform& InRootTransform, Chaos::FReal InTime) const override;
		virtual bool						InitializeForRecord(UPrimitiveComponent* InComponent, FObservedComponent& InObserved) override;
		virtual bool						InitializeForPlayback(UPrimitiveComponent* InComponent, FObservedComponent& InObserved, float InTime) override;
		virtual void						InitializeForLoad(UPrimitiveComponent* InComponent, FObservedComponent& InObserved) override;
		virtual void						Record_PostSolve(UPrimitiveComponent* InComp, const FTransform& InRootTransform, FPendingFrameWrite& OutFrame, Chaos::FReal InTime) const override;
		virtual void						Playback_PreSolve(UPrimitiveComponent* InComponent,
												UChaosCache*										InCache,
												Chaos::FReal                                       InTime,
												FPlaybackTickRecord&								TickRecord,
												TArray<TPBDRigidParticleHandle<Chaos::FReal, 3>*>& OutUpdatedRigids) const override;
		virtual void						Finalize() override;
		// ~End FComponentCacheAdapter interface

		/** Get the component deformable solver */
		static FDeformableSolver* GetDeformableSolver(UPrimitiveComponent* InComponent);

		/** Get the particle indices range for the component in the evolution particles list on the physics thread */
		static Chaos::FRange GetParticleRange(UPrimitiveComponent* InComponent, const int32 NumParticles);
		
		bool SupportsRestartSimulation() const override { return true; };

	private:
		/** Load the cache for a given component at a specific time */
		void LoadCacheAtTime(UPrimitiveComponent* PrimitiveComponent, Chaos::FReal TargetTime, const int32 NumParticles, const bool bNeedsRange,
				const TFunction<void(const int32, const int32, const FVector3f&, const FVector3f&, const float MuscleActivation)>& LoadFunction) const;
		
#if USE_USD_SDK && DO_USD_CACHING

		bool bReadOnly = false;
		bool bUseMonolith = true; // move to cvarparams when value clips is an option

		/** Structure holding the per primitive stage datas */
		struct FPrimitiveStage
		{
			/**
			 * Cache files are stored in a "SimCache" directory at the root of the project.
			 * File name is derived from flesh component name.
			 */
			FString FilePath;
			FString PrimPath;

			/** Min time in the usd stage */
			double MinTime = TNumericLimits<double>::Max();

			/** Max time in the usd stage */
			double MaxTime = TNumericLimits<double>::Lowest();

			/** USD Monolith stage used to load/record datas */
			UE::FUsdStage MonolithStage;
		};

		/** Per primitive stages datas */
		mutable TMap<UPrimitiveComponent*,FPrimitiveStage> PrimitiveStages;

#else // USE_USD_SDK && DO_USD_CACHING

		inline static const FName VelocityXName = TEXT("VelocityX");
		inline static const FName VelocityYName = TEXT("VelocityY");
		inline static const FName VelocityZName = TEXT("VelocityZ");
		inline static const FName PositionXName = TEXT("PositionX");
		inline static const FName PositionYName = TEXT("PositionY");
		inline static const FName PositionZName = TEXT("PositionZ");

#endif // USE_USD_SDK && DO_USD_CACHING
	};
}    // namespace Chaos
