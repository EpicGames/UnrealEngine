// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosFlesh/ChaosDeformableCollisionsComponent.h"
#include "ChaosFlesh/ChaosDeformableSolverActor.h"
#include "CoreMinimal.h"
#include "DeformableInterface.h"
#include "GameFramework/Actor.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"

#include "ChaosDeformableCollisionsActor.generated.h"

class AStaticMeshActor;


UCLASS()
class CHAOSFLESHENGINE_API ADeformableCollisionsActor : public AActor, public IDeformableInterface
{
	GENERATED_UCLASS_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "Physics")
	void EnableSimulation(ADeformableSolverActor* Actor);

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Physics")
	TObjectPtr<UDeformableCollisionsComponent> DeformableCollisionsComponent;
	UDeformableCollisionsComponent* GetCollisionsComponent() const { return DeformableCollisionsComponent; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics", meta = (DisplayPriority = 1))
	TObjectPtr<ADeformableSolverActor> PrimarySolver;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics", meta = (DisplayPriority = 2))
	TArray<TObjectPtr<AStaticMeshActor>> StaticCollisions;

#if WITH_EDITOR
	TArray < TObjectPtr<AStaticMeshActor> > AddedBodies;
	TArray < TObjectPtr<AStaticMeshActor> > RemovedBodies;
	TArray < TObjectPtr<AStaticMeshActor> > PreEditChangeCollisionBodies;
	ADeformableSolverActor* PreEditChangePrimarySolver = nullptr;
	virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
#endif
};
