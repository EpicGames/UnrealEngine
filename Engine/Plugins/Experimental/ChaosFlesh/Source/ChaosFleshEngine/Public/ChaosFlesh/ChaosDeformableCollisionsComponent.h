// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Deformable/ChaosDeformableSolverProxy.h"
#include "Chaos/Deformable/ChaosDeformableCollisionsProxy.h"
#include "ChaosFlesh/ChaosDeformablePhysicsComponent.h"
#include "Components/SceneComponent.h"
#include "UObject/ObjectMacros.h"

#include "ChaosDeformableCollisionsComponent.generated.h"

class UStaticMeshComponent;


/**
*	UDeformableCollisionsComponent
*/
UCLASS(meta = (BlueprintSpawnableComponent))
class CHAOSFLESHENGINE_API UDeformableCollisionsComponent : public UDeformablePhysicsComponent
{
	GENERATED_UCLASS_BODY()

public:
	typedef Chaos::Softs::FDataMapValue FDataMapValue;
	typedef Chaos::Softs::FThreadingProxy FThreadingProxy;
	typedef Chaos::Softs::FCollisionManagerProxy FCollisionThreadingProxy;

	~UDeformableCollisionsComponent() {}

	/** Simulation Interface*/
	virtual FThreadingProxy* NewProxy() override;
	virtual FDataMapValue NewDeformableData() override;

	UFUNCTION(BlueprintCallable, Category = "Physics")
	void AddStaticMeshComponent(UStaticMeshComponent* StaticMeshComponent);
	
	UFUNCTION(BlueprintCallable, Category = "Physics")
	void RemoveStaticMeshComponent(UStaticMeshComponent* StaticMeshComponent);

	UPROPERTY(BlueprintReadOnly, Category = "Physics")
	TArray<TObjectPtr<UStaticMeshComponent>>  CollisionBodies;

protected:
	TArray<UStaticMeshComponent*> RemovedBodies;
	TArray<UStaticMeshComponent*> AddedBodies;

	TMap<TObjectPtr<UObject>, FThreadingProxy*> CollisionsMap;
	TMap<TObjectPtr<UStaticMeshComponent>, TArray<FTransform>> CollisionBodiesPrevXf;
};
