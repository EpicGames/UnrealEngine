// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosFlesh/ChaosDeformableConstraintsComponent.h"
#include "ChaosFlesh/ChaosDeformableSolverActor.h"
#include "CoreMinimal.h"
#include "DeformableInterface.h"
#include "GameFramework/Actor.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"

#include "ChaosDeformableConstraintsActor.generated.h"

class AFleshActor;

UCLASS()
class CHAOSFLESHENGINE_API ADeformableConstraintsActor : public AActor, public IDeformableInterface
{
	GENERATED_UCLASS_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "Physics")
	void EnableSimulation(ADeformableSolverActor* Actor);

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Physics")
	TObjectPtr<UDeformableConstraintsComponent> DeformableConstraintsComponent;
	UDeformableConstraintsComponent* GetConstraintsComponent() const { return DeformableConstraintsComponent; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics", meta = (DisplayPriority = 1))
	TObjectPtr<ADeformableSolverActor> PrimarySolver;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics", meta = (DisplayPriority = 2))
	TArray<TObjectPtr<AFleshActor>> SourceBodies;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics", meta = (DisplayPriority = 2))
	TArray<TObjectPtr<AFleshActor>> TargetBodies;


#if WITH_EDITOR
	TArray < TObjectPtr<AFleshActor> > AddedSourceBodies;
	TArray < TObjectPtr<AFleshActor> > RemovedSourceBodies;

	TArray < TObjectPtr<AFleshActor> > AddedTargetBodies;
	TArray < TObjectPtr<AFleshActor> > RemovedTargetBodies;

	TArray < TObjectPtr<AFleshActor> > PreEditChangeSourceBodies;
	TArray < TObjectPtr<AFleshActor> > PreEditChangeTargetBodies;

	ADeformableSolverActor* PreEditChangePrimarySolver = nullptr;
	virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
#endif
};
