// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Chaos/Deformable/ChaosDeformableSolverProxy.h"
#include "Chaos/Deformable/ChaosDeformableConstraintsProxy.h"
#include "ChaosFlesh/ChaosDeformablePhysicsComponent.h"
#include "UObject/ObjectMacros.h"

#include "ChaosDeformableConstraintsComponent.generated.h"

class UFleshComponent;


//ENUM()
enum  EDeformableConstraintType : uint8
{
	Kinematic,
	LinearSpring,
	MAX
};


USTRUCT(BlueprintType)
struct FDeformableConstraintParameters
{
	GENERATED_BODY()

	FDeformableConstraintParameters(float InStiffness = 100000.f, float InDamping = 1.f,
		EDeformableConstraintType InType = EDeformableConstraintType::Kinematic)
		: Type(InType)
		, Stiffness(InStiffness)
		, Damping(InDamping)
	{}
	
	//UPROPERTY(EditAnywhere)
	EDeformableConstraintType Type;

	UPROPERTY(EditAnywhere, Category = "Constraint")
	float Stiffness = 100000.f;
	
	UPROPERTY(EditAnywhere, Category = "Constraint")
	float Damping = 1.f;

	Chaos::Softs::FDeformableConstraintParameters ToChaos() 
	{ 
		return Chaos::Softs::FDeformableConstraintParameters(Stiffness,Damping,
			(Chaos::Softs::EDeformableConstraintType)Type);
	}

};

USTRUCT(BlueprintType)
struct FConstraintObject
{
	GENERATED_BODY()

	FConstraintObject(
		TObjectPtr<UFleshComponent> InSource = nullptr,
		TObjectPtr<UFleshComponent> InTarget = nullptr,
		FDeformableConstraintParameters InParameters = FDeformableConstraintParameters())
		: Source(InSource)
		, Target(InTarget)
		, Parameters(InParameters)
	{}

	UPROPERTY(EditAnywhere, Category = "Constraint")
	TObjectPtr<UFleshComponent> Source;

	UPROPERTY(EditAnywhere, Category = "Constraint")
	TObjectPtr<UFleshComponent> Target;

	UPROPERTY(EditAnywhere, Category = "Constraint")
	FDeformableConstraintParameters Parameters;

	Chaos::Softs::FConstraintObjectKey ToChaos()
	{
		return Chaos::Softs::FConstraintObjectKey(
			TObjectPtr<UObject>((UObject*)Source.Get()),
			TObjectPtr<UObject>((UObject*)Target.Get()),
			Chaos::Softs::EDeformableConstraintType(Parameters.Type));
	}

	bool operator==(const FConstraintObject& Other) const
	{
		return Other.Source==Source && Other.Target == Target;
	}
};

/**
*	UDeformableConstraintsComponent
*/
UCLASS(meta = (BlueprintSpawnableComponent))
class CHAOSFLESHENGINE_API UDeformableConstraintsComponent : public UDeformablePhysicsComponent
{
	GENERATED_UCLASS_BODY()

public:
	typedef Chaos::Softs::FDataMapValue FDataMapValue;
	typedef Chaos::Softs::FThreadingProxy FThreadingProxy;
	typedef Chaos::Softs::FConstraintManagerProxy FConstraintThreadingProxy;
	typedef Chaos::Softs::FConstraintObjectKey FConstraintObjectKey;

	~UDeformableConstraintsComponent() {}

	/** Simulation Interface*/
	virtual FThreadingProxy* NewProxy() override;
	virtual FDataMapValue NewDeformableData() override;

	UFUNCTION(BlueprintCallable, Category = "Physics")
	void AddConstrainedBodies(
		UFleshComponent* SourceComponent,
		UFleshComponent* TargetComponent,
		FDeformableConstraintParameters InParameters);
	
	UFUNCTION(BlueprintCallable, Category = "Physics")
	void RemoveConstrainedBodies(
		UFleshComponent* SourceComponent,
		UFleshComponent* TargetComponent,
		FDeformableConstraintParameters InParameters);

	UPROPERTY(BlueprintReadOnly, Category = "Physics")
	TArray<FConstraintObject>  Constraints;

protected:
	bool IsValid(const FConstraintObject&) const;

	TArray<FConstraintObject> RemovedConstraints;
	TArray<FConstraintObject> AddedConstraints;

	TMap<FConstraintObjectKey, FThreadingProxy*> ConstraintsMap;
};
