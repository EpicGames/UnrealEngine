// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RenderResource.h"
#include "RawIndexBuffer.h"

namespace UE::ChaosDeformable
{
	/**
	* Floating point GPU buffer.
	*/
	class FFloatArrayBufferWithSRV : public FVertexBufferWithSRV
	{
	public:
		void SetBufferName(const FString& InBufferName) { BufferName = InBufferName; }

		void Init(const float* InArray, const int32 Num);
		void Init(const TArray<float>& InArray) { Array = InArray; NumValues = Array.Num(); }
		void Init(TArray<float>&& InArray) { Array = MoveTemp(InArray); NumValues = Array.Num(); }

		int32 GetNumValues() const { return NumValues; }
		uint32 GetBufferSize() const;

		/** Retain local memory buffer for reuse, rather than reallocating. */
		void RetainLocalMemory() { bEmptyArray = false; }

		/** FRenderResource overrides */
		void InitRHI(FRHICommandListBase& RHICmdList) override;
		FString GetFriendlyName() const override { return BufferName; }
		/** End FRenderResource overrides */
	private:
		bool bEmptyArray = true;
		int32 NumValues = INDEX_NONE;
		FString BufferName;
		TArray<float> Array;
	};

	/**
	* Floating point GPU buffer converting float to Half/FFloat16.
	*/
	class FHalfArrayBufferWithSRV : public FVertexBufferWithSRV
	{
	public:
		void SetBufferName(const FString& InBufferName) { BufferName = InBufferName; }

		void Init(const FFloat16* InArray, const int32 Num);
		void Init(const TArray<FFloat16>& InArray) { Init(InArray.GetData(), InArray.Num()); }
		void Init(const float* InArray, const int32 Num);
		void Init(const TArray<float>& InArray);
		void Init(const FVector3f* InArray, const int32 Num) { Init(Num ? &InArray[0].X : nullptr, Num * 3); }
		void Init(const FVector4f* InArray, const int32 Num) { Init(Num ? &InArray[0].X : nullptr, Num * 4); }

		int32 GetNumValues() const { return NumValues; }
		uint32 GetBufferSize() const;

		/** FRenderResource overrides */
		void InitRHI(FRHICommandListBase& RHICmdList) override;
		FString GetFriendlyName() const override { return BufferName; }
		/** End FRenderResource overrides */
	private:
		int32 NumValues = INDEX_NONE;
		FString BufferName;
		TArray<FFloat16> Array;
	};

	/**
	* Integral GPU buffer compacting int32 to the smallest possible of uint8, uint16, or uint32.
	*/
	class FIndexArrayBufferWithSRV : public FVertexBufferWithSRV
	{
	public:
		void SetBufferName(const FString& InBufferName) { BufferName = InBufferName; }

		//! Convert float to uint8 with clamping, and optionaly shifting via \p MultOffset.
		void Init(const int32* InArray, const int32 Num);
		void Init(const uint32* InArray, const int32 Num);
		void Init(const TArray<int32>& InArray) { Init(InArray.Num() ? &InArray[0] : nullptr, InArray.Num()); }
		void Init(const FIntVector2* InArray, const int32 Num) { Init(Num ? &InArray[0].X : nullptr, Num * 2); }
		void Init(const FIntVector3* InArray, const int32 Num) { Init(Num ? &InArray[0].X : nullptr, Num * 3); }
		void Init(const FIntVector4* InArray, const int32 Num) { Init(Num ? &InArray[0].X : nullptr, Num * 4); }
		void Init(const TArray<FIntVector4>& InArray) { Init(InArray.Num() ? &InArray[0].X : nullptr, InArray.Num() * 4); }

		void Force32BitPacking() { bForce32 = true; }

		int32 GetNumValues() const { return NumValues; }
		int32 GetDataStride() const { return bForce32 ? sizeof(uint32) : bUint8 ? sizeof(uint8) : bUint16 ? sizeof(uint16) : sizeof(uint32); }
		int32 GetOffset() const { return Offset; }
		uint32 GetBufferSize() const;

		/** FRenderResource overrides */
		void InitRHI(FRHICommandListBase& RHICmdList) override;
		FString GetFriendlyName() const override { return BufferName; }
		/** End FRenderResource overrides */
	private:
		bool bUint8 = false;
		bool bUint16 = false;
		bool bForce32 = false;
		int32 Offset = 0;
		int32 NumValues = INDEX_NONE;
		FString BufferName;
		TArray<uint32> Array;
	};

} // namespace UE::ChaosDeformable
