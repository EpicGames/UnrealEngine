// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "ChaosFlesh/ChaosDeformableTetrahedralComponent.h"
#include "UObject/ObjectMacros.h"


#include "ChaosDeformableGameplayComponent.generated.h"

class UBoxComponent;
class UStaticMeshComponent;

USTRUCT(BlueprintType)
struct FRigBoundRayCasts
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics")
		bool bEnableRigBoundRaycasts = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics")
		int32 MaxNumTests = 5;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics")
		bool bTestDownOnly = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics")
		float TestRange = 0.5f;
	
	/** Objects to skip if hit during \c DetectEnvironmentCollisions(). */
	UPROPERTY(EditAnywhere, Category = "Physics")
	TArray<TWeakObjectPtr<UPrimitiveComponent>> EnvironmentCollisionsSkipList;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics")
	TEnumAsByte<ECollisionChannel> CollisionChannel = ECC_WorldStatic;
};


USTRUCT(BlueprintType)
struct FGameplayColllisions
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics")
	FRigBoundRayCasts RigBoundRayCasts;

};

/**
*	UDeformableGameplayComponent
*/
UCLASS(meta = (BlueprintSpawnableComponent))
class CHAOSFLESHENGINE_API UDeformableGameplayComponent : public UDeformableTetrahedralComponent
{
	GENERATED_UCLASS_BODY()

public:
	~UDeformableGameplayComponent();


	virtual void PreSolverUpdate() override;


	/** 
	*  DetectEnvironmentCollisions
	* 
	* Do raycasts against the scene and feed the results into transient constraints in the solver.
	* Setup data for this method is initialized in the FAuthorSceneCollisionCandidates dataflow node,
	* stored in the rest collection. The TargetDeformationSkeleton on the rest collection is used
	* for the origin of the raycasts.
	*
	* \p MaxNumTests is the upper limit on the number of scene line tests.
	* \p bTestDownOnly culls line tests that are not in the "down" direction.
	* \p TestRange indicates how close to "down" tests need to be; 1 means only vectors pointing
	*    straight down are tested, 0.5 would include any vector about 45 degrees from straight down,
	*    0.0 means any vector pointing down at all, and negative values up to -1 would include vectors
	*    pointing up.
	* \p CollisionChannel denotes which scene objects to collide against.
	*/
	void DetectEnvironmentCollisions(const int32 MaxNumTests = 100, const bool bTestDownOnly = true, const float TestRange = 0.0, const ECollisionChannel CollisionChannel = ECollisionChannel::ECC_PhysicsBody);


	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics", DisplayName = "Collisions", meta = (DisplayPriority = 10))
	FGameplayColllisions GameplayColllisions;



private:
	TSet<int32> EnvCollisionsPreviousHits;
};

