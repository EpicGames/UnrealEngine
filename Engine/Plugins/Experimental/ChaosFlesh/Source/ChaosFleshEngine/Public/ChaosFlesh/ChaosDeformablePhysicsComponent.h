// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Deformable/ChaosDeformableSolverProxy.h"
#include "Chaos/Deformable/ChaosDeformableSolver.h"
#include "ChaosFlesh/ChaosDeformableSolverThreading.h"
#include "Components/MeshComponent.h"
#include "DeformableInterface.h"
#include "UObject/ObjectMacros.h"
#include "ProceduralMeshComponent.h"
#include "ChaosDeformablePhysicsComponent.generated.h"

class ADeformableSolverActor;
class UDeformableSolverComponent;


/**
*	UDeformablePhysicsComponent
*/
UCLASS(meta = (BlueprintSpawnableComponent))
class CHAOSFLESHENGINE_API UDeformablePhysicsComponent : public UPrimitiveComponent, public IDeformableInterface
{
	GENERATED_UCLASS_BODY()

public:
	typedef Chaos::Softs::FDeformableSolver FDeformableSolver;
	typedef Chaos::Softs::FThreadingProxy FThreadingProxy;
	typedef Chaos::Softs::FDataMapValue FDataMapValue;

	~UDeformablePhysicsComponent() {}

	UFUNCTION(BlueprintCallable, Category = "Physics")
	void EnableSimulation(UDeformableSolverComponent* DeformableSolverComponent);

	UFUNCTION(BlueprintCallable, Category = "Physics")
	void DisableSimulation();

	UFUNCTION(BlueprintCallable, Category = "Physics")
	void EnableSimulationFromActor(ADeformableSolverActor* DeformableSolverActor);

	virtual FThreadingProxy* NewProxy() { return nullptr; }
	virtual void AddProxy(Chaos::Softs::FDeformableSolver::FGameThreadAccess& GameThreadSolver);
	virtual void RemoveProxy(Chaos::Softs::FDeformableSolver::FGameThreadAccess& GameThreadSolver);
	virtual void PreSolverUpdate() {};
	virtual FDataMapValue NewDeformableData() { return FDataMapValue(nullptr); }
	virtual void UpdateFromSimulation(const FDataMapValue* SimulationBuffer) {}
	virtual FDataMapValue WriteRestartDataToSimulation() { return FDataMapValue(nullptr); }

	virtual void OnCreatePhysicsState() override;
	virtual void OnDestroyPhysicsState() override;
	virtual bool ShouldCreatePhysicsState() const override;
	virtual bool HasValidPhysicsState() const override;

	/** PrimarySolver */
	UDeformableSolverComponent* GetDeformableSolver();
	const UDeformableSolverComponent* GetDeformableSolver() const;
	
	/** Primary solver component in which this flesh component has been registered. 
	 * Soft object ptr since the UDeformablePhysicsComponent is stored in the chaos cache which requires
	 * soft ref in order to record itself from another thread
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics", meta = (DisplayPriority = 2))
	TSoftObjectPtr<UDeformableSolverComponent> PrimarySolverComponent;
	
	const FThreadingProxy* GetPhysicsProxy() const { return PhysicsProxy; }
	FThreadingProxy* GetPhysicsProxy() { return PhysicsProxy; }

#if WITH_EDITOR
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
#endif

protected:
	FThreadingProxy* PhysicsProxy = nullptr;

};
