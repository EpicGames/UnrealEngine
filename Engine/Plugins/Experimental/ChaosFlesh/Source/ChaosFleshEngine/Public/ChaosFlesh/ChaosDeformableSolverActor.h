// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/BillboardComponent.h"
#include "DeformableInterface.h"
#include "GameFramework/Actor.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"

#include "ChaosDeformableSolverActor.generated.h"

class UDeformableSolverComponent;

UCLASS()
class CHAOSFLESHENGINE_API ADeformableSolverActor : public AActor, public IDeformableInterface
{
	GENERATED_UCLASS_BODY()

public:

	UDeformableSolverComponent* GetDeformableSolverComponent() { return SolverComponent; }
	const UDeformableSolverComponent* GetDeformableSolverComponent() const { return SolverComponent; }

	UPROPERTY(Category = "Physics", VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UDeformableSolverComponent> SolverComponent = nullptr;

	/*
	* Display icon in the editor
	*/
	UPROPERTY()
	TObjectPtr<UBillboardComponent> SpriteComponent;

#if WITH_EDITOR
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
#endif

private:
	void CreateBillboardIcon(const FObjectInitializer& ObjectInitializer);

};