// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosDeformablePhysicsComponent.h"
#include "Chaos/Deformable/ChaosDeformableSolver.h"
#include "Chaos/Deformable/ChaosDeformableSolverProxy.h"
#include "Chaos/Deformable/ChaosDeformableSolverTypes.h"
#include "ChaosFlesh/ChaosDeformableSolverThreading.h"
#include "ChaosFlesh/ChaosDeformableSolverGroups.h"
#include "DeformableInterface.h"
#include "Components/SceneComponent.h"
#include "Dataflow/Interfaces/DataflowPhysicsSolver.h"
#include "UObject/ObjectMacros.h"

#include "ChaosDeformableSolverComponent.generated.h"

class UDeformablePhysicsComponent;
class UDeformableCollisionsComponent;

USTRUCT(BlueprintType)
struct FConnectedObjectsGroup
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "ConnectedObjects", meta = (EditCondition = "false"))
	TArray< TObjectPtr<UDeformablePhysicsComponent> > DeformableComponents;
};

USTRUCT(BlueprintType)
struct FDataflowFleshSolverProxy : public FDataflowPhysicsSolverProxy
{
	GENERATED_USTRUCT_BODY()

	FDataflowFleshSolverProxy(Chaos::Softs::FDeformableSolverProperties InProp = Chaos::Softs::FDeformableSolverProperties()) :
		FDataflowPhysicsSolverProxy()
	{
	}
	virtual ~FDataflowFleshSolverProxy() override = default;

	//~ Begin FPhysicsSolverInterface interface
	virtual void AdvanceSolverDatas(const float DeltaTime) override
	{
		Chaos::Softs::FDeformableSolver::FPhysicsThreadAccess PhysicsThreadAccess(Solver.Get(), Chaos::Softs::FPhysicsThreadAccessor());
		PhysicsThreadAccess.Simulate(DeltaTime);
	}
	virtual float GetTimeStep() override
	{
		const Chaos::Softs::FDeformableSolver::FPhysicsThreadAccess PhysicsThreadAccess(Solver.Get(), Chaos::Softs::FPhysicsThreadAccessor());
		return PhysicsThreadAccess.GetProperties().TimeStepSize;
	}
	virtual bool IsValid() const override { return Solver.IsValid();}
	
	virtual const UScriptStruct* GetScriptStruct() const override
	{
		return StaticStruct();
	}
	//~ End FPhysicsSolverInterface interface

	/** Chaos deformable solver that will be used in the component */
	TUniquePtr<Chaos::Softs::FDeformableSolver> Solver;
};

template <>
struct TStructOpsTypeTraits<FDataflowFleshSolverProxy> : public TStructOpsTypeTraitsBase2<FDataflowFleshSolverProxy>
{
	enum { WithCopy = false };
};

/**
*	UDeformableSolverComponent
*/
UCLASS(meta = (BlueprintSpawnableComponent))
class CHAOSFLESHENGINE_API UDeformableSolverComponent : public USceneComponent, public IDeformableInterface, public IDataflowPhysicsSolverInterface
{
	GENERATED_UCLASS_BODY()

public:
	typedef Chaos::Softs::FThreadingProxy FThreadingProxy;
	typedef Chaos::Softs::FFleshThreadingProxy FFleshThreadingProxy;
	typedef Chaos::Softs::FDeformablePackage FDeformablePackage;
	typedef Chaos::Softs::FDataMapValue FDataMapValue;
	typedef Chaos::Softs::FDeformableSolver FDeformableSolver;

	~UDeformableSolverComponent();
	void UpdateTickGroup();

	// Begin IDataflowPhysicsSolverInterface overrides
	virtual FString GetSimulationName() const override {return GetName();};
	virtual FDataflowSimulationAsset& GetSimulationAsset() override {return SimulationAsset;};
	virtual const FDataflowSimulationAsset& GetSimulationAsset() const override {return SimulationAsset;};
	virtual FDataflowSimulationProxy* GetSimulationProxy() override {return &FleshSolverProxy;}
	virtual const FDataflowSimulationProxy* GetSimulationProxy() const  override {return &FleshSolverProxy;}
	virtual void BuildSimulationProxy() override;
	virtual void ResetSimulationProxy() override;
	virtual void WriteToSimulation(const float DeltaTime, const bool bAsyncTask) override;
	virtual void ReadFromSimulation(const float DeltaTime, const bool bAsyncTask) override;
	virtual void ReadRestartData() override;
	// End IDataflowPhysicsSolverInterface overrides

	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif // WITH_EDITOR
	//~ End UObject Interface

	// Begin UActorComponent overrides
	virtual bool ShouldCreatePhysicsState() const override {return true;}
	virtual void BeginPlay() override;
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	// End UActorComponent overrides

	/* Game thread access to the solver proxy */
	FDeformableSolver::FGameThreadAccess GameThreadAccess();

	/* Physics thread access to the solver proxy */
	FDeformableSolver::FPhysicsThreadAccess PhysicsThreadAccess();

	bool IsSimulating(UDeformablePhysicsComponent*) const;
	bool IsSimulatable() const;
	void AddDeformableProxy(UDeformablePhysicsComponent* InComponent);
	void RemoveDeformableProxy(UDeformablePhysicsComponent* InComponent);
	void Simulate(float DeltaTime);
	void SetSimulationTicking(const bool InSimulationTicking) {bSimulationTicking = InSimulationTicking;}

	/* Callback to trigger the deformable update after the simulation */
	void UpdateDeformableEndTickState(bool bRegister);

	/** Stop the simulation, and keep the cloth in its last pose. */
	UFUNCTION(CallInEditor, BlueprintCallable, Category = "Physics")
	void ResetSimulationProperties(const FSolverTimingGroup& TimingGroup, const FSolverEvolutionGroup& EvolutionGroup,
		FSolverCollisionsGroup CollisionsGroup, FSolverConstraintsGroup ConstraintsGroup, FSolverForcesGroup ForcesGroup,
		FSolverDebuggingGroup DebuggingGroup, FSolverMuscleActivationGroup MuscleActivationGroup);
	
	/* Solver dataflow asset used to advance in time */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics", meta=(EditConditionHides), AdvancedDisplay)
	FDataflowSimulationAsset SimulationAsset;

	/* Properties : Do NOT place ungrouped properties in this class */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Physics", meta = (EditCondition = "false"))
	FConnectedObjectsGroup ConnectedObjects;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Physics")
	FSolverTimingGroup SolverTiming;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Physics")
	FSolverEvolutionGroup SolverEvolution;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Physics")
	FSolverCollisionsGroup SolverCollisions;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Physics")
	FSolverConstraintsGroup SolverConstraints;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Physics")
	FSolverForcesGroup SolverForces;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Physics")
	FSolverDebuggingGroup SolverDebugging;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Physics")
	FSolverMuscleActivationGroup SolverMuscleActivation;

	// Simulation Variables
	FDataflowFleshSolverProxy FleshSolverProxy;

#if WITH_EDITOR
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
#endif

protected:

	/** Ref for the deformable solvers parallel task, so we can detect whether or not a sim is running */
	FGraphEventRef ParallelDeformableTask;
	FDeformableEndTickFunction DeformableEndTickFunction;

	/** Boolean to check if we can tick the simulation */
	bool bSimulationTicking = true;
};

