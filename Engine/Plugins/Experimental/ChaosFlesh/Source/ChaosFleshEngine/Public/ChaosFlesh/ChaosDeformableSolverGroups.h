// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosFlesh/ChaosDeformableTypes.h"
#include "UObject/ObjectMacros.h"

#include "ChaosDeformableSolverGroups.generated.h"
USTRUCT(BlueprintType)
struct FSolverTimingGroup
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "SolverTiming")
	int32 NumSubSteps = 2;

	UPROPERTY(EditAnywhere, Category = "SolverTiming")
	int32 NumSolverIterations = 5;

	UPROPERTY(EditAnywhere, Category = "SolverTiming")
	bool FixTimeStep = false;

	UPROPERTY(EditAnywhere, Category = "SolverTiming")
	float TimeStepSize = 0.05f;

	UPROPERTY(EditAnywhere, Category = "SolverTiming")
	bool bDoThreadedAdvance = true;

	/** ObjectType defines how to initialize the rigid objects state, Kinematic, Sleeping, Dynamic. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "SolverTiming")
	EDeformableExecutionModel ExecutionModel = EDeformableExecutionModel::Chaos_Deformable_PostPhysics;
};

USTRUCT(BlueprintType)
struct FSolverDebuggingGroup
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Physics")
	bool CacheToFile = false;
};

USTRUCT(BlueprintType)
struct FSolverQuasistaticsGroup
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Quasistatics")
	bool bDoQuasistatics = false;
};

USTRUCT(BlueprintType)
struct FSolverEvolutionGroup
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Evolution")
	FSolverQuasistaticsGroup SolverQuasistatics;

};

USTRUCT(BlueprintType)
struct FSolverGridBasedCollisionsGroup
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "GridBasedCollisions")
	bool bUseGridBasedConstraints = false;

	UPROPERTY(EditAnywhere, Category = "GridBasedCollisions")
	float GridDx = 25.;
};

USTRUCT(BlueprintType)
struct FInComponentSpringCollisionGroup
{
	GENERATED_BODY()
	
	/** If uses in-component spring self-collision */
	UPROPERTY(EditAnywhere, Category = "InComponentSpringCollision")
	bool bDoInComponentSpringCollision = false;
	
	/** N ring to exclude for in-component spring self-collision */
	UPROPERTY(EditAnywhere, Category = "InComponentSpringCollision")
	int32 NRingExcluded = 1;
};

USTRUCT(BlueprintType)
struct FSpringCollisionGroup
{
	GENERATED_BODY()
	
	/** If uses component-component spring collision */
	UPROPERTY(EditAnywhere, Category = "SpringCollision")
	bool bDoSpringCollision = false;
	
	/** In-component spring self collision detection parameters */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "SpringCollision")
	FInComponentSpringCollisionGroup InComponentSpringCollision;
	
	/** Search radius for point triangle collision pairs */
	UPROPERTY(EditAnywhere, Category = "SpringCollision")
	float CollisionSearchRadius = 0.f;
	
	/** Collision spring stiffness; larger value will stop penetration better */
	UPROPERTY(EditAnywhere, Category = "SpringCollision")
	float SpringCollisionStiffness = 500.f;
	
	/**  Anisotropic springs will allow sliding on the triangle */
	UPROPERTY(EditAnywhere, Category = "SpringCollision")
	bool bAllowSliding = true;
	
	/** Do self collision with kinematic triangles as well */
	UPROPERTY(EditAnywhere, Category = "SpringCollision")
	bool bCollideWithFullmesh = true;
};

USTRUCT(BlueprintType)
struct FSphereRepulsionGroup
{
	GENERATED_BODY()
	
	/** If uses sphere repulsion for collision */
	UPROPERTY(EditAnywhere, Category = "SphereRepulsion")
	bool bDoSphereRepulsion = false;
	
	/** Search radius for repulsion pairs */
	UPROPERTY(EditAnywhere, Category = "SphereRepulsion")
	float SphereRepulsionRadius = 0.f;
	
	/** Stiffness for sphere repulsion */
	UPROPERTY(EditAnywhere, Category = "SphereRepulsion")
	float SphereRepulsionStiffness = 500.f;
};

USTRUCT(BlueprintType)
struct FSolverGaussSeidelConstraintsGroup
{
	GENERATED_BODY()

	/** Enable the Gauss Seidel solver instead of the existing XPBD. */
	UPROPERTY(EditAnywhere, Category = "GaussSeidelConstraints")
	bool bUseGaussSeidelConstraints = false;

	/** Enable another model that runs simulation faster. */
	UPROPERTY(EditAnywhere, Category = "GaussSeidelConstraints")
	bool bUseGSNeohookean = false;

	/** Enable acceleration technique for Gauss Seidel solver to make simulation look better within a limited budget. */
	UPROPERTY(EditAnywhere, Category = "GaussSeidelConstraints")
	bool bUseSOR = true;

	/** Acceleration related parameter. Tune it down if simulation becomes unstable. */
	UPROPERTY(EditAnywhere, Category = "GaussSeidelConstraints")
	float OmegaSOR = 1.6f;

	/** Enable dynamic springs controlled by constraint manager. */
	UPROPERTY(EditAnywhere, Category = "GaussSeidelConstraints")
	bool bEnableDynamicSprings = true;
	
	/** Component-component collision detection radius and stiffness */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "GaussSeidelConstraints")
	FSpringCollisionGroup SpringCollision;

	/** Sphere repulsion parameters */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "GaussSeidelConstraints")
	FSphereRepulsionGroup SphereRepulsion;
};

USTRUCT(BlueprintType)
struct FSolverCollisionsGroup
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Physics")
	bool bUseFloor = true;

	//UPROPERTY(EditAnywhere, Category = "Collisions")
	//FSolverGridBasedCollisionsGroup SolverGridBasedCollisions;
};

USTRUCT(BlueprintType)
struct FSolverCorotatedConstraintsGroup
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Corotated")
	bool bEnableCorotatedConstraint = true;

	UPROPERTY(EditAnywhere, Category = "Corotated")
	bool bDoBlended = false;

	UPROPERTY(EditAnywhere, Category = "Corotated")
	float BlendedZeta = 0;
};

USTRUCT(BlueprintType)
struct FSolverConstraintsGroup
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Constraints")
	bool bEnablePositionTargets = true;

	UPROPERTY(EditAnywhere, Category = "Constraints")
	bool bEnableKinematics = true;

	UPROPERTY(EditAnywhere, Category = "Constraints")
	FSolverCorotatedConstraintsGroup CorotatedConstraints;

	/** These are options for another solver. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Constraints")
	FSolverGaussSeidelConstraintsGroup GaussSeidelConstraints;
};

USTRUCT(BlueprintType)
struct FSolverForcesGroup
{
	GENERATED_BODY()

	UE_DEPRECATED(5.6, "YoungModulus has been deprecated.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Please use SetFleshDefaultProperties dataflow node to set up vertex stiffness."))
	float YoungModulus = 100000;

	UE_DEPRECATED(5.6, "Damping has been deprecated.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Please use SetFleshDefaultProperties dataflow node to set up vertex damping."))
	float Damping = 0;

	UPROPERTY(EditAnywhere, Category = "Forces")
	bool bEnableGravity = true;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FSolverForcesGroup() = default;
	FSolverForcesGroup(const FSolverForcesGroup&) = default;
	FSolverForcesGroup(FSolverForcesGroup&&) = default;
	FSolverForcesGroup& operator=(const FSolverForcesGroup&) = default;
	FSolverForcesGroup& operator=(FSolverForcesGroup&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

USTRUCT(BlueprintType)
struct FSolverMuscleActivationGroup
{
	GENERATED_BODY()
	UE_DEPRECATED(5.6, "Direct access to this variable is forbidden.")
	// Muscle activation is driven by origin-insertion length. Check SetMuscleActivationParameter for more info.
	UPROPERTY(EditAnywhere, Category = "MuscleActivation", DisplayName = "Do Length Based Muscle Activation")
	bool bDoMuscleActivation = false;

	// Override muscle activation with animation curves. Check ReadSkeletalMeshCurves node for more info.
	UPROPERTY(EditAnywhere, Category = "MuscleActivation")
	bool bOverrideMuscleActivationWithAnimatedCurves = false;
	
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void SetDoLengthBasedMuscleActivation(bool bInDoMuscleActivation) { bDoMuscleActivation = bInDoMuscleActivation; }
	bool GetDoLengthBasedMuscleActivation() const { return bDoMuscleActivation; }
	FSolverMuscleActivationGroup() = default;
	~FSolverMuscleActivationGroup() = default;
	FSolverMuscleActivationGroup(const FSolverMuscleActivationGroup&) = default;
	FSolverMuscleActivationGroup(FSolverMuscleActivationGroup&&) = default;
	FSolverMuscleActivationGroup& operator=(const FSolverMuscleActivationGroup&) = default;
	FSolverMuscleActivationGroup& operator=(FSolverMuscleActivationGroup&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

