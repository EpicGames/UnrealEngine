// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineBaseTypes.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "ChaosDeformableSolverThreading.generated.h"

class UDeformableSolverComponent;

class FParallelDeformableTask
{
	UDeformableSolverComponent* DeformableSolverComponent;
	float DeltaTime;

public:
	FParallelDeformableTask(UDeformableSolverComponent* InDeformableSolverComponent, float InDeltaTime)
		: DeformableSolverComponent(InDeformableSolverComponent)
		, DeltaTime(InDeltaTime)
	{
	}

	TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FParallelDeformableTask, STATGROUP_TaskGraphTasks);
	}

	static ENamedThreads::Type GetDesiredThread();

	static ESubsequentsMode::Type GetSubsequentsMode();

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent);
};


/**
* Tick function that does post physics work on deformable components. This executes in EndPhysics (after physics is done)
**/
USTRUCT()
struct FDeformableEndTickFunction : public FTickFunction
{
	GENERATED_USTRUCT_BODY()

	UDeformableSolverComponent* DeformableSolverComponent = nullptr;

	/**
	* Abstract function to execute the tick.
	* @param DeltaTime - frame time to advance, in seconds.
	* @param TickType - kind of tick for this frame.
	* @param CurrentThread - thread we are executing on, useful to pass along as new tasks are created.
	* @param MyCompletionGraphEvent - completion event for this task. Useful for holding the completetion of this task until certain child tasks are complete.
	*/
	virtual void ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
	/** Abstract function to describe this tick. Used to print messages about illegal cycles in the dependency graph. */
	virtual FString DiagnosticMessage() override;
	/** Function used to describe this tick for active tick reporting. **/
	virtual FName DiagnosticContext(bool bDetailed) override;
};

template<>
struct TStructOpsTypeTraits<FDeformableEndTickFunction> : public TStructOpsTypeTraitsBase2<FDeformableEndTickFunction>
{
	enum
	{
		WithCopy = false
	};
};
