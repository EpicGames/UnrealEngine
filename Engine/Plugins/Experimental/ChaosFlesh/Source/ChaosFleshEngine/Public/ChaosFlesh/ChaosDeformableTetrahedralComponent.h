// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Deformable/ChaosDeformableSolverProxy.h"
#include "Chaos/Deformable/ChaosDeformableSolver.h"
#include "ChaosFlesh/ChaosDeformablePhysicsComponent.h"
#include "ChaosFlesh/ChaosDeformableSolverThreading.h"
#include "ChaosFlesh/FleshAsset.h"
#include "ChaosFlesh/FleshDynamicAsset.h"
#include "ChaosFlesh/ChaosFleshDeformerBufferManager.h"
#include "ChaosFlesh/SimulationAsset.h"
#include "Components/MeshComponent.h"
#include "Dataflow/Interfaces/DataflowInterfaceGeometryCachable.h"
#include "UObject/ObjectMacros.h"
#include "ProceduralMeshComponent.h"
#include "ChaosDeformableTetrahedralComponent.generated.h"

class FFleshCollection;
class ADeformableSolverActor;
class UDeformableSolverComponent;
class FChaosDeformableTetrahedralSceneProxy;
class USkinnedAsset;
class USkeletalMesh;

/**
*  Options for binding positions query.
*/
UENUM()
enum ChaosDeformableBindingOption : uint8
{
	WorldPos		UMETA(DisplayName = "World Positions"),
	WorldDelta		UMETA(DisplayName = "World Deltas"),
	ComponentPos    UMETA(DisplayName = "Component Positions"),
	ComponentDelta  UMETA(DisplayName = "Component Deltas"),
	BonePos			UMETA(DisplayName = "Bone Positions"),
	BoneDelta		UMETA(DisplayName = "Bone Deltas"),
};


USTRUCT(BlueprintType)
struct FFleshSimulationSpaceGroup
{
	GENERATED_USTRUCT_BODY()

	/**
	* Bone from the associated skeletal mesh (indicated by RestCollection.TargetSkeletalMesh) to use as
	* the space the sim runs in.
	*/
	UPROPERTY(EditAnywhere, Category = "Physics", meta = (GetOptions = "GetSimSpaceBoneNameOptions", EditCondition = "SimSpace == ChaosDeformableSimSpace::Bone"))
	FName SimSpaceBoneName;

	/** Space the simulation will run in. */
	UPROPERTY(EditAnywhere, Category = "Physics")
	TEnumAsByte<ChaosDeformableSimSpace> SimSpace = ChaosDeformableSimSpace::World;

	/** The skeletal mesh to use pull the \c SimSpaceBoneName from. */
	UPROPERTY()
	TObjectPtr<USkeletalMesh> SimSpaceSkeletalMesh;

	int32 SimSpaceTransformIndex = INDEX_NONE;
	int32 SimSpaceTransformGlobalIndex = INDEX_NONE;
};


USTRUCT(BlueprintType)
struct FBodyForcesGroup
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics")
	bool bApplyGravity = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float DampingMultiplier = 1.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float StiffnessMultiplier = 1.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float IncompressibilityMultiplier = 1.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float InflationMultiplier = 1.f;
};

/**
*	UDeformableTetrahedralComponent
*/
UCLASS(meta = (BlueprintSpawnableComponent))
class CHAOSFLESHENGINE_API UDeformableTetrahedralComponent : public UDeformablePhysicsComponent, public IDataflowGeometryCachable
{
	GENERATED_UCLASS_BODY()

public:
	typedef Chaos::Softs::FFleshThreadingProxy FFleshThreadingProxy;

	~UDeformableTetrahedralComponent();

	/** USceneComponent Interface */
	virtual void BeginPlay() override;
	virtual void OnRegister() override;
	virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	void UpdateLocalBounds();
	virtual void EndPlay(const EEndPlayReason::Type ReasonEnd) override;
	void Invalidate();

	/** UPrimitiveComponent Interface */
	//virtual FPrimitiveSceneProxy* CreateSceneProxy() override;

	/** GPU Interface */
	Chaos::Softs::FChaosFleshDeformableGPUManager& GetGPUBufferManager() { return GPUBufferManager; }

	/** Simulation Interface */
	virtual FThreadingProxy* NewProxy() override;
	virtual FDataMapValue NewDeformableData() override;
	virtual void UpdateFromSimulation(const FDataMapValue* SimulationBuffer) override;
	virtual FDataMapValue WriteRestartDataToSimulation() override;

	/** RestCollection */
	UFUNCTION(BlueprintCallable, Category = "Physics")
	void SetRestCollection(const UFleshAsset * InRestCollection);
	
	UFUNCTION(BlueprintCallable, Category = "Physics")
	const UFleshAsset* GetRestCollection() const { return RestCollection; }

	/** DynamicCollection */
	void ResetDynamicCollection();
	UFleshDynamicAsset* GetDynamicCollection() { return DynamicCollection; }
	const UFleshDynamicAsset* GetDynamicCollection() const { return DynamicCollection; }

	/** SimulationCollection */
	void ResetSimulationCollection();
	USimulationAsset* GetSimulationCollection() { return SimulationCollection; }
	const USimulationAsset* GetSimulationCollection() const { return SimulationCollection; }

	/** @deprecated Use GetSkeletalMeshEmbeddedPositions() instead. */
	UFUNCTION(BlueprintCallable, Category = "Physics", meta = (DeprecatedFunction, DeprecationMessage = "Use GetSkeletalMeshEmbeddedPositions() instead."))
	TArray<FVector> GetSkeletalMeshBindingPositions(const USkeletalMesh* InSkeletalMesh) const;
	
	//~ Begin IDataflowGeometryCachable Interface
	virtual TArray<FVector3f> GetGeometryCachePositions(const USkeletalMesh* SkeletalMesh) const override;
	virtual TArray<FVector3f> GetGeometryCachePositions(const UStaticMesh* StaticMesh) const override;

	virtual TOptional<TArray<int32>> GetMeshImportVertexMap(const USkinnedAsset& SkinnedMeshAsset) const override;

	virtual bool IsSkeletalMeshAnimationCompatible(const USkeletalMeshComponent* SkeletalMeshComponent) const override;
	//~ End IDataflowGeometryCachable Interface

	virtual void SetMaterial(int32 Index, UMaterialInterface* InMaterial) override
	{
		Material = InMaterial;
	}
	/**
	* Get the current positions of the transformation hierarchy from \c TargetDeformationSkeleton,
	* deformed by the tetrahedral mesh.  Results can be in world space postions/deltas, component space
	* positions/deltas, or bone space positions/deltas.  If a bone space is desired \p TargetBone
	* must indicate which bone to use. TargetDeformationSkeletonOffset is an offset transform that moves 
	* the \c TargetDeformationSkeleton to be co-located with the flesh mesh.
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics")
	TArray<FVector> GetSkeletalMeshEmbeddedPositions(const ChaosDeformableBindingOption Format, const FTransform TargetDeformationSkeletonOffset, const FName TargetBone = "", const float SimulationBlendWeight = 1.f) const;

	UPROPERTY(EditAnywhere, Category = "Physics", meta = (DisplayPriority = 10))
	FFleshSimulationSpaceGroup SimulationSpace;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics", meta = (DisplayPriority = 10))
	FBodyForcesGroup BodyForces;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0", DisplayPriority = 100))
	float MassMultiplier = 1.f;

	UPROPERTY(EditAnywhere, Category = "Rendering")
	TObjectPtr<UProceduralMeshComponent> Mesh;

	UPROPERTY(BlueprintReadWrite, Category = "Rendering")
	TObjectPtr<UMaterialInterface> Material = nullptr;

	UPROPERTY(EditAnywhere, Category = "Rendering")
	TArray<int32> HideTetrahedra;

private:
	/** FleshAsset that describes the simulation rest state. */
	UPROPERTY(EditAnywhere, Category = "Physics", meta = (DisplayPriority = 1))
	TObjectPtr<const UFleshAsset> RestCollection;

	/** Current simulation state. */
	UPROPERTY()
	TObjectPtr<UFleshDynamicAsset> DynamicCollection;

	/** Simulator input */
	UPROPERTY()
	TObjectPtr<USimulationAsset> SimulationCollection;

	/* Returns a list of bone names from the currently selected skeletal mesh. */
	UFUNCTION(CallInEditor)
	TArray<FString> GetSimSpaceBoneNameOptions() const;

	//! Update \c SimSpaceSkeletalMesh and \c SimSpaceTransformIndex according to
	//! \c RestCollection->TargetSkeletalMesh and SimSpaceBoneName.
	//! \ret \c true if a valid sim space transform is found.
	bool UpdateSimSpaceTransformIndex();

	//! Return the rest transform to be used as the simulation space. 
	//! \c UpdateSimSpaceTransformIndex() must be called prior to calling this function.
	FTransform GetSimSpaceRestTransform() const;

	//
	// Render the Procedural Mesh
	//
	struct FFleshRenderMesh
	{
		TArray<FVector> Vertices;
		TArray<int32> Triangles;
		TArray<FVector> Normals;
		TArray<FVector2D> UVs;
		TArray<FLinearColor> Colors;
		TArray<FProcMeshTangent> Tangents;
	};
	FFleshRenderMesh* RenderMesh = nullptr;
	void RenderProceduralMesh();
	void ResetProceduralMesh();

	bool bBoundsNeedsUpdate = true;
	FBoxSphereBounds BoundingBox = FBoxSphereBounds(ForceInitToZero);

	FTransform PrevTransform = FTransform::Identity;

	TArray<FVector> GetSkeletalMeshEmbeddedPositionsInternal(const ChaosDeformableBindingOption Format, const FTransform TargetDeformationSkeletonOffset, const FName TargetBone = "", const float SimulationBlendWeight = 1.f, TArray<bool>* OutInfluence = nullptr) const;
	TArray<FVector> GetEmbeddedPositionsInternal(const TArray<FVector>& InPositions, const FName SkeletalMeshName, const float SimulationBlendWeight = 1.f, TArray<bool>* OutInfluence = nullptr) const;
	TArray<FVector> GetSkeletalMeshBindingPositionsInternal(const USkeletalMesh* InSkeletalMesh, TArray<bool>* OutInfluence = nullptr) const;
	void DebugDrawSkeletalMeshBindingPositions() const;

	//FChaosDeformableTetrahedralSceneProxy* RenderProxy = nullptr;
	Chaos::Softs::FChaosFleshDeformableGPUManager GPUBufferManager;
};

