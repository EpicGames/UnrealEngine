// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Chaos/DebugDrawQueue.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineBaseTypes.h"

#include "ChaosDeformableTypes.generated.h"

UENUM(BlueprintType)
enum class EDeformableExecutionModel : uint8
{
	Chaos_Deformable_PrePhysics UMETA(DisplayName = "Before Physics"),
	Chaos_Deformable_DuringPhysics UMETA(DisplayName = "During Physics"),
	Chaos_Deformable_PostPhysics UMETA(DisplayName = "After Physics"),
	//
	Chaos_Max UMETA(Hidden)
};

struct FChaosEngineDeformableCVarParams
{
	bool bEnableDeformableSolver = true;
#ifdef WITH_ENGINE
	bool bDoDrawSimulationMesh = true;
#else
	bool bDoDrawSimulationMesh = false;
#endif
	bool bDoDrawSkeletalMeshBindingPositions = false;
	float DrawSkeletalMeshBindingPositionsSimulationBlendWeight = 1.f;
	bool bDoDrawSceneRaycasts = false;
	bool bDoDrawCandidateRaycasts = false;

	int32 EnvCollisionsLineTraceBatchSize = 10;

	bool bTestUnpacking = false;
	bool bTestSparseMappings = false;
	bool bTestCompressedIndexing = false;
	bool bUpdateGPUBuffersOnTick = true;

	bool IsDebugDrawingEnabled()
	{
#if WITH_EDITOR
		// p.Chaos.DebugDraw.Enabled 1
		return Chaos::FDebugDrawQueue::GetInstance().IsDebugDrawingEnabled();
#else
		return false;
#endif
	}
};