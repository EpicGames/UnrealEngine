// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ChaosFlesh/ChaosDeformableGPUBuffers.h"
#include "GeometryCollection/ManagedArrayAccessor.h"
#include "RenderCommandFence.h"

class UDeformableTetrahedralComponent;
class FFleshCollection;

namespace Chaos::Softs 
{
	class /*CHAOS_API*/ FChaosFleshDeformableGPUManager
	{
	public:
		class FBindingsBuffer
		{
		public:
			FBindingsBuffer();
			~FBindingsBuffer();

			void SetOwnerName(const FName& OwnerName);

			/** Initialize GPU buffers. */
			bool InitData(const TManagedArrayAccessor<FIntVector4>* ParentsArray, const TManagedArrayAccessor<FVector4f>* WeightsArray, const TManagedArrayAccessor<FVector3f>* OffsetArray, const TManagedArrayAccessor<float>* MaskArray, const TManagedArray<FVector3f>* RestVerticesArray, const TManagedArray<FVector3f>* VerticesArray);

			/**
			* Update local sparse vertices buffer from dense vertices, and uploads to GPU.
			* Returns \c true if \p DenseVertex differs from prior values.
			*/
			bool UpdateVertices(const TArray<FVector3f>& DenseVertex);

			UE::ChaosDeformable::FFloatArrayBufferWithSRV& GetRestVerticesBuffer() { return RestVerticesBuffer; }
			const UE::ChaosDeformable::FFloatArrayBufferWithSRV& GetRestVerticesBuffer() const { return RestVerticesBuffer; }

			UE::ChaosDeformable::FFloatArrayBufferWithSRV& GetVerticesBuffer() { return VerticesBuffer; }
			const UE::ChaosDeformable::FFloatArrayBufferWithSRV& GetVerticesBuffer() const { return VerticesBuffer; }

			UE::ChaosDeformable::FIndexArrayBufferWithSRV& GetParentsBuffer() { return ParentsBuffer; }
			const UE::ChaosDeformable::FIndexArrayBufferWithSRV& GetParentsBuffer() const { return ParentsBuffer; }

			UE::ChaosDeformable::FIndexArrayBufferWithSRV& GetVertsToParentsBuffer() { return VertsToParentsBuffer; }
			const UE::ChaosDeformable::FIndexArrayBufferWithSRV& GetVertsToParentsBuffer() const { return VertsToParentsBuffer; }

			UE::ChaosDeformable::FHalfArrayBufferWithSRV& GetWeightsBuffer() { return WeightsBuffer; }
			const UE::ChaosDeformable::FHalfArrayBufferWithSRV& GetWeightsBuffer() const { return WeightsBuffer; }

			UE::ChaosDeformable::FHalfArrayBufferWithSRV& GetOffsetsBuffer() { return OffsetsBuffer; }
			const UE::ChaosDeformable::FHalfArrayBufferWithSRV& GetOffsetsBuffer() const { return OffsetsBuffer; }

			UE::ChaosDeformable::FHalfArrayBufferWithSRV& GetMaskBuffer() { return MaskBuffer; }
			const UE::ChaosDeformable::FHalfArrayBufferWithSRV& GetMaskBuffer() const { return MaskBuffer; }

			//const TArray<int32>& GetCondensationMapping() const { return CondensationMapping; }
			//const TArray<FIntVector4>& GetSparseParents() const { return SparseParents; }
			//const TArray<FVector3f>& GetSparseRestVertices() const { return SparseRestVertices; }
			//const TArray<FVector3f>& GetSparseVertices() const { return SparseVertices; }

		private:
			bool UpdateSparseVertices(const TArray<FVector3f>& DenseVertexIn, TArray<FVector3f>& SparseVerticesIn) const;

			/**
			* Create a mapping that reduces the number of vertices referenced by the parents array,
			* and the vertices required by culling unreferenced vertices.
			*/
			void InitCondensationMapping(const TManagedArrayAccessor<FIntVector4>& ParentsArray);

			/** Release GPU resources. */
			void DestroyGPUBuffers();

			/** Fence used in render thread cleanup on destruction. */
			FRenderCommandFence RenderResourceDestroyFence;

			// Non-const
			UE::ChaosDeformable::FFloatArrayBufferWithSRV VerticesBuffer;

			// Const
			UE::ChaosDeformable::FFloatArrayBufferWithSRV RestVerticesBuffer;
			UE::ChaosDeformable::FIndexArrayBufferWithSRV ParentsBuffer;
			UE::ChaosDeformable::FIndexArrayBufferWithSRV VertsToParentsBuffer;
			UE::ChaosDeformable::FHalfArrayBufferWithSRV WeightsBuffer;
			UE::ChaosDeformable::FHalfArrayBufferWithSRV OffsetsBuffer;
			UE::ChaosDeformable::FHalfArrayBufferWithSRV MaskBuffer;

			// Sparse vertex index to dense (excludes -1) - maps dense vertices to SparseVertices.
			TArray<int32> SparseToDenseIndices;
			// For each render vertex, the corresponding sparse parent index.
			TArray<uint32> VertsToSparseParents;
			// Sparse (unique) parents.
			TArray<FIntVector4> SparseParents;

			// Local buffer for sparse rest vertices.
			TArray<FVector3f> SparseRestVertices;
			// Local buffer for sparse time varying vertices.
			TArray<FVector3f> SparseVertices;
		};

	public:
		FChaosFleshDeformableGPUManager();
		FChaosFleshDeformableGPUManager(UDeformableTetrahedralComponent* InComponent);
		virtual ~FChaosFleshDeformableGPUManager();

		void SetOwner(UDeformableTetrahedralComponent* InComponent);

		/** Register a consumer of a buffer this class will manage. */
		void RegisterGPUBufferConsumer(const void* ID) const { GPUBufferConsumers.AddUnique(ID); }
		/** Remove a registered consumer. */
		void UnRegisterGPUBufferConsumer(const void* ID) const;
		/** Returns \c true if there's at least 1 registered consumer. */
		bool HasRegisteredGPUBufferConsumer() const { return !GPUBufferConsumers.IsEmpty(); }

		/** Initialize the binding data for a render mesh. Will register \p ID as consumer. Returns success. */
		bool InitGPUBindingsBuffer(const void* ID, const FName MeshName, const int32 LodIndex, const bool ForceInit=false);
		/** Returns true if we have a matching bindings buffer. */
		bool HasGPUBindingsBuffer(const void* ID, const FName MeshName, const int32 LodIndex) const;

		/** 
		 * If a consumer has been registered, updates time varying data from the component's 
		 * dynamic collection. Returns success. 
		 */
		bool UpdateGPUBuffers();

		/** Look up bindings data buffers associated with \p ID, \p MeshName, and \p LodIndex. */
		const FBindingsBuffer* GetBindingsBuffer(const void* ID, const FName MeshName, const int32 LodIndex) const;
		FBindingsBuffer* GetBindingsBuffer(const void* ID, const FName MeshName, const int32 LodIndex);

	private:
		UDeformableTetrahedralComponent* Component = nullptr;

		// Convenience functions
		const FFleshCollection* GetRestCollection() const;
		const TManagedArray<FVector3f>* GetRestVertexArray() const;
		const TManagedArray<FVector3f>* GetVertexArray() const;

		// Consumers
		mutable TArray<const void*> GPUBufferConsumers;

		// Data tables
		typedef TTuple<void*, FName, int32> BufferId;
		typedef TMap<BufferId, TSharedPtr<FBindingsBuffer>> IdToBuffers;
		IdToBuffers BufferTable;
	};

} // namespace Chaos::Softs