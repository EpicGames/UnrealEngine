// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosFlesh/FleshComponent.h"
#include "ChaosFlesh/ChaosDeformableSolverActor.h"
#include "CoreMinimal.h"
#include "DeformableInterface.h"
#include "GameFramework/Actor.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"

#include "FleshActor.generated.h"



UCLASS()
class CHAOSFLESHENGINE_API AFleshActor : public AActor, public IDeformableInterface
{
	GENERATED_UCLASS_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "Physics")
	void EnableSimulation(ADeformableSolverActor* Actor);

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Physics")
	TObjectPtr<UFleshComponent> FleshComponent;
	UFleshComponent* GetFleshComponent() const { return FleshComponent; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics", meta = (DisplayPriority = 1))
	TObjectPtr<ADeformableSolverActor> PrimarySolver;

#if WITH_EDITOR
	ADeformableSolverActor* PreEditChangePrimarySolver = nullptr;
	virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual bool GetReferencedContentObjects(TArray<UObject*>& Objects) const override;
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
#endif
};
