// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ChaosFlesh/ChaosDeformableSolverGroups.h"
#include "ChaosFlesh/FleshCollection.h"
#include "Dataflow/DataflowNodeParameters.h"
#include "Dataflow/DataflowEngineTypes.h"
#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowInstance.h"
#include "Engine/StaticMesh.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ObjectMacros.h"

#include "FleshAsset.generated.h"

class UFleshAsset;
class UDataflow;
struct FReferenceSkeleton;
struct FSkeletalMaterial;
class USkeletalMesh;
class USkeleton;
class FSkeletalMeshRenderData;

/**
*	FFleshAssetEdit
*     Structured RestCollection access where the scope
*     of the object controls serialization back into the
*     dynamic collection
*
*/
class CHAOSFLESHENGINE_API FFleshAssetEdit
{
public:
	typedef TFunctionRef<void()> FPostEditFunctionCallback;
	friend UFleshAsset;

	/**
	 * @param UFleshAsset				The FAsset to edit
	 */
	FFleshAssetEdit(UFleshAsset* InAsset, FPostEditFunctionCallback InCallable);
	~FFleshAssetEdit();

	FFleshCollection* GetFleshCollection();

private:
	FPostEditFunctionCallback PostEditCallback;
	UFleshAsset* Asset;
};

/**
* UFleshAsset (UObject)
*
* UObject wrapper for the FFleshAsset
*
*/
UCLASS(BlueprintType, customconstructor)
class CHAOSFLESHENGINE_API UFleshAsset : public UObject, public IDataflowContentOwner, public IDataflowInstanceInterface
{
	GENERATED_UCLASS_BODY()
	friend class FFleshAssetEdit;

	//
	// FleshCollection
	// 
	// The FleshCollection stores all the user per-particle properties 
	// for the asset. This is used for simulation and artists 
	// configuration. Only edit the FleshCollection using its Edit
	// object. For example;
	// 
	// {
	//		FFleshAssetEdit EditObject = UFleshAsset->EditCollection();
	//		if( TSharedPtr<FFleshCollection> FleshCollection = EditObject.GetFleshCollection() )
	//      {
	//		}
	//		// the destructor of the edit object will perform invalidation. 
	// }
	//
	TSharedPtr<FFleshCollection, ESPMode::ThreadSafe> FleshCollection;

	void PostEditCallback();

public:

	UFleshAsset(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

#if WITH_EDITOR
	/** Post edit change property */
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditUndo() override;
	/** Propogate the fact that transform have been updated to all components */
	void PropagateTransformUpdateToComponents() const;
	void BuildResourceForRendering(FSkeletalMeshRenderData& OutSkeletalMeshRenderData) const;
#endif //if WITH_EDITOR

	/**Editing the collection should only be through the edit object.*/
	UE_DEPRECATED(5.7, "Use SetFleshCollection instead")
	void SetCollection(FFleshCollection* InCollection);
	UE_DEPRECATED(5.7, "Use GetFleshCollection instead")
	const FFleshCollection* GetCollection() const { return FleshCollection.Get(); }
	UE_DEPRECATED(5.7, "Access to non-const FleshCollection is now deprecated, use SetFleshCollection to modify FleshCollection.")
	FFleshCollection* GetCollection() { return FleshCollection.Get(); }

	void SetFleshCollection(TUniquePtr<FFleshCollection>&& InCollection);
	TSharedPtr<const FFleshCollection> GetFleshCollection() const { return FleshCollection; }

	TArray<FSkeletalMaterial> GetMaterials() const;
	const FReferenceSkeleton& GetRefSkeleton() const;
	FGuid GetAssetGuid() const;

	TManagedArray<FVector3f>& GetPositions();
	const TManagedArray<FVector3f>* FindPositions() const;

	FFleshAssetEdit EditCollection() const {
		UFleshAsset* ThisNC = const_cast<UFleshAsset*>(this); 
		return FFleshAssetEdit(ThisNC, [ThisNC]() {ThisNC->PostEditCallback(); });
	}

	void Serialize(FArchive& Ar);
	virtual void PostLoad() override;

	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;

	//~ Begin IDataflowContentOwner interface
	virtual TObjectPtr<UDataflowBaseContent> CreateDataflowContent() override;
	virtual void WriteDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent) const override;
	virtual void ReadDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent) override;
	//~ End IDataflowContentOwner interface

	//~ Begin IDataflowInstanceInterface
	virtual const FDataflowInstance& GetDataflowInstance() const;
	virtual FDataflowInstance& GetDataflowInstance();
	//~ End IDataflowInstanceInterface
	
	//
	// Dataflow
	//
#if WITH_EDITORONLY_DATA
	// this is deprecated data replaced with Setter and Getter but it is exposed to blueprint 
	// so we use the Setter/Getter to make sure the blueprint will use them 
	UE_DEPRECATED(5.8, "Direct set/get is no longer authorized, use SetDataflowAsset instead")
	UPROPERTY(BlueprintReadWrite, BlueprintSetter = SetDataflowAsset, BlueprintGetter = GetDataflowAsset, Category = "Dataflow")
	TObjectPtr<UDataflow> DataflowAsset;

	UE_DEPRECATED(5.8, "Direct set/get is no longer authorized, use SetDataflowTerminal instead")
	UPROPERTY(BlueprintReadWrite, BlueprintSetter = SetDataflowTerminal, BlueprintGetter = GetDataflowTerminal, Category = "Dataflow")
	FString DataflowTerminal = "FleshAssetTerminal";
#endif

	UFUNCTION(BlueprintCallable, Category = "Dataflow")
	void SetDataflowAsset(UDataflow* InDataflowAsset);

	UFUNCTION(BlueprintCallable, Category = "Dataflow")
	UDataflow* GetDataflowAsset() const;

	UFUNCTION(BlueprintCallable, Category = "Dataflow")
	void SetDataflowTerminal(const FString& InDataflowTerminal);

	UFUNCTION(BlueprintCallable, Category = "Dataflow")
	FString GetDataflowTerminal() const;

private:
	void MigrateDeprecatedDataflowData();

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	FDataflowInstance DataflowInstance;

public:
	UPROPERTY(EditAnywhere, Category = "Dataflow")
	TArray<FStringValuePair> Overrides;

	//
	// SkeletalMesh
	//
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animation")
	TObjectPtr<USkeletalMesh> SkeletalMesh;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animation")
	TObjectPtr<USkeleton> Skeleton;

	/**
	* Skeleton to use with the flesh deformer or \c GetSkeletalMeshEmbeddedPositions() on the flesh component. 
	* Bindings for this skeletal mesh must be stored in the rest collection.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering")
	TObjectPtr<USkeletalMesh> TargetDeformationSkeleton;

	UE_DEPRECATED(5.8, "FleshAsset StaticMesh property is deprecated, as it is not no longer used")
	UPROPERTY()
	TObjectPtr<UStaticMesh> StaticMesh;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Render")
	bool bRenderInEditor = true;

	/** Information for thumbnail rendering */
	UPROPERTY()
	TObjectPtr<class UThumbnailInfo> ThumbnailInfo;

	/*
	* The following PreviewScene properties are modeled after PreviewSkeletalMesh in USkeleton
	*	- they are inside WITH_EDITORONLY_DATA because they are not used at game runtime
	*	- TSoftObjectPtrs since that will make it possible to avoid loading these assets until the PreviewScene asks for them
	*	- DuplicateTransient so that if you copy a ClothAsset it won't copy these preview properties
	*	- AssetRegistrySearchable makes it so that if the user searches the name of a PreviewScene asset in the Asset Browser
	*/

	/** Animation asset used in this asset */
	UPROPERTY(DuplicateTransient, AssetRegistrySearchable)
	TSoftObjectPtr<UAnimationAsset> PreviewAnimationAsset = nullptr;

	UPROPERTY(DuplicateTransient, AssetRegistrySearchable)
	FSolverTimingGroup PreviewSolverTiming;

	UPROPERTY(DuplicateTransient, AssetRegistrySearchable)
	FSolverEvolutionGroup PreviewSolverEvolution;

	UPROPERTY(DuplicateTransient, AssetRegistrySearchable)
	FSolverCollisionsGroup PreviewSolverCollisions;

	UPROPERTY(DuplicateTransient, AssetRegistrySearchable)
	FSolverConstraintsGroup PreviewSolverConstraints;

	UPROPERTY(DuplicateTransient, AssetRegistrySearchable)
	FSolverForcesGroup PreviewSolverForces;

	UPROPERTY(DuplicateTransient, AssetRegistrySearchable)
	FSolverDebuggingGroup PreviewSolverDebugging;

	UPROPERTY(DuplicateTransient, AssetRegistrySearchable)
	FSolverMuscleActivationGroup PreviewSolverMuscleActivation;
	
#endif // WITH_EDITORONLY_DATA

private:
	/** A unique identifier as used by the section rendering code. */
	UPROPERTY()
	FGuid AssetGuid = FGuid::NewGuid();
};


/** 
 * Dataflow content owning dataflow and solver properties that will be used to evaluate the graph
 */
UCLASS()
class CHAOSFLESHENGINE_API  UDataflowFleshContent : public UDataflowSkeletalContent
{
	GENERATED_BODY()

public:
	UDataflowFleshContent();
	virtual ~UDataflowFleshContent() override {}

	//~ UObject interface
	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	/** Set all the preview actor exposed properties */
	virtual void SetActorProperties(TObjectPtr<AActor>& PreviewActor) const override;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif //if WITH_EDITOR

	UPROPERTY(EditAnywhere, Category = "Preview", Transient, SkipSerialization)
	FSolverTimingGroup SolverTiming;

	UPROPERTY(EditAnywhere, Category = "Preview", Transient, SkipSerialization)
	FSolverEvolutionGroup SolverEvolution;

	UPROPERTY(EditAnywhere, Category = "Preview", Transient, SkipSerialization)
	FSolverCollisionsGroup SolverCollisions;

	UPROPERTY(EditAnywhere, Category = "Preview", Transient, SkipSerialization)
	FSolverConstraintsGroup SolverConstraints;

	UPROPERTY(EditAnywhere, Category = "Preview", Transient, SkipSerialization)
	FSolverForcesGroup SolverForces;

	UPROPERTY(EditAnywhere, Category = "Preview", Transient, SkipSerialization)
	FSolverDebuggingGroup SolverDebugging;

	UPROPERTY(EditAnywhere, Category = "Preview", Transient, SkipSerialization)
	FSolverMuscleActivationGroup SolverMuscleActivation;
};

