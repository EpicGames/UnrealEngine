// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "GeometryCollection/ManagedArray.h"

class USkeletalMesh;
class FFleshCollection;
class UStaticMesh;

namespace ChaosFlesh 
{
	CHAOSFLESHENGINE_API FString GetMeshId(const USkeletalMesh* SkeletalMesh, const bool bUseImportModel);

	CHAOSFLESHENGINE_API FString GetMeshId(const UStaticMesh* StaticMesh);

	void BoundSurfacePositions(
		const FString& MeshId,
		const FFleshCollection* FleshCollection,
		const TManagedArray<FVector3f>* RestVertices,
		const TManagedArray<FVector3f>* SimulatedVertices,
		TArray<FVector3f>& OutPositions);

	CHAOSFLESHENGINE_API void BoundSurfacePositions(
		const USkeletalMesh* SkeletalMesh,
		const FFleshCollection* FleshCollection,
		const TManagedArray<FVector3f>* RestVertices,
		const TManagedArray<FVector3f>* SimulatedVertices,
		TArray<FVector3f>& OutPositions);

	CHAOSFLESHENGINE_API void BoundSurfacePositions(
		const UStaticMesh* StaticMesh,
		const FFleshCollection* FleshCollection,
		const TManagedArray<FVector3f>* RestVertices,
		const TManagedArray<FVector3f>* SimulatedVertices,
		TArray<FVector3f>& OutPositions);
}
