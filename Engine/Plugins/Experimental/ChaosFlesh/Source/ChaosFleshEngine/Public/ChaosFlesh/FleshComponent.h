// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Deformable/ChaosDeformableSolverProxy.h"
#include "Chaos/Deformable/ChaosDeformableSolver.h"
#include "ChaosFlesh/ChaosDeformableSolverThreading.h"
#include "ChaosFlesh/ChaosDeformablePhysicsComponent.h"
#include "ChaosFlesh/ChaosDeformableTetrahedralComponent.h"
#include "ChaosFlesh/ChaosDeformableGameplayComponent.h"
#include "ChaosFlesh/FleshAsset.h"
#include "ChaosFlesh/FleshDynamicAsset.h"
#include "ChaosFlesh/SimulationAsset.h"
#include "Components/MeshComponent.h"
#include "UObject/ObjectMacros.h"
#include "ProceduralMeshComponent.h"
#include "FleshComponent.generated.h"

class FFleshCollection;
class ADeformableSolverActor;
class UDeformableSolverComponent;




/**
*	FleshComponent
*/
UCLASS(meta = (BlueprintSpawnableComponent))
class CHAOSFLESHENGINE_API UFleshComponent : public UDeformableGameplayComponent
{
	GENERATED_UCLASS_BODY()
public:

};
