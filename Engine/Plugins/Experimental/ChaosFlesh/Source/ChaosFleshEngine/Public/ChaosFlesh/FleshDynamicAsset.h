// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "GeometryCollection/ManagedArrayCollection.h"
#include "UObject/ObjectMacros.h"

#include "FleshDynamicAsset.generated.h"

class UFleshAsset;
 
/**
* UFleshDynamicAsset (UObject)
*
* UObject wrapper for the dynamic attributes from the tetrahedral simulation. 
* 
*	FVectorArray Vertex = GetAttribute<FVector3f>("Vertex", VerticesGroup)
*
*/
UCLASS(customconstructor)
class CHAOSFLESHENGINE_API UFleshDynamicAsset : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	UFleshDynamicAsset(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	void Init();
	void Reset(const FManagedArrayCollection* InCopyFrom = nullptr);
	void ResetAttributesFrom(const FManagedArrayCollection* InCopyFrom);

	TManagedArray<FVector3f>& GetPositions();
	const TManagedArray<FVector3f>* FindPositions() const;
	TManagedArray<int32>& GetObjectState();
	TManagedArray<float>& GetActivations();
	const TManagedArray<float>* FindActivations() const;

	const FManagedArrayCollection* GetCollection() const { return DynamicCollection.Get(); }
	FManagedArrayCollection* GetCollection() { return DynamicCollection.Get(); }
private:

	//
	// FManagedArrayCollection
	// 
	// The ManagedArrayCollection stores all the dynamic properties 
	// of the simulation. This is non-const data stored on the 
	// component. 
	//
	TSharedPtr<FManagedArrayCollection, ESPMode::ThreadSafe> DynamicCollection;
};
