// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "GeometryCollection/ManagedArrayCollection.h"
#include "UObject/ObjectMacros.h"

#include "SimulationAsset.generated.h"
 
/**
* USimulationAsset (UObject)
*
* UObject wrapper for the dynamic attributes that are inputs to the tetrahedral simulation. 
*/
UCLASS(customconstructor)
class CHAOSFLESHENGINE_API USimulationAsset : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	USimulationAsset(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	void Init();
	void Reset(const FManagedArrayCollection* InCopyFrom = nullptr);
	void ResetAttributesFrom(const FManagedArrayCollection* InCopyFrom);

	TManagedArray<int32>& GetObjectState();

	const FManagedArrayCollection* GetCollection() const { return SimulationCollection.Get(); }
	FManagedArrayCollection* GetCollection() { return SimulationCollection.Get(); }
private:

	//
	// FManagedArrayCollection
	// 
	// The ManagedArrayCollection stores all the dynamic properties 
	// for input to the simulation. This is non-const data stored on the 
	// component. 
	//
	TSharedPtr<FManagedArrayCollection, ESPMode::ThreadSafe> SimulationCollection;
};
