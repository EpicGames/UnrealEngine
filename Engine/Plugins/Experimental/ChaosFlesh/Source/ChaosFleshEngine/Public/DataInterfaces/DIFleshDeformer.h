// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "OptimusComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "ChaosFlesh/ChaosDeformableTetrahedralComponent.h"
#include "PrimitiveComponentId.h"

#include "DIFleshDeformer.generated.h"

class FFleshDeformerDataInterfaceParameters;
class FRDGBuffer;
class FRDGBufferSRV;
class FSceneInterface;
class FSkeletalMeshObject;
class USkinnedMeshComponent;

DECLARE_LOG_CATEGORY_EXTERN(LogFleshDeformer, Verbose, All);

USTRUCT()
struct FFleshDeformerParameters
{
	GENERATED_BODY();
};


/**
* Compute Framework Data Interface for reading skeletal mesh and tetrahedral mesh bindings.
*/
UCLASS(Category = ComputeFramework)
class /*OPTIMUSCORE_API*/ UDIFleshDeformer : public UOptimusComputeDataInterface
{
	GENERATED_BODY()

public:
	virtual ~UDIFleshDeformer();

	//~ Begin UOptimusComputeDataInterface Interface
	FString GetDisplayName() const override;
	TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	//~ End UOptimusComputeDataInterface Interface

	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("FleshDeformer"); }
	void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	void GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const override;
	void GetShaderHash(FString& InOutKey) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface

	UPROPERTY(EditAnywhere, Category = Deformer, meta = (ShowOnlyInnerProperties))
	FFleshDeformerParameters FleshDeformerParameters;

private:
	mutable TWeakObjectPtr<UDeformableTetrahedralComponent> ProducerComponent;
};

/** Compute Framework Data Provider for reading skeletal mesh. */
UCLASS(BlueprintType, editinlinenew, Category = ComputeFramework)
class UDIFleshDeformerDataProvider : public UComputeDataProvider
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Binding)
	TObjectPtr<USkinnedMeshComponent> SkinnedMesh = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Binding)
	TObjectPtr<UStaticMeshComponent> StaticMesh = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Binding)
	TObjectPtr<UDeformableTetrahedralComponent> FleshMesh = nullptr;

	UPROPERTY()
	FFleshDeformerParameters FleshDeformerParameters;

	//~ Begin UComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface
};

class FDIFleshDeformerProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FDIFleshDeformerProviderProxy(USkinnedMeshComponent* SkinnedMeshComponentIn, UDeformableTetrahedralComponent* FleshComponentIn, const FFleshDeformerParameters& FleshDeformerParametersIn);

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void GatherPermutations(FPermutationData& InOutPermutationData) const override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	void GatherDispatchData(FDispatchData const& InDispatchData);
	//~ End FComputeDataProviderRenderProxy Interface

private:
	using FParameters = FFleshDeformerDataInterfaceParameters;

	USkinnedMeshComponent* SkinnedMeshComponent = nullptr;
	FSceneInterface* SceneInterface = nullptr;
	FPrimitiveComponentId ComponentId;
	UDeformableTetrahedralComponent* FleshComponent = nullptr;

	FFleshDeformerParameters FleshDeformerParameters;

	int32 TetIndex = INDEX_NONE;
	FString MeshId;

	uint32 NumVertices = 0;

	uint32 NumTetRestVertices = 0;
	uint32 NumTetVertices = 0;

	uint32 NumParents = 0;
	uint32 NumWeights = 0;
	uint32 NumOffset = 0;
	uint32 NumMask = 0;

	FRDGBuffer* TetRestVertexBuffer = nullptr;
	FRDGBufferSRV* TetRestVertexBufferSRV = nullptr;
	FRDGBuffer* TetVertexBuffer = nullptr;
	FRDGBufferSRV* TetVertexBufferSRV = nullptr;

	FRDGBuffer* ParentsBuffer = nullptr;
	FRDGBufferSRV* ParentsBufferSRV = nullptr;
	FRDGBuffer* WeightsBuffer = nullptr;
	FRDGBufferSRV* WeightsBufferSRV = nullptr;
	FRDGBuffer* OffsetBuffer = nullptr;
	FRDGBufferSRV* OffsetBufferSRV = nullptr;
	FRDGBuffer* MaskBuffer = nullptr;
	FRDGBufferSRV* MaskBufferSRV = nullptr;

	float NullFloatBuffer = 0.0f;
	int32 NullIntBuffer = 0;
};