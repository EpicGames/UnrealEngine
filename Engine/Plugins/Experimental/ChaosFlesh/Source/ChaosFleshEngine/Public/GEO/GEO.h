// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"

#include <string>

namespace ChaosFlesh
{
#if WITH_EDITOR
	bool CHAOSFLESHENGINE_API ReadGEO(const std::string& filename, TMap<FString, int32>& IntVars, TMap<FString, TArray<int32>>& IntVectorVars, TMap<FString, TArray<float>>& FloatVectorVars, TMap<FString, TPair<TArray<std::string>, TArray<int32>>>& IndexedStringVars, std::ostream* errorStream=nullptr);
#endif // WITH_EDITOR
} // namespace ChaosFlesh
