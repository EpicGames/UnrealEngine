// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ChaosFleshNodes : ModuleRules
	{
        public ChaosFleshNodes(ReadOnlyTargetRules Target) : base(Target)
        {
	        PrivateDependencyModuleNames.AddRange(new string[] {"ChaosCaching"});
	        PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"ChaosCore",
					"Chaos",
					"DataflowCore",
					"DataflowEngine",
					"DataflowSimulation",
					"ChaosFlesh",
					"ChaosFleshEngine",
					"Engine",
					"GeometryAlgorithms",
					"GeometryCollectionEngine",
					"GeometryCore",
					"GeometryFramework",
					"MeshConversion",
					"MeshConversionEngineTypes",
					"MeshDescription",
					"TetMeshing",
					"SkeletalMeshDescription",
				}
			);
        }
	}
}
