// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshAddKinematicParticlesNode.h"

#include "Dataflow/DataflowEngineUtil.h"
#include "Engine/SkeletalMesh.h"
#include "GeometryCollection/Facades/CollectionKinematicBindingFacade.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"
#include "GeometryCollection/GeometryCollection.h"
#include "BoneWeights.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFleshAddKinematicParticlesNode)

//DEFINE_LOG_CATEGORY_STATIC(ChaosFleshAddKinematicParticlesDataflowNodeLog, Log, All);

void FAddKinematicParticlesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<DataType>(&Collection) || Out->IsA<TArray<int32>>(&TargetIndicesOut))
	{
		DataType InCollection = GetValue<DataType>(Context, &Collection);
		TArray<int32> TargetIndices;
		if (TManagedArray<FVector3f>* Vertices = InCollection.FindAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup))
		{
			if (FindInput(&VertexIndicesIn) && FindInput(&VertexIndicesIn)->GetConnection())
			{
				TArray<int32> BoundVerts;
				TArray<float> BoundWeights;

				for (int32 SelectionIndex : GetValue<TArray<int32>>(Context, &VertexIndicesIn))
				{
					if (0 <= SelectionIndex && SelectionIndex < Vertices->Num())
					{
						BoundVerts.Add(SelectionIndex);
					}
				}
				if (BoundVerts.Num())
				{
					BoundWeights.Init(1.0, BoundVerts.Num());
					GeometryCollection::Facades::FKinematicBindingFacade Kinematics(InCollection);
					Kinematics.AddKinematicBinding(Kinematics.SetBoneBindings(INDEX_NONE, BoundVerts, BoundWeights));
				}
			}
			else if (TObjectPtr<USkeletalMesh> SkeletalMesh = GetValue<TObjectPtr<USkeletalMesh>>(Context, &SkeletalMeshIn))
			{
				int32 IndexValue = GetValue<int32>(Context, &BoneIndexIn);
				if (IndexValue != INDEX_NONE)
				{
					TArray<FTransform> ComponentPose;
					UE::Dataflow::Animation::GlobalTransforms(SkeletalMesh->GetRefSkeleton(), ComponentPose);

					TArray<int32> BranchIndices;
					if (SkeletalSelectionMode == ESkeletalSeletionMode::Dataflow_SkeletalSelection_Branch)
					{
						TArray<int32> ToProcess;
						int32 CurrentIndex = IndexValue;
						while (SkeletalMesh->GetRefSkeleton().IsValidIndex(CurrentIndex))
						{
							TArray<int32> Buffer;
							SkeletalMesh->GetRefSkeleton().GetDirectChildBones(CurrentIndex, Buffer);

							if (Buffer.Num())
							{
								ToProcess.Append(Buffer);
							}

							BranchIndices.Add(CurrentIndex);

							CurrentIndex = INDEX_NONE;
							if (ToProcess.Num())
							{
								CurrentIndex = ToProcess.Pop();
							}
						}
					}
					else // ESkeletalSeletionMode::Dataflow_SkeletalSelection_Single
					{
						BranchIndices.Add(IndexValue);
					}

					// Add standalone particles, not bound to a transform group - so for these particles BoneMap = INDEX_NONE.
					int32 ParticleIndex = InCollection.AddElements(BranchIndices.Num(), FGeometryCollection::VerticesGroup);
					TManagedArray<FVector3f>& CurrentVertices = InCollection.ModifyAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);

					for (int32 Index = 0; Index < BranchIndices.Num(); Index++)
					{
						const int32 BoneIndex = BranchIndices[Index];
						FVector3f BonePosition(ComponentPose[BoneIndex].GetTranslation());

						CurrentVertices[ParticleIndex+Index] = BonePosition;

						TArray<int32> BoundVerts;
						TArray<float> BoundWeights;

						BoundVerts.Add(ParticleIndex + Index);
						BoundWeights.Add(1.0);
						TargetIndices.Emplace(ParticleIndex + Index);

						if (BoundVerts.Num())
						{
							GeometryCollection::Facades::FKinematicBindingFacade Kinematics(InCollection);
							Kinematics.AddKinematicBinding(Kinematics.SetBoneBindings(BoneIndex, BoundVerts, BoundWeights));
						}
					}

					//debugging code for tet binding:
					//int32 ParticleIndex1 = InCollection.AddElements(1, FGeometryCollection::VerticesGroup);
					//TManagedArray<FVector3f>& CurrentVertices1 = InCollection.ModifyAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
					//CurrentVertices1[ParticleIndex1][0] = -40.f;
					//CurrentVertices1[ParticleIndex1][1] = -50.f;
					//CurrentVertices1[ParticleIndex1][2] = -10.f;
					//GeometryCollection::Facades::FKinematicBindingFacade Kinematics(InCollection);
					//TArray<int32> BoundVerts;
					//TArray<float> BoundWeights;
					//BoundVerts.Add(ParticleIndex1);
					//BoundWeights.Add(1.0);
					//Kinematics.AddKinematicBinding(Kinematics.SetBoneBindings(BranchIndices[0], BoundVerts, BoundWeights));
					//TargetIndices.Emplace(ParticleIndex1);

					GeometryCollection::Facades::FVertexBoneWeightsFacade(InCollection).AddBoneWeightsFromKinematicBindings();
				}
			}
		}
		SetValue(Context, MoveTemp(InCollection), &Collection);
		SetValue(Context, MoveTemp(TargetIndices), &TargetIndicesOut);
	}
}

