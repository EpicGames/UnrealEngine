// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshAppendTetrahedralCollectionNode.h"

#include "ChaosFlesh/FleshCollection.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "GeometryCollection/Facades/CollectionTransformFacade.h"


FAppendTetrahedralCollectionDataflowNode_v2::FAppendTetrahedralCollectionDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid) 
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&CollectionToAppend);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&CollectionGeometrySelection);
	RegisterOutputConnection(&CollectionToAppendGeometrySelection);
	RegisterOutputConnection(&CollectionGeometryGroupGuids);
	RegisterOutputConnection(&CollectionToAppendGeometryGroupGuids);
}

void FAppendTetrahedralCollectionDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		TUniquePtr<FFleshCollection> InCollection(GetValue(Context, &Collection).NewCopy<FFleshCollection>());
		TUniquePtr<FFleshCollection> InCollectionToAppend(GetValue(Context, &CollectionToAppend).NewCopy<FFleshCollection>());
		TArray<FString> CollectionGeometryGroupGuidsLocal, CollectionToAppendGeometryGroupGuidsLocal;
		if (const TManagedArray<FString>* CollectionGuidArray = InCollection->FindAttribute<FString>("Guid", FGeometryCollection::GeometryGroup))
		{
			CollectionGeometryGroupGuidsLocal = CollectionGuidArray->GetConstArray();
		}
		if (const TManagedArray<FString>* CollectionToAppendGuidArray = InCollectionToAppend->FindAttribute<FString>("Guid", FGeometryCollection::GeometryGroup))
		{
			CollectionToAppendGeometryGroupGuidsLocal = CollectionToAppendGuidArray->GetConstArray();
		}
		GeometryCollection::Facades::FCollectionTransformFacade InCollectionTransformFacade(*InCollection);
		GeometryCollection::Facades::FCollectionTransformFacade InCollectionToAppendTransformFacade(*InCollectionToAppend);
		const TMap<FString, int32> InCollectionBoneNameMap = InCollectionTransformFacade.BoneNameIndexMap();
		const int32 CollectionNumTransform = InCollectionTransformFacade.Num();
		const int32 CollectionToAppendNumTransform = InCollectionToAppendTransformFacade.Num();
		//Append
		InCollectionToAppend->AppendCollection(*InCollection); //InCollection is inserted to the front

		if (bMergeTransform)
		{
			//Reorder and delete transform
			if (const TManagedArray<FString>* InCollectionBoneNames = InCollectionToAppendTransformFacade.FindBoneNames())
			{
				TArray<int32> MergeRemapIndex;
				TArray<int32> SortedMergeList;
				for (int32 Idx = CollectionNumTransform; Idx < InCollectionBoneNames->Num(); ++Idx)
				{
					const FString& BoneName = (*InCollectionBoneNames)[Idx];
					if (InCollectionBoneNameMap.Contains(BoneName) && !InCollectionToAppend->IsGeometry(Idx))
					{
						SortedMergeList.Add(Idx);
						MergeRemapIndex.Add(InCollectionBoneNameMap[BoneName]);
					}
				}
				InCollectionToAppend->MergeElements(FTransformCollection::TransformGroup, SortedMergeList, MergeRemapIndex);
			}
		}
		const int32 CollectionNumGeometries = InCollection->NumElements(FGeometryCollection::GeometryGroup);
		const int32 TotalNumGeometries = InCollectionToAppend->NumElements(FGeometryCollection::GeometryGroup);
		FDataflowGeometrySelection InCollectionGeometrySelection;
		FDataflowGeometrySelection InCollectionToAppendGeometrySelection;
		InCollectionGeometrySelection.Initialize(TotalNumGeometries, false);
		InCollectionToAppendGeometrySelection.Initialize(TotalNumGeometries, false);
		for (int32 GeometryIdx = 0; GeometryIdx < TotalNumGeometries; ++GeometryIdx)
		{
			if (GeometryIdx < CollectionNumGeometries)
			{
				InCollectionGeometrySelection.SetSelected(GeometryIdx);
			}
			else
			{
				InCollectionToAppendGeometrySelection.SetSelected(GeometryIdx);
			}
		}
		SetValue<const FManagedArrayCollection&>(Context, MoveTemp(*InCollectionToAppend), &Collection);
		SetValue(Context, MoveTemp(InCollectionGeometrySelection), &CollectionGeometrySelection);
		SetValue(Context, MoveTemp(InCollectionToAppendGeometrySelection), &CollectionToAppendGeometrySelection);
		SetValue(Context, MoveTemp(CollectionGeometryGroupGuidsLocal), &CollectionGeometryGroupGuids);
		SetValue(Context, MoveTemp(CollectionToAppendGeometryGroupGuidsLocal), &CollectionToAppendGeometryGroupGuids);
	}
}

void FAppendTetrahedralCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection1))
	{
		TUniquePtr<FFleshCollection> InCollection1(GetValue<FManagedArrayCollection>(Context, &Collection1).NewCopy<FFleshCollection>());
		TUniquePtr<FFleshCollection> InCollection2(GetValue<FManagedArrayCollection>(Context, &Collection2).NewCopy<FFleshCollection>());
		TArray<FString> GeometryGroupGuidsLocal1, GeometryGroupGuidsLocal2;
		if (const TManagedArray<FString>* GuidArray1 = InCollection1->FindAttribute<FString>("Guid", FGeometryCollection::GeometryGroup))
		{
			GeometryGroupGuidsLocal1 = GuidArray1->GetConstArray();
		}
		GeometryCollection::Facades::FCollectionTransformFacade InTransformFacade1(*InCollection1);
		GeometryCollection::Facades::FCollectionTransformFacade InTransformFacade2(*InCollection2);
		const TMap<FString, int32> InBoneNameMap = InTransformFacade1.BoneNameIndexMap();
		const int32 InNumTransform2 = InTransformFacade2.Num();
		const int32 InNumTransform1 = InTransformFacade1.Num();
		//Append
		InCollection1->AppendCollection(*InCollection2);

		if (const TManagedArray<FString>* GuidArray2 = InCollection2->FindAttribute<FString>("Guid", FGeometryCollection::GeometryGroup))
		{
			GeometryGroupGuidsLocal2 = GuidArray2->GetConstArray();
		}
		if (bMergeTransform)
		{
			//Reorder and delete transform
			const TManagedArray<FString>* OutBoneNames = InTransformFacade1.FindBoneNames();
			TArray<int32> MergeRemapIndex;
			TArray<int32> SortedMergeList;
			for (int32 Idx = 0; Idx < InNumTransform2; ++Idx)
			{
				const FString& BoneName = (*OutBoneNames)[Idx];
				if (Idx < InNumTransform2 && InBoneNameMap.Contains(BoneName))
				{
					SortedMergeList.Add(Idx);
					MergeRemapIndex.Add(InBoneNameMap[BoneName] + InNumTransform2); //SKM collection is appended to the front
				}
			}
			InCollection1->MergeElements(FTransformCollection::TransformGroup, SortedMergeList, MergeRemapIndex);
		}
		const int32 NumGeometries = InCollection1->NumElements(FGeometryCollection::GeometryGroup);
		const int32 NumGeometries2 = InCollection2->NumElements(FGeometryCollection::GeometryGroup);
		FDataflowGeometrySelection InGeometrySelection1;
		FDataflowGeometrySelection InGeometrySelection2;
		InGeometrySelection1.Initialize(NumGeometries, false);
		InGeometrySelection2.Initialize(NumGeometries, false);
		for (int32 GeometryIdx = 0; GeometryIdx < NumGeometries2; ++GeometryIdx)
		{
			InGeometrySelection2.SetSelected(GeometryIdx);
		}
		for (int32 GeometryIdx = NumGeometries2; GeometryIdx < NumGeometries; ++GeometryIdx)
		{
			InGeometrySelection1.SetSelected(GeometryIdx);
		}
		SetValue<const FManagedArrayCollection&>(Context, MoveTemp(*InCollection1), &Collection1);
		SetValue<FDataflowGeometrySelection>(Context, MoveTemp(InGeometrySelection1), &GeometrySelection1);
		SetValue<FDataflowGeometrySelection>(Context, MoveTemp(InGeometrySelection2), &GeometrySelection2);
		SetValue(Context, MoveTemp(GeometryGroupGuidsLocal1), &GeometryGroupGuidsOut1);
		SetValue(Context, MoveTemp(GeometryGroupGuidsLocal2), &GeometryGroupGuidsOut2);
	}
}

void FDeleteFleshVerticesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		TUniquePtr<FFleshCollection> InCollection(GetValue<FManagedArrayCollection>(Context, &Collection).NewCopy<FFleshCollection>());
		if (IsConnected(&Collection) && IsConnected(&VertexSelection))
		{
			const FDataflowVertexSelection& InDataflowVertexSelection = GetValue<FDataflowVertexSelection>(Context, &VertexSelection);
			if (InDataflowVertexSelection.Num() == InCollection->NumElements(FGeometryCollection::VerticesGroup))
			{
				TArray<int32> VertexSelected = InDataflowVertexSelection.AsArray();
				InCollection->RemoveVertices(VertexSelected);
			}
			else
			{
				Context.Warning(FString::Printf(
					TEXT("DeleteFleshVertices Node: VertexSelection has different size (%d) than the number of vertices (%d) in the Collection."),
					InDataflowVertexSelection.Num(),
					InCollection->NumElements(FGeometryCollection::VerticesGroup)),
					this, Out);
			}
		}
		SetValue<const FManagedArrayCollection&>(Context, MoveTemp(*InCollection), &Collection);
	}
}