// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshAuthorSceneCollisionCandidatesNode.h"

#include "ChaosFlesh/FleshCollection.h"
#include "GeometryCollection/Facades/CollectionConstraintOverrideFacade.h"
#include "Logging/LogMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFleshAuthorSceneCollisionCandidatesNode)

DEFINE_LOG_CATEGORY(LogAuthorSceneCollisionCandidates);

void
FAuthorSceneCollisionCandidates::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<DataType>(&Collection))
	{
		DataType InCollection = GetValue<DataType>(Context, &Collection);
		const int32 BoneIndex = GetValue<int32>(Context, &OriginBoneIndex);

		int32 Num = 0;
		GeometryCollection::Facades::FConstraintOverrideCandidateFacade CnstrCandidates(InCollection);
		CnstrCandidates.DefineSchema();

		if (IsConnected(&VertexIndices))
		{
			if (TManagedArray<FVector3f>* Vertices = InCollection.FindAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup))
			{
				const TArray<int32>& Indices = GetValue<TArray<int32>>(Context, &VertexIndices);
				for (int32 i = 0; i < Indices.Num(); i++)
				{
					if (Indices[i] >= 0 && Indices[i] < Vertices->Num())
					{
						GeometryCollection::Facades::FConstraintOverridesCandidateData Data;
						Data.VertexIndex = Indices[i];
						Data.BoneIndex = BoneIndex;
						CnstrCandidates.Add(Data);
						Num++;
					}
				}
			}
		}
		else
		{
			TSet<int32> UniqueIndices;
			if (bSurfaceVerticesOnly)
			{
				if (TManagedArray<FIntVector>* Indices =
					InCollection.FindAttribute<FIntVector>("Indices", FGeometryCollection::FacesGroup))
				{
					for (int32 i = 0; i < Indices->Num(); i++)
					{
						for (int32 j = 0; j < 3; j++)
						{
							UniqueIndices.Add((*Indices)[i][j]);
						}
					}
				}
			}
			else
			{
				if (TManagedArray<FIntVector4>* Indices =
					InCollection.FindAttribute<FIntVector4>(
						FTetrahedralCollection::TetrahedronAttribute, FTetrahedralCollection::TetrahedralGroup))
				{
					for (int32 i = 0; i < Indices->Num(); i++)
					{
						for (int32 j = 0; j < 4; j++)
						{
							UniqueIndices.Add((*Indices)[i][j]);
						}
					}
				}
			}
			for (TSet<int32>::TConstIterator It = UniqueIndices.CreateConstIterator(); It; ++It)
			{
				GeometryCollection::Facades::FConstraintOverridesCandidateData Data;
				Data.VertexIndex = *It;
				Data.BoneIndex = BoneIndex;
				CnstrCandidates.Add(Data);
				Num++;
			}
		}
		UE_LOGF(LogAuthorSceneCollisionCandidates, Display,
			"'%ls' - Added %d scene collision candidates.",
			*GetName().ToString(), Num);
		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}
