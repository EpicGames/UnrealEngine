// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshCalculateTetrehedralMetricsNode.h"

#include "ChaosFlesh/ChaosFlesh.h"
#include "Chaos/Tetrahedron.h"
#include "ChaosFlesh/FleshCollection.h"
#include "ChaosLog.h"
#include "GeometryCollection/Facades/CollectionTetrahedralMetricsFacade.h"


//=============================================================================
// FCalculateTetMetrics
//=============================================================================

void
FCalculateTetMetrics::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<DataType>(&Collection))
	{
		TUniquePtr<FFleshCollection> InCollection(GetValue<DataType>(Context, &Collection).NewCopy<FFleshCollection>());

		TManagedArray<FIntVector4>* TetMesh =
			InCollection->FindAttribute<FIntVector4>(
				FTetrahedralCollection::TetrahedronAttribute, FTetrahedralCollection::TetrahedralGroup);
		TManagedArray<int32>* TetrahedronStart =
			InCollection->FindAttribute<int32>(
				FTetrahedralCollection::TetrahedronStartAttribute, FGeometryCollection::GeometryGroup);
		TManagedArray<int32>* TetrahedronCount =
			InCollection->FindAttribute<int32>(
				FTetrahedralCollection::TetrahedronCountAttribute, FGeometryCollection::GeometryGroup);

		TManagedArray<FVector3f>* Vertex =
			InCollection->FindAttribute<FVector3f>(
				"Vertex", "Vertices");

		GeometryCollection::Facades::FTetrahedralMetrics TetMetrics(*InCollection);
		TManagedArrayAccessor<float>& SignedVolume = TetMetrics.GetSignedVolume();
		TManagedArrayAccessor<float>& AspectRatio = TetMetrics.GetAspectRatio();

		for (int32 TetMeshIdx = 0; TetMeshIdx < TetrahedronStart->Num(); TetMeshIdx++)
		{
			const int32 TetMeshStart = (*TetrahedronStart)[TetMeshIdx];
			const int32 TetMeshCount = (*TetrahedronCount)[TetMeshIdx];
			
			float MinVol = TNumericLimits<float>::Max();
			float MaxVol = -TNumericLimits<float>::Max();
			double AvgVol = 0.0;

			float MinAR = TNumericLimits<float>::Max();
			float MaxAR = -TNumericLimits<float>::Max();
			double AvgAR = 0.0;
			
			for (int32 i = 0; i < TetMeshCount; i++)
			{
				const int32 Idx = TetMeshStart + i;
				const FIntVector4& Tet = (*TetMesh)[Idx];
				Chaos::TTetrahedron<Chaos::FReal> Tetrahedron(
					(*Vertex)[Tet[0]],
					(*Vertex)[Tet[1]],
					(*Vertex)[Tet[2]],
					(*Vertex)[Tet[3]]);

				float Vol = Tetrahedron.GetSignedVolume();
				SignedVolume.ModifyAt(Idx, Vol);
				MinVol = MinVol < Vol ? MinVol : Vol;
				MaxVol = MaxVol < Vol ? Vol : MaxVol;
				AvgVol += Vol;

				float AR = Tetrahedron.GetAspectRatio();
				AspectRatio.ModifyAt(Idx, AR);
				MinAR = MinAR < AR ? MinAR : AR;
				MaxAR = MaxAR < AR ? AR : MaxAR;
				AvgAR += AR;
			}
			if (TetMeshCount)
			{
				AvgVol /= TetMeshCount;
				AvgAR /= TetMeshCount;
			}
			else
			{
				MinVol = MaxVol = 0.0f;
				MinAR = MaxAR = 0.0f;
			}

			UE_LOGF(LogChaosFlesh, Display,
				"'%ls' - Tet mesh %d of %d stats:\n"
				"    Num Tetrahedra: %d\n"
				"    Volume (min, avg, max): %g, %g, %g\n"
				"    Aspect ratio (min, avg, max): %g, %g, %g",
				*GetName().ToString(),
				(TetMeshIdx+1), TetrahedronStart->Num(),
				TetMeshCount,
				MinVol, AvgVol, MaxVol,
				MinAR, AvgAR, MaxAR);
		}

		SetValue<const DataType&>(Context, *InCollection, &Collection);
	}
}

