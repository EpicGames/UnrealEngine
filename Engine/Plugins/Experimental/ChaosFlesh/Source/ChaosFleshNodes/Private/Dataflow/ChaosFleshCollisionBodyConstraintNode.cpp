// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshCollisionBodyConstraintNode.h"

#include "Engine/SkeletalMesh.h"
#include "GeometryCollection/Facades/CollectionKinematicBindingFacade.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "PhysicsEngine/SphylElem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFleshCollisionBodyConstraintNode)

void FKinematicBodySetupInitializationDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<DataType>(&Collection))
	{
		DataType InCollection = GetValue<DataType>(Context, &Collection);

		if (TManagedArray<FVector3f>* Vertices = InCollection.FindAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup))
		{	
			if (TObjectPtr<USkeletalMesh> SkeletalMesh = GetValue<TObjectPtr<USkeletalMesh>>(Context, &SkeletalMeshIn))
			{
				if (UPhysicsAsset* PhysicsAsset = SkeletalMesh->GetPhysicsAsset())
				{
					TArray<TObjectPtr<USkeletalBodySetup>> SkeletalBodySetups = PhysicsAsset->SkeletalBodySetups;
					TArray<FTransform> ComponentPose;
					UE::Dataflow::Animation::GlobalTransforms(SkeletalMesh->GetRefSkeleton(), ComponentPose);
					for (const TObjectPtr<USkeletalBodySetup>& BodySetup : SkeletalBodySetups)
					{	
						TArray<FKSphylElem> SphylElems = BodySetup->AggGeom.SphylElems;
						int32 BoneIndex = SkeletalMesh->GetRefSkeleton().FindBoneIndex(BodySetup->BoneName);
						if (0 <= BoneIndex && BoneIndex < ComponentPose.Num())
						{
							TArray<int32> BoundVerts;
							TArray<float> BoundWeights;
							for (FKSphylElem Capsule : SphylElems)
							{
								for (int32 i = 0; i < Vertices->Num(); ++i)
								{
									float DistanceToCapsule = Capsule.GetShortestDistanceToPoint(FVector((*Vertices)[i]), Capsule.GetTransform());
									DistanceToCapsule = Capsule.GetShortestDistanceToPoint(FVector((*Vertices)[i]), ComponentPose[BoneIndex]);
									if (DistanceToCapsule < UE_SMALL_NUMBER)
									{
										if (BoundVerts.Find(i) == INDEX_NONE)
										{
											BoundVerts.Add(i);
											BoundWeights.Add(1.0);
										}
									}
								}
							}
							//get local coords of bound verts
							typedef GeometryCollection::Facades::FKinematicBindingFacade FKinematics;
							FKinematics Kinematics(InCollection); Kinematics.DefineSchema();
							if (Kinematics.IsValid())
							{
								FKinematics::FBindingKey Binding = Kinematics.SetBoneBindings(BoneIndex, BoundVerts, BoundWeights);
								TManagedArray<TArray<FVector3f>>& LocalPos = InCollection.AddAttribute<TArray<FVector3f>>("LocalPosition", Binding.GroupName);
								Kinematics.AddKinematicBinding(Binding);

								auto FloatVert = [](FVector3d V) { return FVector3f(V.X, V.Y, V.Z); };
								auto DoubleVert = [](FVector3f V) { return FVector3d(V.X, V.Y, V.Z); };
								LocalPos[Binding.Index].SetNum(BoundVerts.Num());
								for (int32 i = 0; i < BoundVerts.Num(); i++)
								{
									FVector3f Temp = (*Vertices)[BoundVerts[i]];
									LocalPos[Binding.Index][i] = FloatVert(ComponentPose[BoneIndex].InverseTransformPosition(DoubleVert(Temp)));
								}
							}
						}
					}
				}
				GeometryCollection::Facades::FVertexBoneWeightsFacade(InCollection).AddBoneWeightsFromKinematicBindings();
			}
		}
		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}
