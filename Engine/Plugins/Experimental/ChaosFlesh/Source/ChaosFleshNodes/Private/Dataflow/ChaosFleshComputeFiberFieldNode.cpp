// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshComputeFiberFieldNode.h"

#include "Chaos/Math/Poisson.h"
#include "ChaosFlesh/ChaosFlesh.h"
#include "ChaosFlesh/TetrahedralCollection.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "GeometryCollection/Facades/CollectionMuscleActivationFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFleshComputeFiberFieldNode)

#define LOCTEXT_NAMESPACE "DataflowGraphEditor"

void FComputeFiberFieldNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		//
		// Gather inputs
		//
		FManagedArrayCollection InCollection = GetValue(Context, &Collection);
		if (!IsConnected(&OriginIndices) || !IsConnected(&InsertionIndices))
		{
			const FText ErrorMessage = LOCTEXT("IndicesNotConnected", "OriginIndices/InsertionIndices is not connected.");
			Context.Error(ErrorMessage, this, Out);
			SetValue(Context, MoveTemp(InCollection), &Collection);
			return;
		}
		TArray<int32> InOriginIndices = GetValue(Context, &OriginIndices);
		TArray<int32> InInsertionIndices = GetValue(Context, &InsertionIndices);

		// Tetrahedra
		const TManagedArray<FIntVector4>* Elements = InCollection.FindAttributeTyped<FIntVector4>(
			FTetrahedralCollection::TetrahedronAttribute, FTetrahedralCollection::TetrahedralGroup);
		if (!Elements)
		{
			const FText ErrorMessage = FText::Format(
				LOCTEXT("MissingElements", "ComputeFiberFieldNode: Failed to find geometry collection attr '{0}' in group '{1}'"),
				FText::FromName(FTetrahedralCollection::TetrahedronAttribute),
				FText::FromName(FTetrahedralCollection::TetrahedralGroup)
			);
			Context.Error(ErrorMessage, this, Out);
			SetValue(Context, MoveTemp(InCollection), &Collection);
			return;
		}

		// Vertices
		TManagedArray<FVector3f>* Vertex = InCollection.FindAttribute<FVector3f>("Vertex", "Vertices");
		if (!Vertex)
		{
			const FText ErrorMessage = LOCTEXT("NoVertexAttr", "Failed to find geometry collection attr 'Vertex' in group 'Vertices'");
			Context.Error(ErrorMessage, this, Out);
			SetValue(Context, MoveTemp(InCollection), &Collection);
			return;
		}

		// Incident elements
		TManagedArray<TArray<int32>>* IncidentElements = InCollection.FindAttribute<TArray<int32>>(
			FTetrahedralCollection::IncidentElementsAttribute, FGeometryCollection::VerticesGroup);
		if (!IncidentElements)
		{
			const FText ErrorMessage = FText::Format(
				LOCTEXT("MissingIncidentElements", "Failed to find geometry collection attr '{0}' in group '{1}'"),
				FText::FromName(FTetrahedralCollection::IncidentElementsAttribute),
				FText::FromName(FTetrahedralCollection::VerticesGroup)
			); 
			Context.Error(ErrorMessage, this, Out);
			SetValue(Context, MoveTemp(InCollection), &Collection);
			return;
		}
		TManagedArray<TArray<int32>>* IncidentElementsLocalIndex = InCollection.FindAttribute<TArray<int32>>(
			FTetrahedralCollection::IncidentElementsLocalIndexAttribute, FGeometryCollection::VerticesGroup);
		if (!IncidentElementsLocalIndex)
		{
			const FText ErrorMessage = FText::Format(
				LOCTEXT("MissingIncidentElementsIndex", "Failed to find geometry collection attr '{0}' in group '{1}'"),
				FText::FromName(FTetrahedralCollection::IncidentElementsLocalIndexAttribute),
				FText::FromName(FTetrahedralCollection::VerticesGroup)
			);
			Context.Error(ErrorMessage, this, Out);
			SetValue(Context, MoveTemp(InCollection), &Collection);
			return;
		}

		//
		// Pull Origin & Insertion data out of the geometry collection.  We may want other ways of specifying
		// these via an input on the node...
		//

		// Origin & Insertion
		const TManagedArray<int32>* Origin = nullptr;
		const TManagedArray<int32>* Insertion = nullptr;
		if (InOriginIndices.IsEmpty() || InInsertionIndices.IsEmpty())
		{
			// Origin & Insertion group
			if (OriginInsertionGroupName.IsEmpty())
			{
				const FText ErrorMessage = LOCTEXT("OriginNameEmpty", "Attr 'OriginInsertionGroupName' cannot be empty.");
				Context.Error(ErrorMessage, this, Out);
				SetValue(Context, MoveTemp(InCollection), &Collection);
				return;
			}

			// Origin vertices
			if (InOriginIndices.IsEmpty())
			{
				if (OriginVertexFieldName.IsEmpty())
				{
					const FText ErrorMessage = LOCTEXT("OriginVertexNameEmpty", "Attr 'OriginVertexFieldName' cannot be empty.");
					Context.Error(ErrorMessage, this, Out);
					SetValue(Context, MoveTemp(InCollection), &Collection);
					return;
				}
				Origin = InCollection.FindAttribute<int32>(FName(OriginVertexFieldName), FName(OriginInsertionGroupName));
				if (!Origin)
				{
					const FText ErrorMessage = FText::Format(
						LOCTEXT("MissingIncidentElementsIndex", "Failed to find geometry collection attr '{0}' in group '{1}'"),
						FText::FromString(OriginVertexFieldName),
						FText::FromString(OriginInsertionGroupName)
					);
					Context.Error(ErrorMessage, this, Out);
					SetValue(Context, MoveTemp(InCollection), &Collection);
					return;
				}
				else
				{
					InOriginIndices = Origin->GetConstArray();
				}
			}

			// Insertion vertices
			if (InInsertionIndices.IsEmpty())
			{
				if (InsertionVertexFieldName.IsEmpty())
				{
					const FText ErrorMessage = LOCTEXT("InsertionVertexNameEmpty", "Attr 'InsertionVertexFieldName' cannot be empty.");
					Context.Error(ErrorMessage, this, Out);
					SetValue(Context, MoveTemp(InCollection), &Collection);
					return;
				}
				Insertion = InCollection.FindAttribute<int32>(FName(InsertionVertexFieldName), FName(OriginInsertionGroupName));
				if (!Insertion)
				{
					const FText ErrorMessage = FText::Format(
						LOCTEXT("InsertionNameEmpty", "Failed to find geometry collection attr '{0}' in group '{1}'"),
						FText::FromString(InsertionVertexFieldName),
						FText::FromString(OriginInsertionGroupName)
					);
					Context.Error(ErrorMessage, this, Out);
					SetValue(Context, MoveTemp(InCollection), &Collection);
					return;
				}
				else
				{
					InInsertionIndices = Insertion->GetConstArray();
				}
			}
		}

		// Only solve for fiber field on muscle geometries
		TSet<int32> MuscleGeometries;
		GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(InCollection);
		TArray<int32> GeometryIndex = MeshFacade.GetGeometryGroupIndexArray();
		for (int32 OriginIdx : InOriginIndices)
		{
			// Verify origins
			if (!Vertex->IsValidIndex(OriginIdx))
			{
				const FText ErrorMessage = FText::Format(
					LOCTEXT("InvalidOriginVertexIndex", "OriginIdx {0} is not a valid vertex index for vertex group size {1}"),
					FText::AsNumber(OriginIdx),
					FText::AsNumber(Vertex->Num())
				);
				Context.Error(ErrorMessage, this, Out);
				SetValue(Context, MoveTemp(InCollection), &Collection);
				return;
			}
			if (GeometryIndex.IsValidIndex(OriginIdx))
			{
				MuscleGeometries.Add(GeometryIndex[OriginIdx]);
			}
		}
		for (int32 InsertionIdx : InInsertionIndices)
		{
			// Verify insertions
			if (!Vertex->IsValidIndex(InsertionIdx))
			{
				const FText ErrorMessage = FText::Format(
					LOCTEXT("InvalidInsertionVertexIndex", "InsertionIdx {0} is not a valid vertex index for vertex group size {1}"),
					FText::AsNumber(InsertionIdx),
					FText::AsNumber(Vertex->Num())
					);
				Context.Error(ErrorMessage, this, Out);
				SetValue(Context, MoveTemp(InCollection), &Collection);
				return;
			}
			if (GeometryIndex.IsValidIndex(InsertionIdx))
			{
				MuscleGeometries.Add(GeometryIndex[InsertionIdx]);
			}
		}
		TArray<int32> MuscleElementIndices;
		TArray<FIntVector4> MuscleElements;

		TManagedArray<int32>* TetrahedronStart = InCollection.FindAttribute<int32>(
			FTetrahedralCollection::TetrahedronStartAttribute, FTetrahedralCollection::GeometryGroup);
		TManagedArray<int32>* TetrahedronCount = InCollection.FindAttribute<int32>(
			FTetrahedralCollection::TetrahedronCountAttribute, FTetrahedralCollection::GeometryGroup);
		TArray<TArray<int32>> MuscleConstraints;
		if (TetrahedronStart && TetrahedronCount)
		{
			for (int32 GeometryIdx : MuscleGeometries)
			{
				if (TetrahedronStart->IsValidIndex(GeometryIdx))
				{
					for (int32 ElemIdx = (*TetrahedronStart)[GeometryIdx]; ElemIdx < (*TetrahedronStart)[GeometryIdx] + (*TetrahedronCount)[GeometryIdx]; ++ElemIdx)
					{
						MuscleElementIndices.Add(ElemIdx);
						MuscleElements.Add((*Elements)[ElemIdx]);
					}
				}
			}
		}
		MuscleConstraints.SetNum(MuscleElements.Num());
		for (int32 LocalIdx = 0; LocalIdx < MuscleElements.Num(); ++LocalIdx)
		{
			for (int32 Index = 0; Index < 4; ++Index)
			{
				MuscleConstraints[LocalIdx].Add(MuscleElements[LocalIdx][Index]);
			}
		}
		TArray<TArray<int32>> MuscleIncidentElementsLocalIndex;
		TArray<TArray<int32>> MuscleIncidentElements = Chaos::Utilities::ComputeIncidentElements(MuscleConstraints, &MuscleIncidentElementsLocalIndex);

		//
		// Do the thing
		//
		TArray<FVector3f> MuscleFiberDirs;
		TArray<float> MuscleAttachmentScalarFieldTArray; //continuous field where origin = 1, insertion = 2, othernodes = 0
		Chaos::ComputeFiberField<float>(
			MuscleElements,
			Vertex->GetConstArray(),
			MuscleIncidentElements,
			MuscleIncidentElementsLocalIndex,
			InOriginIndices,
			InInsertionIndices,
			MuscleFiberDirs,
			MuscleAttachmentScalarFieldTArray,
			MaxIterations,
			Tolerance);

		//
		// Set output(s)
		//

		TManagedArray<FVector3f>& FiberDirections =
			InCollection.AddAttribute<FVector3f>("FiberDirection", FTetrahedralCollection::TetrahedralGroup);
		FiberDirections.Fill(FVector3f(0, 0, 0));
		for (int32 LocalIdx = 0; LocalIdx < MuscleElements.Num(); ++LocalIdx)
		{
			FiberDirections[MuscleElementIndices[LocalIdx]] = MuscleFiberDirs[LocalIdx];
		}

		if (bShowMuscleColor)
		{
			TManagedArray<FLinearColor>& Color =
				InCollection.AddAttribute<FLinearColor>("Color", FGeometryCollection::VerticesGroup);
			for (int32 Index = 0; Index < Color.Num(); ++Index)
			{
				const float Value = MuscleAttachmentScalarFieldTArray[Index];
				if (1 <= Value && Value <= 2) // 1 <= Value <= 2 if muscle with origin and insertion
				{
					Color[Index] = FLinearColor(FVector(Value - 1, 0, 2 - Value));
				}
			}
		}
		Out->SetValue(MoveTemp(InCollection), Context);
	}
}

TArray<int32>
FComputeFiberFieldNode::GetNonZeroIndices(const TArray<uint8>& Map) const
{
	int32 NumNonZero = 0;
	for (uint8 Value: Map)
	{
		if (Value)
		{
			NumNonZero++;
		}
	}
	TArray<int32> Indices; Indices.AddUninitialized(NumNonZero);
	int32 Idx = 0;
	for (int32 Index = 0; Index < Map.Num(); Index++)
	{
		if (Map[Index])
		{
			Indices[Idx++] = Index;
		}
	}
	return Indices;
}

void FComputeFiberStreamlineNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
	else if (Out->IsA(&VectorField))
	{
		//
		// Gather inputs
		//

		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		FFieldCollection OutVectorField;
		TArray<int32> InOriginIndices = GetValue<TArray<int32>>(Context, &OriginIndices);
		TArray<int32> InInsertionIndices = GetValue<TArray<int32>>(Context, &InsertionIndices);

		//
		// Pull Origin & Insertion data out of the geometry collection.  We may want other ways of specifying
		// these via an input on the node...
		//

		// Origin & Insertion
		const TManagedArray<int32>* Origin = nullptr;
		const TManagedArray<int32>* Insertion = nullptr;
		if (InOriginIndices.IsEmpty() || InInsertionIndices.IsEmpty())
		{
			// Origin & Insertion group
			if (OriginInsertionGroupName.IsEmpty())
			{
				Context.Error(LOCTEXT("OriginInsertionGroupEmpty", "Attr 'OriginInsertionGroupName' cannot be empty."), this, Out);
				SetValue(Context, MoveTemp(OutVectorField), &VectorField);
				return;
			}

			// Origin vertices
			if (InOriginIndices.IsEmpty())
			{
				if (OriginVertexFieldName.IsEmpty())
				{
					Context.Error(LOCTEXT("OriginVertexFieldEmpty", "Attr 'OriginVertexFieldName' cannot be empty."), this, Out);
					SetValue(Context, MoveTemp(OutVectorField), &VectorField);
					return;
				}
				Origin = InCollection.FindAttribute<int32>(FName(OriginVertexFieldName), FName(OriginInsertionGroupName));
				if (!Origin)
				{
					const FText ErrorMessage = FText::Format(
						LOCTEXT("NullOrigin", "Failed to find geometry collection attr '{0}' in group '{1}'"),
						FText::FromString(OriginVertexFieldName),
						FText::FromString(OriginInsertionGroupName));
					Context.Error(ErrorMessage, this, Out);
					SetValue(Context, MoveTemp(OutVectorField), &VectorField);
					return;
				}
			}

			// Insertion vertices
			if (InInsertionIndices.IsEmpty())
			{
				if (InsertionVertexFieldName.IsEmpty())
				{
					Context.Error(LOCTEXT("InsertionVertexFieldNameEmpty", "Attr 'InsertionVertexFieldName' cannot be empty."), this, Out);
					SetValue(Context, MoveTemp(OutVectorField), &VectorField);
					return;
				}
				Insertion = InCollection.FindAttributeTyped<int32>(FName(InsertionVertexFieldName), FName(OriginInsertionGroupName));
				if (!Insertion)
				{
					const FText ErrorMessage = FText::Format(
						LOCTEXT("NullOrigin", "Failed to find geometry collection attr '{0}' in group '{1}'"),
						FText::FromString(InsertionVertexFieldName),
						FText::FromString(OriginInsertionGroupName));
					Context.Error(ErrorMessage, this, Out);
					SetValue(Context, MoveTemp(OutVectorField), &VectorField);
					return;
				}
			}
		}

		InOriginIndices = Origin ? Origin->GetConstArray() : InOriginIndices;
		InInsertionIndices = Insertion ? Insertion->GetConstArray() : InInsertionIndices;
		if (InOriginIndices.Num() == 0 || InInsertionIndices.Num() == 0)
		{
			Context.Warning(LOCTEXT("ZeroIndices", "Origin or Insertion array empty, no fiber to compute"), this, Out);
			SetValue(Context, MoveTemp(OutVectorField), &VectorField);
			return;
		}
		//
		// Compute muscle fiber streamlines
		// Save streamlines to muscle group
		//
		GeometryCollection::Facades::FMuscleActivationFacade MuscleActivation(InCollection);

		TArray<TArray<TArray<FVector3f>>> Streamlines = MuscleActivation.BuildStreamlines(Origin ? Origin->GetConstArray() : InOriginIndices,
			Insertion ? Insertion->GetConstArray() : InInsertionIndices, NumLinesMultiplier, MaxStreamlineIterations, MaxPointsPerLine);

		//Render streamlines
		for (int32 i = 0; i < Streamlines.Num(); ++i)
		{
			for (int32 j = 0; j < Streamlines[i].Num(); ++j)
			{
				for (int32 k = 1; k < Streamlines[i][j].Num(); ++k)
				{
					OutVectorField.AddVectorToField(Streamlines[i][j][k - 1], Streamlines[i][j][k]);
				}
			}
		}

		//
		// Set output(s)
		//
		SetValue(Context, MoveTemp(OutVectorField), &VectorField);
	}
}

#undef LOCTEXT_NAMESPACE
