// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshComputeIslandsNode.h"

#include "Chaos/Utilities.h"
#include "ChaosFlesh/TetrahedralCollection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFleshComputeIslandsNode)

void FComputeIslandsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		TManagedArray<int32>& ParticleComponentIndex = InCollection.AddAttribute<int32>("ComponentIndex", FGeometryCollection::VerticesGroup);
		TManagedArray<FIntVector4>* Elements = InCollection.FindAttribute<FIntVector4>(
			FTetrahedralCollection::TetrahedronAttribute, FTetrahedralCollection::TetrahedralGroup);


		if (Elements)
		{

			int32 VertsNum = InCollection.NumElements(FGeometryCollection::VerticesGroup);
			int32 TetsNum = InCollection.NumElements(FTetrahedralCollection::TetrahedralGroup);
			if (VertsNum && TetsNum)
			{
				TArray<TArray<int32>> ConnectedComponents;
				Chaos::Utilities::FindConnectedRegions(Elements->GetConstArray(), ConnectedComponents);
				TManagedArray<int32>& ComponentIndex = InCollection.ModifyAttribute<int32>("ComponentIndex", FGeometryCollection::VerticesGroup);

				ComponentIndex.Fill(INDEX_NONE); //Isolated points will get index -1 

				for (int32 i = 0; i < ConnectedComponents.Num(); i++)
				{
					for (int32 j = 0; j < ConnectedComponents[i].Num(); j++)
					{
						int32 ElementIndex = ConnectedComponents[i][j];
						for (int32 ie = 0; ie < 4; ie++)
						{
							int32 ParticleIndex = (*Elements)[ElementIndex][ie];
							if (ComponentIndex[ParticleIndex] == INDEX_NONE)
							{
								ComponentIndex[ParticleIndex] = i;
							}
						}
					}
				}
			}
		}
		
		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}
