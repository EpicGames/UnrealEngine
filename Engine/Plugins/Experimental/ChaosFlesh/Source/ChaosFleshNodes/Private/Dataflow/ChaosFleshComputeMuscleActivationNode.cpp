// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshComputeMuscleActivationNode.h"

#include "AnimationRuntime.h"
#include "Animation/AnimSequence.h"
#include "Animation/AttributesRuntime.h"
#include "Animation/Skeleton.h"
#include "BonePose.h"
#include "ChaosFlesh/ChaosFlesh.h"
#include "ChaosFlesh/TetrahedralCollection.h"
#include "Chaos/Curve.h"
#include "Dataflow/DataflowSelection.h"
#include "Engine/SkeletalMesh.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "GeometryCollection/Facades/CollectionMuscleActivationFacade.h"
#include "GeometryCollection/Facades/CollectionTransformFacade.h"
#include "GeometryCollection/Facades/CollectionTransformSourceFacade.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFleshComputeMuscleActivationNode)

void FComputeMuscleActivationDataNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace GeometryCollection::Facades;
	if (Out->IsA(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue(Context, &Collection);
		const TArray<int32>& InOriginIndices = GetValue(Context, &OriginIndicesIn);
		const TArray<int32>& InInsertionIndices = GetValue(Context, &InsertionIndicesIn);
		
		FMuscleActivationFacade MuscleActivation(InCollection);
		MuscleActivation.SetUpMuscleActivation(InOriginIndices, InInsertionIndices, ContractionVolumeScale);
		Out->SetValue(MoveTemp(InCollection), Context);
	}
}

void FComputeMuscleActivationDataNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace GeometryCollection::Facades;
	if (Out->IsA(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue(Context, &Collection);
		TArray<int32> InOriginIndices = GetValue(Context, &OriginIndicesIn);
		TArray<int32> InInsertionIndices = GetValue(Context, &InsertionIndicesIn);

		FMuscleActivationFacade MuscleActivation(InCollection);
		if (!MuscleActivation.SetUpMuscleActivation(InOriginIndices, InInsertionIndices))
		{
			Context.Warning(TEXT("Setup failed, please check the Log for more info."), this, Out);
		}
		Out->SetValue(MoveTemp(InCollection), Context);
	}
}

bool FindLowestMuscleLengthRatio(const FManagedArrayCollection& InCollection, const UAnimSequence& InAnimationAsset, 
	const USkeletalMesh& InSkeletalMesh, TArray<float>& MinLengthRatio)
{
#if WITH_EDITOR
	using namespace GeometryCollection::Facades;
	const TManagedArray<FVector3f>* Vertex = InCollection.FindAttributeTyped<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
	if (!Vertex)
	{
		UE_LOGF(LogChaosFlesh, Error, "FindLowestMuscleLengthRatio: No Vertex attribute in the Collection.");
		return false;
	}
	// Match transform source skeleton with SkeletalMesh
	FTransformSource TransformSource(InCollection);
	if (!TransformSource.IsValid())
	{
		UE_LOGF(LogChaosFlesh, Error, "FindLowestMuscleLengthRatio: TransformSource group in the Collection is not valid.");
		return false;
	}
	TSet<int32> Roots;
	const USkeleton* Skeleton = InSkeletalMesh.GetSkeleton();
	const FReferenceSkeleton& ReferenceSkeleton = InSkeletalMesh.GetRefSkeleton();
	if (Skeleton)
	{
		Roots = TransformSource.GetTransformSource(Skeleton->GetName(), Skeleton->GetGuid().ToString(), InSkeletalMesh.GetName());
		if (Roots.IsEmpty())
		{
			UE_LOGF(LogChaosFlesh, Error, "FindLowestMuscleLengthRatio: The root in the TransformSource group is incompatible with the SkeletalMesh.");
			return false;
		}
		else
		{
			ensureMsgf(Roots.Num() == 1, TEXT("Only supports a single root per skeleton.(%s)"), *Skeleton->GetName());
		}
	}
	else
	{
		UE_LOGF(LogChaosFlesh, Error, "FindLowestMuscleLengthRatio: There's no skeleton in the SkeletalMesh.");
		return false;
	}
	const int32 RootTransformOffset = Roots.Array()[0];
	auto RootShift = [&RootTransformOffset](int32 Transform) { return Transform - RootTransformOffset; };
	// Rest transforms
	FCollectionTransformFacade TransformFacade(InCollection);
	TArray<FTransform> RestTransforms = TransformFacade.ComputeCollectionSpaceTransforms();

	// For extracting pose transforms
	FMemMark Mark(FMemStack::Get());
	const int32 NumBones = ReferenceSkeleton.GetNum();
	TArray<FBoneIndexType> BoneIndices;
	BoneIndices.SetNumUninitialized(NumBones);
	for (int32 Index = 0; Index < NumBones; ++Index)
	{
		int32 SkeletonBoneIndex = Skeleton->GetReferenceSkeleton().FindBoneIndex(ReferenceSkeleton.GetBoneName(Index));
		BoneIndices[Index] = StaticCast<FBoneIndexType>(SkeletonBoneIndex);
	}

	FBoneContainer BoneContainer;
	BoneContainer.SetUseRAWData(true);
	BoneContainer.InitializeTo(BoneIndices, UE::Anim::FCurveFilterSettings(), *Skeleton);

	FCompactPose CompactPose;
	CompactPose.SetBoneContainer(&BoneContainer);

	FBlendedCurve BlendedCurve;
	BlendedCurve.InitFrom(BoneContainer);

	UE::Anim::FStackAttributeContainer TempAttributes;
	FAnimationPoseData AnimationPoseData(CompactPose, BlendedCurve, TempAttributes);

	// Prepare kinematic origin insertion weights
	FVertexBoneWeightsFacade WeightsFacade(InCollection);
	if (!WeightsFacade.IsValid())
	{
		UE_LOGF(LogChaosFlesh, Error, "FindLowestMuscleLengthRatio: No kinematics information in the Collection.");
		return false;
	}
	const FMuscleActivationFacade MuscleActivation(InCollection);
	if (!MuscleActivation.IsValid())
	{
		UE_LOGF(LogChaosFlesh, Error, "FindLowestMuscleLengthRatio: No muscle activation information in the Collection.");
		return false;
	}
	const int32 NumMuscles = MuscleActivation.NumMuscles();
	TArray<TArray<int32>> OriginBoneIndices, InsertionBoneIndices;
	TArray<TArray<float>> OriginBoneWeights, InsertionBoneWeights;
	TArray<FVector3d> OriginPosition, InsertionPosition;
	TArray<float> OIRestLength;
	OriginBoneIndices.SetNum(NumMuscles);
	InsertionBoneIndices.SetNum(NumMuscles);
	OriginBoneWeights.SetNum(NumMuscles);
	InsertionBoneWeights.SetNum(NumMuscles);
	OriginPosition.SetNum(NumMuscles);
	InsertionPosition.SetNum(NumMuscles);
	OIRestLength.SetNum(NumMuscles);
	MinLengthRatio.Init(FLT_MAX, NumMuscles);
	auto ChaosVert = [](FVector3f V) { return Chaos::FVec3(V.X, V.Y, V.Z); };
	auto DoubleVert = [](FVector3f V) { return FVector3d(V.X, V.Y, V.Z); };
	for (int32 MuscleIdx = 0; MuscleIdx < NumMuscles; ++MuscleIdx)
	{
		const FMuscleActivationData MuscleActivationData = MuscleActivation.GetMuscleActivationData(MuscleIdx);
		const int32 OriginIdx = MuscleActivationData.OriginInsertionPair[0];
		const int32 InsertionIdx = MuscleActivationData.OriginInsertionPair[1];
		if (ensureMsgf(WeightsFacade.GetBoneIndices().IsValidIndex(OriginIdx), TEXT("Origin index %d is not a valid index in WeightsFacade BoneIndices"), OriginIdx) &&
			ensureMsgf(WeightsFacade.GetBoneIndices().IsValidIndex(InsertionIdx), TEXT("Insertion index %d is not a valid index in WeightsFacade BoneIndices"), InsertionIdx) &&
			ensureMsgf(Vertex->IsValidIndex(OriginIdx), TEXT("Origin index %d is not a valid index in Vertex"), OriginIdx) &&
			ensureMsgf(Vertex->IsValidIndex(InsertionIdx), TEXT("Insertion index %d is not a valid index in Vertex"), InsertionIdx))
		{
			OriginBoneIndices[MuscleIdx] = WeightsFacade.GetBoneIndices()[OriginIdx];
			InsertionBoneIndices[MuscleIdx] = WeightsFacade.GetBoneIndices()[InsertionIdx];
			OriginBoneWeights[MuscleIdx] = WeightsFacade.GetBoneWeights()[OriginIdx];
			InsertionBoneWeights[MuscleIdx] = WeightsFacade.GetBoneWeights()[InsertionIdx];
			OriginPosition[MuscleIdx] = DoubleVert((*Vertex)[OriginIdx]);
			InsertionPosition[MuscleIdx] = DoubleVert((*Vertex)[InsertionIdx]);
			OIRestLength[MuscleIdx] = ((*Vertex)[OriginIdx] - (*Vertex)[InsertionIdx]).Size();
		}
	}

	for (int32 Frame = 0; Frame < InAnimationAsset.GetNumberOfSampledKeys(); ++Frame)
	{
		const FAnimExtractContext ExtractionContext(double(InAnimationAsset.GetTimeAtFrame(Frame)));
		InAnimationAsset.GetAnimationPose(AnimationPoseData, ExtractionContext);
		TArray<FTransform> ComponentSpaceTransforms;
		FAnimationRuntime::FillUpComponentSpaceTransforms(ReferenceSkeleton, AnimationPoseData.GetPose().GetBones(), ComponentSpaceTransforms);

		for (int32 MuscleIdx = 0; MuscleIdx < NumMuscles; ++MuscleIdx)
		{
			FVector3d OriginPos(0.f), InsertionPos(0.f);
			for (int32 InfluenceIdx = 0; InfluenceIdx < OriginBoneIndices[MuscleIdx].Num(); ++InfluenceIdx)
			{
				const int32 OBoneIdx = OriginBoneIndices[MuscleIdx][InfluenceIdx];
				const float OBoneWeight = OriginBoneWeights[MuscleIdx][InfluenceIdx];
				const int32 ShiftedOBoneIdx = RootShift(OBoneIdx);
				if (RestTransforms.IsValidIndex(OBoneIdx) &&
					ComponentSpaceTransforms.IsValidIndex(ShiftedOBoneIdx))
				{
					OriginPos += ComponentSpaceTransforms[ShiftedOBoneIdx].TransformPosition(RestTransforms[OBoneIdx].InverseTransformPosition(OriginPosition[MuscleIdx])) * OBoneWeight;
				}
			}
			for (int32 InfluenceIdx = 0; InfluenceIdx < InsertionBoneIndices[MuscleIdx].Num(); ++InfluenceIdx)
			{
				const int32 IBoneIdx = InsertionBoneIndices[MuscleIdx][InfluenceIdx];
				const float IBoneWeight = InsertionBoneWeights[MuscleIdx][InfluenceIdx];
				const int32 ShiftedIBoneIdx = RootShift(IBoneIdx);
				if (RestTransforms.IsValidIndex(IBoneIdx) &&
					ComponentSpaceTransforms.IsValidIndex(ShiftedIBoneIdx))
				{
					InsertionPos += ComponentSpaceTransforms[ShiftedIBoneIdx].TransformPosition(RestTransforms[IBoneIdx].InverseTransformPosition(InsertionPosition[MuscleIdx])) * IBoneWeight;
				}
			}
			float Ratio = (OriginPos - InsertionPos).Size() / OIRestLength[MuscleIdx];
			MinLengthRatio[MuscleIdx] = FMath::Min(MinLengthRatio[MuscleIdx], Ratio);
		}
	}
	return true;
#else
	return false;
#endif
}

FSetMuscleActivationParameterNode::FSetMuscleActivationParameterNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
	, ApplyGlobalParameters(
		FDataflowFunctionProperty::FDelegate::CreateLambda([this](UE::Dataflow::FContext& Context)
			{
				// triggers node invalidation
				ParameterMethod = EParameterMethod::Global;
			}))
	, ImportLowestMuscleLengthRatio(
		FDataflowFunctionProperty::FDelegate::CreateLambda([this](UE::Dataflow::FContext& Context)
			{
				TObjectPtr<UAnimSequence> InAnimationAsset = GetValue(Context, &AnimationAsset);
				TObjectPtr<USkeletalMesh> InSkeletalMesh = GetValue(Context, &SkeletalMesh);
				if (InAnimationAsset && InSkeletalMesh)
				{
					const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
					GeometryCollection::Facades::FMuscleActivationFacade MuscleActivation(InCollection);
					TArray<float> MinLengthRatio;
					auto ScaledMinLength = [this](float Length) {
						return 1.f - (1.f - Length) * ThresholdScalingPercent / 100.f;
					};
					
					if (FindLowestMuscleLengthRatio(InCollection, *InAnimationAsset, *InSkeletalMesh, MinLengthRatio)) //success
					{
						const int32 NumMuscles = MinLengthRatio.Num();
						for (int32 Idx = 0; Idx < ParameterArray.Num(); ++Idx)
						{
							const FString MuscleName = ParameterArray[Idx].MuscleName;
							const int32 MuscleIndex = MuscleActivation.FindMuscleIndexByName(MuscleName);
							if (MinLengthRatio.IsValidIndex(MuscleIndex))
							{
								const float ScaledRatio = ScaledMinLength(MinLengthRatio[MuscleIndex]);
								if (ScaledRatio > 0 && ScaledRatio < 1)
								{
									ParameterArray[Idx].MuscleLengthRatioThresholdForMaxActivation = ScaledRatio;
								}
								else
								{
									if (MinLengthRatio[MuscleIndex] > 0 && MinLengthRatio[MuscleIndex] < 1)
									{
										UE_LOGF(LogChaosFlesh, Error,
											"SetMuscleActivationParameter::ImportLowestMuscleLengthRatio: Muscle [%ls] index [%d] has minimum origin-insertion length ratio %.2f (scaled to %.2f) across the whole animation.",
											*MuscleName, MuscleIndex, MinLengthRatio[MuscleIndex], ScaledRatio);
									}
									else
									{
										UE_LOGF(LogChaosFlesh, Warning,
											"SetMuscleActivationParameter::ImportLowestMuscleLengthRatio: Muscle [%ls] index [%d] has minimum origin-insertion length ratio %.2f across the whole animation.",
											*MuscleName, MuscleIndex, MinLengthRatio[MuscleIndex]);
									}
								}
							}
							else
							{
								UE_LOGF(LogChaosFlesh, Error,
									"SetMuscleActivationParameter::ImportLowestMuscleLengthRatio: Geometry [%ls] is not a valid muscle.",
									*MuscleName);
								Context.Error(FString::Printf(
									TEXT("ImportLowestMuscleLengthRatio: Geometry [%s] is not a valid muscle."),
									*MuscleName),
									this);
							}
						}
					}
					else
					{
						Context.Error(TEXT("FindLowestMuscleLengthRatio failed, please check the Log for more info."), this);
					}
				}
			}))
	, ImportAllMuscleNames(
		FDataflowFunctionProperty::FDelegate::CreateLambda([this](UE::Dataflow::FContext& Context)
			{
				using namespace GeometryCollection::Facades;
				const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
				const FMuscleActivationFacade MuscleActivation(InCollection);
				ParameterArray.SetNum(MuscleActivation.NumMuscles());
				for (int32 MuscleIdx = 0; MuscleIdx < MuscleActivation.NumMuscles(); ++MuscleIdx)
				{
					ParameterArray[MuscleIdx].MuscleName = MuscleActivation.FindMuscleName(MuscleIdx);
					// load existing attributes
					if (MuscleActivation.IsValid())
					{
						const FMuscleActivationData MuscleActivationData = MuscleActivation.GetMuscleActivationData(MuscleIdx);
						ParameterArray[MuscleIdx].ContractionVolumeScale = MuscleActivationData.ContractionVolumeScale.Num() ? MuscleActivationData.ContractionVolumeScale[0] : ContractionVolumeScale;
						ParameterArray[MuscleIdx].FiberLengthRatioAtMaxActivation = MuscleActivationData.FiberLengthRatioAtMaxActivation;
						ParameterArray[MuscleIdx].MuscleLengthRatioThresholdForMaxActivation = MuscleActivationData.MuscleLengthRatioThresholdForMaxActivation;
						ParameterArray[MuscleIdx].InflationVolumeScale = MuscleActivationData.InflationVolumeScale;
					}
					else
					{
						ParameterArray[MuscleIdx].ContractionVolumeScale = ContractionVolumeScale;
						ParameterArray[MuscleIdx].FiberLengthRatioAtMaxActivation = GlobalFiberLengthRatioAtMaxActivation;
						ParameterArray[MuscleIdx].MuscleLengthRatioThresholdForMaxActivation = GlobalMuscleLengthRatioThresholdForMaxActivation;
						ParameterArray[MuscleIdx].InflationVolumeScale = GlobalInflationVolumeScale;
					}
				}
			}))
	, ResetToGlobalParameters(
		FDataflowFunctionProperty::FDelegate::CreateLambda([this](UE::Dataflow::FContext& Context)
			{
				for (int32 Idx = 0; Idx < ParameterArray.Num(); ++Idx)
				{
					ParameterArray[Idx].ContractionVolumeScale = ContractionVolumeScale;
					ParameterArray[Idx].FiberLengthRatioAtMaxActivation = GlobalFiberLengthRatioAtMaxActivation;
					ParameterArray[Idx].MuscleLengthRatioThresholdForMaxActivation = GlobalMuscleLengthRatioThresholdForMaxActivation;
					ParameterArray[Idx].InflationVolumeScale = GlobalInflationVolumeScale;
				}
			}))
	, ApplyCustomParameters(
		FDataflowFunctionProperty::FDelegate::CreateLambda([this](UE::Dataflow::FContext& Context)
			{
				// triggers node invalidation
				ParameterMethod = EParameterMethod::Custom;
			}))
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&AnimationAsset)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&SkeletalMesh)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterOutputConnection(&Collection, &Collection);
	// Set default linear curve
	FLengthActivationUtils::SetDefaultLengthActivationCurve(GlobalLengthActivationCurve);
}


void FSetMuscleActivationParameterNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace GeometryCollection::Facades;
	if (Out->IsA(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue(Context, &Collection);
		if (IsConnected(&Collection))
		{
			FMuscleActivationFacade MuscleActivation(InCollection);
			MuscleActivation.UpdateGlobalMuscleActivationParameters(
					ContractionVolumeScale,
					GlobalFiberLengthRatioAtMaxActivation,
					GlobalMuscleLengthRatioThresholdForMaxActivation,
					GlobalInflationVolumeScale);
			if (bUseLengthActivationCurve)
			{
				Chaos::FLinearCurve ChaosCurve;
				GlobalLengthActivationCurve.GetRichCurveConst()->ConvertToChaosCurve(ChaosCurve);
				MuscleActivation.UpdateGlobalLengthActivationCurve(ChaosCurve);
			}
			if (ParameterMethod == EParameterMethod::Custom)
			{
				// match muscle names and override parameters
				for (int32 Idx = 0; Idx < ParameterArray.Num(); ++Idx)
				{
					FPerMuscleParameter Params = ParameterArray[Idx];
					int32 MuscleIndex = MuscleActivation.FindMuscleIndexByName(Params.MuscleName);
					if (MuscleActivation.IsValidMuscleIndex(MuscleIndex))
					{
						MuscleActivation.UpdateMuscleActivationParameters(
							MuscleIndex,
							Params.ContractionVolumeScale,
							Params.FiberLengthRatioAtMaxActivation,
							Params.MuscleLengthRatioThresholdForMaxActivation,
							Params.InflationVolumeScale);
						if (Params.bUseLengthActivationCurve)
						{
							Chaos::FLinearCurve ChaosCurve;
							Params.LengthActivationCurve.GetRichCurveConst()->ConvertToChaosCurve(ChaosCurve);
							MuscleActivation.UpdateLengthActivationCurve(MuscleIndex, ChaosCurve);
						}
					}
				}
			}
		}
		Out->SetValue(MoveTemp(InCollection), Context);
	}
}

bool FSetMuscleActivationParameterNode::ShouldInvalidateOnPropertyChanged(const FPropertyChangedEvent& InPropertyChangedEvent) const
{
	const FName ChangedPropertyName = InPropertyChangedEvent.GetMemberPropertyName();
	if (ChangedPropertyName == GET_MEMBER_NAME_CHECKED(FSetMuscleActivationParameterNode, ParameterArray) ||
		ChangedPropertyName == GET_MEMBER_NAME_CHECKED(FSetMuscleActivationParameterNode, ImportAllMuscleNames) ||
		ChangedPropertyName == GET_MEMBER_NAME_CHECKED(FSetMuscleActivationParameterNode, ResetToGlobalParameters) ||
		ChangedPropertyName == GET_MEMBER_NAME_CHECKED(FSetMuscleActivationParameterNode, ThresholdScalingPercent) ||
		ChangedPropertyName == GET_MEMBER_NAME_CHECKED(FSetMuscleActivationParameterNode, ImportLowestMuscleLengthRatio))
	{
		return false;
	}
	return Super::ShouldInvalidateOnPropertyChanged(InPropertyChangedEvent);
}

FReadSkeletalMeshCurvesDataflowNode::FReadSkeletalMeshCurvesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
	, ImportSKMCurveNames(
		FDataflowFunctionProperty::FDelegate::CreateLambda([this](UE::Dataflow::FContext& Context)
			{
				CurveMuscleNameArray.Empty();
				if (TObjectPtr<const USkeletalMesh> InSkeletalMesh = GetValue(Context, &SkeletalMesh))
				{
					for (const UAssetUserData* AssetUserData : *InSkeletalMesh->GetAssetUserDataArray())
					{
						if (const UAnimCurveMetaData* AnimCurveMetaData = Cast<UAnimCurveMetaData>(AssetUserData))
						{
							TArray<FName> CurveNamesArray;
							AnimCurveMetaData->GetCurveMetaDataNames(CurveNamesArray);
							CurveNamesArray.Sort([](const FName& A, const FName& B)
							{
								return A.ToString() < B.ToString();
							});

							for (FName Curvename : CurveNamesArray)
							{
								CurveMuscleNameArray.Add(FCurveMuscleName(Curvename.ToString(), FString()));
							}
						}
					}
				}
			}))
	, AssignSKMCurveToMuscle(
		FDataflowFunctionProperty::FDelegate::CreateLambda([this](UE::Dataflow::FContext& Context)
			{
				// just triggers node invalidation
			}))
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&SkeletalMesh);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&GeometrySelection);
}

void FReadSkeletalMeshCurvesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace GeometryCollection::Facades;
	if (Out->IsA(&Collection) || Out->IsA(&GeometrySelection))
	{
		FManagedArrayCollection InCollection = GetValue(Context, &Collection);
		FMuscleActivationFacade MuscleActivationFacade(InCollection);
		TArray<int32> MuscleCurveGeometry;
		if (MuscleActivationFacade.IsValid())
		{
			for (const FCurveMuscleName& CurveMuscleName : CurveMuscleNameArray)
			{
				const int32 MuscleIdx = MuscleActivationFacade.AssignCurveName(CurveMuscleName.CurveName, CurveMuscleName.MuscleName);
				if (MuscleIdx != INDEX_NONE)
				{
					MuscleCurveGeometry.Add(MuscleActivationFacade.FindMuscleGeometryIndex(MuscleIdx));
				}
				else
				{
					UE_LOGF(LogChaosFlesh, Error,
						"ReadSkeletalMeshCurves: Geometry %ls (connecting to curve %ls) is not an active muscle.",
						*CurveMuscleName.MuscleName, *CurveMuscleName.CurveName);
				}
			}
		}
		FDataflowGeometrySelection OutGeometrySelection;
		OutGeometrySelection.Initialize(InCollection.NumElements(FGeometryCollection::GeometryGroup), false);
		OutGeometrySelection.SetFromArray(MuscleCurveGeometry);
		SetValue(Context, MoveTemp(InCollection), &Collection);
		SetValue(Context, MoveTemp(OutGeometrySelection), &GeometrySelection);
	}
}

bool FReadSkeletalMeshCurvesDataflowNode::ShouldInvalidateOnPropertyChanged(const FPropertyChangedEvent& InPropertyChangedEvent) const
{
	const FName ChangedPropertyName = InPropertyChangedEvent.GetMemberPropertyName();
	if (ChangedPropertyName == GET_MEMBER_NAME_CHECKED(FReadSkeletalMeshCurvesDataflowNode, CurveMuscleNameArray) ||
		ChangedPropertyName == GET_MEMBER_NAME_CHECKED(FReadSkeletalMeshCurvesDataflowNode, ImportSKMCurveNames))
	{
		return false;
	}
	return Super::ShouldInvalidateOnPropertyChanged(InPropertyChangedEvent);
}
