// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshCreateTetrahedronNode.h"

#include "Async/ParallelFor.h"
#include "Chaos/Deformable/Utilities.h"
#include "ChaosFlesh/ChaosFlesh.h"
#include "Chaos/Tetrahedron.h"
#include "Chaos/Utilities.h"
#include "Chaos/UniformGrid.h"
#include "ChaosFlesh/FleshCollection.h"
#include "ChaosFlesh/FleshCollectionUtility.h"
#include "ChaosFlesh/ChaosFleshCollectionFacade.h"
#include "ChaosLog.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Dataflow/ChaosFleshNodesUtility.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "FTetWildWrapper.h"
#include "Generate/IsosurfaceStuffing.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/Facades/CollectionTetrahedralMetricsFacade.h"
#include "GeometryCollection/Facades/CollectionTransformFacade.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "MeshDescription.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "Rendering/SkeletalMeshLODImporterData.h"
#include "Rendering/SkeletalMeshModel.h"
#include "SkeletalMeshLODModelToDynamicMesh.h" // MeshModelingBlueprints
#include "Spatial/FastWinding.h"
#include "Spatial/MeshAABBTree3.h"
#include "Dataflow/ChaosFleshNodesUtility.h"

//=============================================================================
// FCreateTetrahedronDataflowNode
//=============================================================================

void FCreateTetrahedronDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<DataType>(&Collection))
	{
		TUniquePtr<FFleshCollection> InCollectionVal(GetValue<DataType>(Context, &Collection).NewCopy<FFleshCollection>());
		Chaos::FFleshCollectionFacade Target(*InCollectionVal.Get());

		TUniquePtr<FFleshCollection> InSourceCollectionVal(GetValue<DataType>(Context, &SourceCollection).NewCopy<FFleshCollection>());
		Chaos::FFleshCollectionFacade Source(*InSourceCollectionVal.Get());

		if (Source.IsValid() && Target.IsValid())
		{
			auto GetParentIndex = [&Target, &Source](int32 Gdx)
			{
				if (Gdx >= 0)
				{
					if (Target.IsHierarchyValid())
					{
						if (Gdx < Source.GeometryToTransformIndex.Num() && Gdx < Target.GeometryToTransformIndex.Num())
						{
							int32 SrcTdx = Source.GeometryToTransformIndex[Gdx], Tdx = Target.GeometryToTransformIndex[Gdx];
							if (0 <= SrcTdx && SrcTdx < Source.BoneName.Num() && 0 <= Tdx && Tdx < Target.BoneName.Num())
							{
								if (Source.BoneName[SrcTdx].Equals(Target.BoneName[Tdx]))
								{
									return Target.Parent[Tdx];
								}
							}
						}
					}
				}
				return (int32)INDEX_NONE;
			};

			TArray<int32> ProcessGeometryIndices = UE::Dataflow::GetMatchingMeshIndices(Selection, InSourceCollectionVal.Get());

			TArray<TUniquePtr<FFleshCollection>> CollectionBuffer;
			for (int32 Gdx = 0; Gdx < Source.NumGeometry(); Gdx++)
			{
				if (ProcessGeometryIndices.Contains(Gdx))
				{
					CollectionBuffer.Add(TUniquePtr<FFleshCollection>(new FFleshCollection()));
				}
				else 
					CollectionBuffer.Add(nullptr);
			}

			TArray<FTransform> ComponentTransform;
			Source.GlobalMatrices(ComponentTransform);

			ParallelFor(ProcessGeometryIndices.Num(), [&](int32 i)
			{
				int32 Gdx = ProcessGeometryIndices[i];
			    int32 Tdx = Source.GeometryToTransformIndex[Gdx];
				FFleshCollection& TetCollection = *CollectionBuffer[Gdx];

				UE::Geometry::FDynamicMesh3 DynamicMesh;
				int32 vStart = Source.VertexStart[Gdx], vEnd = vStart + Source.VertexCount[Gdx];
				for (int Vdx = vStart; Vdx < vEnd; Vdx++) DynamicMesh.AppendVertex(ComponentTransform[Tdx].TransformPosition(FVector(Source.Vertex[Vdx])));
				int32 fStart = Source.FaceStart[Gdx], fEnd = fStart + Source.FaceCount[Gdx];
				for (int Fdx = fStart; Fdx < fEnd; Fdx++) DynamicMesh.AppendTriangle(Source.Indices[Fdx] - FIntVector(vStart));
				DynamicMesh.CompactInPlace();

				if (Method == TetMeshingMethod::IsoStuffing)
				{
					EvaluateIsoStuffing(Context, TetCollection, DynamicMesh);
				}
				else if (Method == TetMeshingMethod::TetWild)
				{
					EvaluateTetWild(Context, TetCollection, DynamicMesh);
				}

				if (TetCollection.NumElements(FGeometryCollection::VerticesGroup))
				{
					TSet<int32> VertexToDeleteSet;
					GeometryCollectionAlgo::ComputeStaleVertices(&TetCollection, VertexToDeleteSet);
					TArray<int32> SortedVertices = VertexToDeleteSet.Array(); SortedVertices.Sort();
					if (VertexToDeleteSet.Num()) TetCollection.RemoveElements(FGeometryCollection::VerticesGroup, SortedVertices);
				}
			}, ProcessGeometryIndices.Num() < 2? EParallelForFlags::ForceSingleThread : EParallelForFlags::None);

			auto AppendProcessedGeometry = [&ProcessGeometryIndices, &CollectionBuffer, &Source, &Target]()
			{
				// find the index in the to transform group based on name
				auto FindParentIndexFromName = [&Target](FString fpName)
				{
					if (!fpName.IsEmpty())
					{
						for (int32 i = 0; i < Target.BoneName.Num(); i++) if (Target.BoneName[i].Equals(fpName)) return i;
					}
					return (int32)INDEX_NONE;
				};

				for (int32 Sdx = 0; Sdx < ProcessGeometryIndices.Num(); Sdx++)
				{
					int32 Gdx = ProcessGeometryIndices[Sdx];
					if (CollectionBuffer[Gdx] && CollectionBuffer[Gdx]->NumElements(FGeometryCollection::GeometryGroup))
					{
						int32 GeomIndex = Target.AppendGeometry(*CollectionBuffer[Gdx]);
						// source data
						int32 SourceNumTransforms = Source.NumTransforms();
						int32 SourceTransformIndex = Source.GeometryToTransformIndex[Gdx];
						int32 SourceTransformParent = (0 <= SourceTransformIndex && SourceTransformIndex < SourceNumTransforms) ? Source.Parent[SourceTransformIndex] : INDEX_NONE;
						FString SourceParentName = (0 <= SourceTransformParent && SourceTransformParent < SourceNumTransforms) ? Source.BoneName[SourceTransformParent] : FString("");
						// target data
						int32 ToNumTransforms = Target.NumTransforms();
						int32 ToGeomTransformIndex = Target.GeometryToTransformIndex[GeomIndex];

						FString TetName = FString::Printf(TEXT("Tet%d"), GeomIndex);
						if (0 <= SourceTransformIndex && SourceTransformIndex < SourceNumTransforms)
						{
							if (!Source.BoneName[SourceTransformIndex].IsEmpty())
							{
								TetName = FString::Printf(TEXT("%s_%s"), *Source.BoneName[SourceTransformIndex], *TetName);
							}
						}
						if (0 <= ToGeomTransformIndex && ToGeomTransformIndex < ToNumTransforms)
						{
							// set the name
							Target.BoneName.ModifyAt(ToGeomTransformIndex,TetName);

							// set transform to geometry and geometry to transform mappings
							Target.TransformToGeometryIndex.ModifyAt(ToGeomTransformIndex, GeomIndex);
							Target.GeometryToTransformIndex.ModifyAt(GeomIndex, ToGeomTransformIndex);
							// set the parent and child mappings
							Target.Parent.ModifyAt(ToGeomTransformIndex,FindParentIndexFromName(SourceParentName));
							if (Target.Parent[ToGeomTransformIndex] != INDEX_NONE)
							{
								Target.Child.Modify()[Target.Parent[ToGeomTransformIndex]].Add(ToGeomTransformIndex);
							}

							if (ToGeomTransformIndex != INDEX_NONE)
							{
								int32 VertexEnd = Target.VertexStart[GeomIndex] + Target.VertexCount[GeomIndex];
								FTransform3f ParentTransform = Target.GlobalMatrix3f(ToGeomTransformIndex);
								for (int Vdx = Target.VertexStart[GeomIndex]; Vdx < VertexEnd; Vdx++)
								{
									Target.Vertex.ModifyAt(Vdx,ParentTransform.InverseTransformPosition(Target.Vertex[Vdx]));
								}
							}
						}
					}
				}
			};
			AppendProcessedGeometry();

			GeometryCollection::Facades::FCollectionTransformFacade(*InCollectionVal.Get()).EnforceSingleRoot("root");
		}
		SetValue<const DataType&>(Context, *InCollectionVal, &Collection);
	}
}

void FCreateTetrahedronDataflowNode::EvaluateIsoStuffing(
	UE::Dataflow::FContext& Context, 
	FFleshCollection& InCollection,
	const UE::Geometry::FDynamicMesh3& DynamicMesh) const
{
#if WITH_EDITORONLY_DATA
	if (NumCells > 0 && (-.5 <= OffsetPercent && OffsetPercent <= 0.5))
	{
		// Tet mesh generation
		UE::Geometry::TIsosurfaceStuffing<double> IsosurfaceStuffing;
		UE::Geometry::FDynamicMeshAABBTree3 Spatial(&DynamicMesh);
		UE::Geometry::TFastWindingTree<UE::Geometry::FDynamicMesh3> FastWinding(&Spatial);
		UE::Geometry::FAxisAlignedBox3d Bounds = Spatial.GetBoundingBox();
		IsosurfaceStuffing.Bounds = FBox(Bounds);
		double CellSize = Bounds.MaxDim() / NumCells;
		IsosurfaceStuffing.CellSize = CellSize;
		IsosurfaceStuffing.IsoValue = .5 + OffsetPercent;
		IsosurfaceStuffing.Implicit = [&FastWinding, &Spatial](FVector3d Pos)
		{
			FVector3d Nearest = Spatial.FindNearestPoint(Pos);
			double WindingSign = FastWinding.FastWindingNumber(Pos) - .5;
			return FVector3d::Distance(Nearest, Pos) * FMathd::SignNonZero(WindingSign);
		};

		UE_LOGF(LogChaosFlesh, Display, "Generating tet mesh via IsoStuffing...");
		IsosurfaceStuffing.Generate();
		if (IsosurfaceStuffing.Tets.Num() > 0)
		{
			TArray<FVector> Vertices; Vertices.SetNumUninitialized(IsosurfaceStuffing.Vertices.Num());
			TArray<FIntVector4> Elements; Elements.SetNumUninitialized(IsosurfaceStuffing.Tets.Num());
			TArray<FIntVector3> SurfaceElements = UE::Dataflow::GetSurfaceTriangles(IsosurfaceStuffing.Tets, !bDiscardInteriorTriangles);

			for (int32 Tdx = 0; Tdx < IsosurfaceStuffing.Tets.Num(); ++Tdx)
			{
				Elements[Tdx] = IsosurfaceStuffing.Tets[Tdx];
			}
			for (int32 Vdx = 0; Vdx < IsosurfaceStuffing.Vertices.Num(); ++Vdx)
			{
				Vertices[Vdx] = IsosurfaceStuffing.Vertices[Vdx];
			}

			TUniquePtr<FTetrahedralCollection> TetCollection(FTetrahedralCollection::NewTetrahedralCollection(Vertices, SurfaceElements, Elements));
			InCollection.AppendGeometry(*TetCollection.Get());

			UE_LOGF(LogChaosFlesh, Display,
				"Generated tet mesh via IsoStuffing, num vertices: %d num tets: %d", Vertices.Num(), Elements.Num());
		}
		else
		{
			UE_LOGF(LogChaosFlesh, Warning, "IsoStuffing produced 0 tetrahedra.");
		}
	}
#else
	ensureMsgf(false, TEXT("FCreateTetrahedronDataflowNodes is an editor only node."));
#endif
}

void FCreateTetrahedronDataflowNode::EvaluateTetWild(
	UE::Dataflow::FContext& Context, 
	FFleshCollection& InCollection,
	const UE::Geometry::FDynamicMesh3& DynamicMesh) const
{
#if WITH_EDITORONLY_DATA
	if (/* placeholder for conditions for exec */true)
	{
		// Pull out Vertices and Triangles
		TArray<FVector> Verts;
		TArray<FIntVector3> Tris;
		for (FVector V : DynamicMesh.VerticesItr())
		{
			Verts.Add(V);
		}
		for (UE::Geometry::FIndex3i Tri : DynamicMesh.TrianglesItr())
		{
			Tris.Emplace(Tri.A, Tri.B, Tri.C);
		}

		// Tet mesh generation
		UE::Geometry::FTetWild::FTetMeshParameters Params;
		Params.bCoarsen = bCoarsen;
		Params.bExtractManifoldBoundarySurface = bExtractManifoldBoundarySurface;
		Params.bSkipSimplification = bSkipSimplification;

		Params.EpsRel = EpsRel;
		Params.MaxIts = MaxIterations;
		Params.StopEnergy = StopEnergy;
		Params.IdealEdgeLengthRel = IdealEdgeLengthRel;

		Params.bInvertOutputTets = bInvertOutputTets;

		TArray<FVector> TetVerts;
		TArray<FIntVector4> Tets;
		FProgressCancel Progress;
		UE_LOGF(LogChaosFlesh, Display,"Generating tet mesh via TetWild...");
		if (UE::Geometry::FTetWild::ComputeTetMesh(Params, Verts, Tris, TetVerts, Tets, &Progress) && Tets.Num() > 0)
		{
			TArray<FIntVector3> SurfaceElements = UE::Dataflow::GetSurfaceTriangles(Tets, !bDiscardInteriorTriangles);
			TUniquePtr<FTetrahedralCollection> TetCollection(FTetrahedralCollection::NewTetrahedralCollection(TetVerts, SurfaceElements, Tets));
			InCollection.AppendGeometry(*TetCollection.Get());

			UE_LOGF(LogChaosFlesh, Display,
				"Generated tet mesh via TetWild, num vertices: %d num tets: %d", TetVerts.Num(), Tets.Num());
		}
		else
		{
			UE_LOGF(LogChaosFlesh, Error,
				"TetWild tetrahedral mesh generation failed.");
		}
	}
#else
	ensureMsgf(false, TEXT("FCreateTetrahedronDataflowNodes is an editor only node."));
#endif
}
