// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshFleshAssetTerminalNode.h"
#include "Animation/Skeleton.h"
#include "AnimationUtils.h"
#include "Engine/SkeletalMesh.h"
#include "Misc/PackageName.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFleshFleshAssetTerminalNode)


void FFleshAssetTerminalDataflowNode::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
	if (UFleshAsset* InFleshAsset = Cast<UFleshAsset>(Asset.Get()))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		TUniquePtr<FFleshCollection> NewFleshCollection(InCollection.NewCopy<FFleshCollection>());
		InFleshAsset->SetFleshCollection(MoveTemp(NewFleshCollection));
#if WITH_EDITOR
		InFleshAsset->PropagateTransformUpdateToComponents();
#endif
	}
}

void FFleshAssetTerminalDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
	SetValue(Context, InCollection, &Collection);
}

void FCurveSamplingAnimationAssetTerminalNode::Evaluate(UE::Dataflow::FContext& Context) const
{
	TObjectPtr<USkeletalMesh> InSkeletalMesh = GetValue(Context, &SkeletalMeshAsset);
	TObjectPtr<UAnimSequence> InAnimationAsset = GetValue(Context, &AnimationAsset);
}


void FCurveSamplingAnimationAssetTerminalNode::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
	// See Engine\Source\Editor\SequenceRecorder\Private\AnimationRecorder.cpp for reference
	if (TObjectPtr<USkeletalMesh> InSkeletalMesh = GetValue(Context, &SkeletalMeshAsset))
	{
		UAnimSequence* AssetToSet = Cast<UAnimSequence>(Asset.Get());
		if (!AssetToSet)
		{
			// use the input instead
			AssetToSet = GetValue(Context, &AnimationAsset);
		}
#if WITH_EDITOR
		if (AssetToSet)
		{
			if (!InSkeletalMesh->GetSkeleton())
			{
				Context.Error(FString::Printf(
					TEXT("Input skeletal Mesh [%s] has no skeleton."),
					*InSkeletalMesh->GetName()),
					this);
				return;
			}
			static bool bTransactRecording = false;

			// set skeleton
			AssetToSet->SetSkeleton(InSkeletalMesh->GetSkeleton());

			if (!AssetToSet->BoneCompressionSettings)
			{
				AssetToSet->BoneCompressionSettings = FAnimationUtils::GetDefaultAnimationRecorderBoneCompressionSettings();
			}
			const int32 NumBones = InSkeletalMesh->GetRefSkeleton().GetNum();
			TArray<FTransform> RestTransforms;
			InSkeletalMesh->GetRefSkeleton().GetBoneAbsoluteTransforms(RestTransforms);

			TArray<FName> CurveNamesArray;
			for (const UAssetUserData* AssetUserData : *InSkeletalMesh->GetAssetUserDataArray())
			{
				if (const UAnimCurveMetaData* AnimCurveMetaData = Cast<UAnimCurveMetaData>(AssetUserData))
				{
					AnimCurveMetaData->GetCurveMetaDataNames(CurveNamesArray);
				}
			}
			const int32 NumCurves = CurveNamesArray.Num();
			IAnimationDataController& Controller = AssetToSet->GetController();
			Controller.SetModel(AssetToSet->GetDataModelInterface());
			Controller.InitializeModel();
			Controller.RemoveAllCurvesOfType(ERawCurveTrackTypes::RCT_Float);
			Controller.RemoveAllBoneTracks(bTransactRecording);
			AssetToSet->ResetAnimation();
			TArray<FRawAnimSequenceTrack> RawTracks;

			USkeleton* AnimSkeleton = AssetToSet->GetSkeleton();
			// add all frames
			for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
			{
				// verify if this bone exists in skeleton
				const int32 BoneTreeIndex = AnimSkeleton->GetSkeletonBoneIndexFromMeshBoneIndex(
					InSkeletalMesh, BoneIndex);
				if (BoneTreeIndex != INDEX_NONE)
				{
					// add tracks for the bone existing
					const FName BoneTreeName = AnimSkeleton->GetReferenceSkeleton().GetBoneName(BoneTreeIndex);
					Controller.AddBoneCurve(BoneTreeName, bTransactRecording);
					RawTracks.AddDefaulted();
				}
			}

			AssetToSet->RetargetSource = AnimSkeleton->GetRetargetSourceForMesh(InSkeletalMesh);

			// record transforms

			TArray<FName> TrackNames;
			const IAnimationDataModel* DataModel = AssetToSet->GetDataModel();
			DataModel->GetBoneTrackNames(TrackNames);

			//FSerializedAnimation SerializedAnimation;
			for (int32 TrackIndex = 0; TrackIndex < TrackNames.Num(); ++TrackIndex)
			{
				const FName& TrackName = TrackNames[TrackIndex];
				FRawAnimSequenceTrack& RawTrack = RawTracks[TrackIndex];

				// verify if this bone exists in skeleton
				const int32 BoneTreeIndex = AnimSkeleton->GetReferenceSkeleton().FindBoneIndex(TrackName);
				if (BoneTreeIndex != INDEX_NONE)
				{
					const int32 BoneIndex = AnimSkeleton->GetMeshBoneIndexFromSkeletonBoneIndex(InSkeletalMesh, BoneTreeIndex);
					const int32 ParentIndex = InSkeletalMesh->GetRefSkeleton().GetParentIndex(BoneIndex);
					// Only record the rest pose for activation MLD training
					FTransform LocalTransform = RestTransforms[BoneIndex];
					if (ParentIndex != INDEX_NONE)
					{
						LocalTransform.SetToRelativeTransform(RestTransforms[ParentIndex]);
					}

					RawTrack.PosKeys.Add(FVector3f(LocalTransform.GetTranslation()));
					RawTrack.RotKeys.Add(FQuat4f(LocalTransform.GetRotation()));
					RawTrack.ScaleKeys.Add(FVector3f(LocalTransform.GetScale3D()));
				}
			}

			const int32 NumFrames = NumFramesPerMuscle * NumCurves;

			//Set Interpolation type (Step or Linear), doesn't look like there is a controller for this.
			AssetToSet->Interpolation = EAnimInterpolationType::Linear;
			const FFrameRate RecordingRate = FFrameRate(FrameRate, 1);
			// Set frame rate and number of frames
			Controller.SetFrameRate(RecordingRate, bTransactRecording);
			Controller.SetNumberOfFrames(NumFrames, bTransactRecording);

			// add to real curve data 
			static int32 NumKeys = 3; // Muscle activation goes 0 - 1 - 0

			for (int32 CurveIdx = 0; CurveIdx < CurveNamesArray.Num(); ++CurveIdx)
			{
				FName CurveName = CurveNamesArray[CurveIdx];
				const FFloatCurve* FloatCurveData = nullptr;

				TArray<float> TimesToRecord;
				TArray<float> ValuesToRecord;
				TimesToRecord.SetNum(NumKeys);
				ValuesToRecord.SetNum(NumKeys);

				bool bSeenThisCurve = false;
				int32 WriteIndex = 0;
				for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
				{
					const float TimeToRecord = RecordingRate.AsSeconds(NumFramesPerMuscle * CurveIdx + NumFramesPerMuscle * KeyIndex / 2);
					bool bIsCurveValid = false;
					const float CurCurveValue = KeyIndex == 1 ? 1 : 0; // 0 - 1 - 0 curve value for 3 keys
					if (!bSeenThisCurve)
					{
						bSeenThisCurve = true;
						const FAnimationCurveIdentifier CurveId(CurveName, ERawCurveTrackTypes::RCT_Float);
						Controller.AddCurve(CurveId, AACF_DefaultCurve, bTransactRecording);
						FloatCurveData = AssetToSet->GetDataModel()->FindFloatCurve(CurveId);
					}

					if (FloatCurveData)
					{
						TimesToRecord[WriteIndex] = TimeToRecord;
						ValuesToRecord[WriteIndex] = CurCurveValue;
						++WriteIndex;
					}
				}

				// Fill all the curve data at once
				if (FloatCurveData)
				{
					TArray<FRichCurveKey> Keys;
					for (int32 Index = 0; Index < WriteIndex; ++Index)
					{
						FRichCurveKey Key(TimesToRecord[Index], ValuesToRecord[Index]);
						Key.InterpMode = ERichCurveInterpMode::RCIM_Linear;
						Key.TangentMode = ERichCurveTangentMode::RCTM_SmartAuto;
						Keys.Add(Key);
					}

					const FAnimationCurveIdentifier CurveId(FloatCurveData->GetName(), ERawCurveTrackTypes::RCT_Float);
					Controller.SetCurveKeys(CurveId, Keys, bTransactRecording);
				}
			}

			// Populate bone tracks
			for (int32 TrackIndex = 0; TrackIndex < TrackNames.Num(); ++TrackIndex)
			{
				const FRawAnimSequenceTrack& RawTrack = RawTracks[TrackIndex];
				FName BoneName = TrackNames[TrackIndex];
				Controller.SetBoneTrackKeys(BoneName, RawTrack.PosKeys, RawTrack.RotKeys, RawTrack.ScaleKeys, bTransactRecording);
			}

			AssetToSet->PostEditChange();
			AssetToSet->MarkPackageDirty();

			// save the package to disk, for convenience and so we can run this in standalone mode
			UPackage* const Package = AssetToSet->GetOutermost();
			FString const PackageName = Package->GetName();
			FString const PackageFileName = FPackageName::LongPackageNameToFilename(PackageName, FPackageName::GetAssetPackageExtension());

			FSavePackageArgs SaveArgs;
			SaveArgs.TopLevelFlags = RF_Standalone;
			SaveArgs.SaveFlags = SAVE_NoError;
			UPackage::SavePackage(Package, nullptr, *PackageFileName, SaveArgs);
		} // if (AssetToSet)
#endif // WITH_EDITOR
	}
}