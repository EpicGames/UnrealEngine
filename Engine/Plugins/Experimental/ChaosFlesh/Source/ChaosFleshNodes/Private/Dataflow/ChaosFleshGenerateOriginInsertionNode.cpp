// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshGenerateOriginInsertionNode.h"

#include "ChaosFlesh/ChaosFlesh.h"
#include "ChaosFlesh/TetrahedralCollection.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFleshGenerateOriginInsertionNode)

void FGenerateOriginInsertionNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		//
		// Gather inputs
		//

		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		TArray<int32> InOriginIndices = GetValue<TArray<int32>>(Context, &OriginIndicesIn);
		TArray<int32> InInsertionIndices = GetValue<TArray<int32>>(Context, &InsertionIndicesIn);
		TArray<int32> OutOriginIndices;
		TArray<int32> OutInsertionIndices;
		// Tetrahedra
		TManagedArray<FIntVector4>* Elements = InCollection.FindAttribute<FIntVector4>(
			FTetrahedralCollection::TetrahedronAttribute, FTetrahedralCollection::TetrahedralGroup);
		if (!Elements)
		{
			UE_LOGF(LogChaosFlesh, Warning,
				"GenerateOriginInsertionNode: Failed to find geometry collection attr '%ls' in group '%ls'",
				*FTetrahedralCollection::TetrahedronAttribute.ToString(), *FTetrahedralCollection::TetrahedralGroup.ToString());
			Out->SetValue(MoveTemp(InCollection), Context);
			return;
		}

		// Vertices
		TManagedArray<FVector3f>* Vertex = InCollection.FindAttribute<FVector3f>("Vertex", "Vertices");
		//TArray<FVector3f>* MeshVertex;
		if (!Vertex)
		{
			UE_LOGF(LogChaosFlesh, Warning,
				"GenerateOriginInsertionNode: Failed to find geometry collection attr 'Vertex' in group 'Vertices'");
			Out->SetValue(MoveTemp(InCollection), Context);
			return;
		}

		// Incident elements
		TManagedArray<TArray<int32>>* IncidentElements = InCollection.FindAttribute<TArray<int32>>(
			FTetrahedralCollection::IncidentElementsAttribute, FGeometryCollection::VerticesGroup);
		if (!IncidentElements)
		{
			UE_LOGF(LogChaosFlesh, Warning,
				"GenerateOriginInsertionNode: Failed to find geometry collection attr '%ls' in group '%ls'",
				*FTetrahedralCollection::IncidentElementsAttribute.ToString(), *FGeometryCollection::VerticesGroup.ToString());
			Out->SetValue(MoveTemp(InCollection), Context);
			return;
		}
		TManagedArray<TArray<int32>>* IncidentElementsLocalIndex = InCollection.FindAttribute<TArray<int32>>(
			FTetrahedralCollection::IncidentElementsLocalIndexAttribute, FGeometryCollection::VerticesGroup);
		if (!IncidentElementsLocalIndex)
		{
			UE_LOGF(LogChaosFlesh, Warning,
				"GenerateOriginInsertionNode: Failed to find geometry collection attr '%ls' in group '%ls'",
				*FTetrahedralCollection::IncidentElementsLocalIndexAttribute.ToString(), *FGeometryCollection::VerticesGroup.ToString());
			Out->SetValue(MoveTemp(InCollection), Context);
			return;
		}

		//
		// Pull Origin & Insertion data out of the geometry collection.  We may want other ways of specifying
		// these via an input on the node...
		//
		auto DoubleVert = [](FVector3f V) { return FVector3d(V.X, V.Y, V.Z); };
		GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(InCollection);
		TArray<int32> ComponentIndex = MeshFacade.GetGeometryGroupIndexArray();
		// Origin vertices
		if (!InOriginIndices.IsEmpty())
		{
			for (int32 i = 0; i < InOriginIndices.Num(); ++i)
			{
				if (InOriginIndices[i] < Vertex->Num())
				{
					for (int32 j = 0; j < Vertex->Num(); ++j)
					{
						if (ComponentIndex[InOriginIndices[i]] == ComponentIndex[j] 
							&& ComponentIndex[InOriginIndices[i]] >= 0
							&& ComponentIndex[j] >= 0
							&& ((*Vertex)[InOriginIndices[i]] - (*Vertex)[j]).Size() < Radius)
						{
							OutOriginIndices.Add(j);
						}
					}
				}
			}
		}

		// Insertion vertices
		if (!InInsertionIndices.IsEmpty())
		{
			for (int32 i = 0; i < InInsertionIndices.Num(); ++i)
			{
				if (InInsertionIndices[i] < Vertex->Num())
				{
					for (int32 j = 0; j < Vertex->Num(); ++j)
					{
						if (ComponentIndex[InInsertionIndices[i]] == ComponentIndex[j]
							&& ComponentIndex[InInsertionIndices[i]] >= 0
							&& ComponentIndex[j] >= 0
							&& ((*Vertex)[InInsertionIndices[i]] - (*Vertex)[j]).Size() < Radius)
						{
							OutInsertionIndices.Add(j);
						}
					}
				}
			}
		}

		SetValue(Context, MoveTemp(InCollection), &Collection);
		SetValue(Context, MoveTemp(OutOriginIndices), &OriginIndicesOut);
		SetValue(Context, MoveTemp(OutInsertionIndices), &InsertionIndicesOut);
	}
}
