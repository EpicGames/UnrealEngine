// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshGenerateSkeletalBindingsNode.h"

#include "Chaos/AABBTree.h"
#include "Chaos/BoundingVolumeHierarchy.h"
#include "Chaos/Tetrahedron.h"
#include "Chaos/TriangleMesh.h"
#include "ChaosFlesh/TetrahedralCollection.h"
#include "Containers/Map.h"
#include "Engine/StaticMesh.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Engine/SkeletalMesh.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "GeometryCollection/Facades/CollectionTetrahedralSkeletalBindingsFacade.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "UObject/PrimaryAssetId.h"

DEFINE_LOG_CATEGORY(LogSkeletalBindings);


namespace UE::Dataflow
{

	TArray<int32> ChildIndices(TObjectPtr<const USkeletalMesh> SkeletalMesh, int32 StartIndex)
	{
		int32 IndexCount = 0;
		TArray<int32> BranchIndices;

		TArray<int32> ToProcess;
		int32 CurrentIndex = StartIndex;
		while (SkeletalMesh->GetRefSkeleton().IsValidIndex(CurrentIndex))
		{
			TArray<int32> Buffer;
			SkeletalMesh->GetRefSkeleton().GetDirectChildBones(CurrentIndex, Buffer);

			if (Buffer.Num())
			{
				ToProcess.Append(Buffer);
			}

			if (!IndexCount) // skip the first index to get only the chidren. 
			{
				BranchIndices.Add(CurrentIndex);
			}

			CurrentIndex = INDEX_NONE;
			if (ToProcess.Num())
			{
				CurrentIndex = ToProcess.Pop();
			}
		}
		return BranchIndices;
	}

}

void
FGenerateSkeletalBindings::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	auto UEVertd = [](FVector3f V) { return FVector3d(V.X, V.Y, V.Z); };
	auto UEVertf = [](FVector3d V) { return FVector3f(V.X, V.Y, V.Z); };

	if (Out->IsA<DataType>(&Collection))
	{
		DataType OutCollection = GetValue<DataType>(Context, &Collection); // Deep copy

		TManagedArray<FIntVector4>* Tetrahedron = OutCollection.FindAttribute<FIntVector4>(FTetrahedralCollection::TetrahedronAttribute, FTetrahedralCollection::TetrahedralGroup);
		TManagedArray<int32>* TetrahedronStart =OutCollection.FindAttribute<int32>(FTetrahedralCollection::TetrahedronStartAttribute, FGeometryCollection::GeometryGroup);
		TManagedArray<int32>* TetrahedronCount =OutCollection.FindAttribute<int32>(FTetrahedralCollection::TetrahedronCountAttribute, FGeometryCollection::GeometryGroup);
		TManagedArray<TArray<int32>>* IncidentElements = OutCollection.FindAttribute<TArray<int32>>(FTetrahedralCollection::IncidentElementsAttribute, FGeometryCollection::VerticesGroup);
		TManagedArray<FVector3f>* Vertex = OutCollection.FindAttribute<FVector3f>("Vertex", "Vertices");

		TObjectPtr<const USkeletalMesh> SkeletalMesh = GetValue<TObjectPtr<const USkeletalMesh>>(Context, &SkeletalMeshIn);
		if (SkeletalMesh && Tetrahedron && TetrahedronStart && TetrahedronCount && Vertex)
		{
			// Iterate all the tetrahedral meshs
			for (int32 TetMeshIdx = 0; TetMeshIdx < TetrahedronStart->Num(); TetMeshIdx++)
			{
				const int32 TetMeshStart = (*TetrahedronStart)[TetMeshIdx];
				const int32 TetMeshCount = (*TetrahedronCount)[TetMeshIdx];

				// Build Tetrahedra
				TArray<Chaos::TTetrahedron<Chaos::FReal>> Tets;			// Index 0 == TetMeshStart
				TArray<Chaos::TTetrahedron<Chaos::FReal>*> BVHTetPtrs;
				Tets.SetNumUninitialized(TetMeshCount);
				BVHTetPtrs.SetNumUninitialized(TetMeshCount);
				for (int32 i = 0; i < TetMeshCount; i++)
				{
					const int32 Idx = TetMeshStart + i;
					const FIntVector4& Tet = (*Tetrahedron)[Idx];
					Tets[i] = Chaos::TTetrahedron<Chaos::FReal>(
						(*Vertex)[Tet[0]],
						(*Vertex)[Tet[1]],
						(*Vertex)[Tet[2]],
						(*Vertex)[Tet[3]]);
					BVHTetPtrs[i] = &Tets[i];
				}

				// Init BVH for tetrahedra.
				Chaos::TBoundingVolumeHierarchy<
					TArray<Chaos::TTetrahedron<Chaos::FReal>*>,
					TArray<int32>,
					Chaos::FReal,
					3> TetBVH(BVHTetPtrs);

				FName SkeletalMeshName = FName(SkeletalMesh->GetName());
				TArray<FTransform> ComponentPose;
				UE::Dataflow::Animation::GlobalTransforms(SkeletalMesh->GetRefSkeleton(), ComponentPose);
				TArray<int32> Indices = UE::Dataflow::ChildIndices(SkeletalMesh, GetValue<int32>(Context, &BoneIndexIn));

				//
				// Do intersection tests against tets
				//

				int CurrentIndex = 0;
				TArray<int32> TetIndex; TetIndex.SetNum(Indices.Num());
				TArray<FVector4f> Weights;		Weights.SetNum(Indices.Num());
				TArray<FVector3f> Offsets;		Offsets.SetNum(Indices.Num());
				TArray<int32> IndexKey;			IndexKey.SetNum(Indices.Num());

				TArray<int32> TetIntersections; TetIntersections.Reserve(64);
				for (int32 i = 0; i < Indices.Num(); i++)
				{
					int32 TransformIndex = Indices[i];

					FVector TestPosition = ComponentPose[TransformIndex].GetTranslation();
					TetIntersections = TetBVH.FindAllIntersections(TestPosition);
					for (int32 j = 0; j < TetIntersections.Num(); j++)
					{
						const int32 TetIdx = TetIntersections[j];
						if (!Tets[TetIdx].Outside(TestPosition, 1.0e-2)) // includes boundary
						{
							Chaos::TVector<Chaos::FReal, 4> WeightsD = Tets[TetIdx].GetBarycentricCoordinates(TestPosition);
							Weights[CurrentIndex] = FVector4f(WeightsD[0], WeightsD[1], WeightsD[2], WeightsD[3]);
							TetIndex[CurrentIndex] = TetIdx + TetMeshStart;
							IndexKey[CurrentIndex] = TransformIndex;

							// validate
							{
								FIntVector4 Parent = (*Tetrahedron)[TetIndex[CurrentIndex]];
								FVector3f EmbeddedPos =
									(*Vertex)[Parent[0]] * Weights[CurrentIndex][0] +
									(*Vertex)[Parent[1]] * Weights[CurrentIndex][1] +
									(*Vertex)[Parent[2]] * Weights[CurrentIndex][2] +
									(*Vertex)[Parent[3]] * Weights[CurrentIndex][3];
								ensureMsgf((UEVertf(TestPosition) - EmbeddedPos).SquaredLength() < 1.0, TEXT("ERROR: Skeletal Bindings mis alignment."));
								check((UEVertf(TestPosition) - EmbeddedPos).SquaredLength() < 1.0);
							}

							CurrentIndex++;
							break;
						}
					}
				}
				TetIndex.SetNum(CurrentIndex, EAllowShrinking::No);
				Weights.SetNum(CurrentIndex, EAllowShrinking::No);
				IndexKey.SetNum(CurrentIndex, EAllowShrinking::No);

				UE_LOGF(LogSkeletalBindings, Display,
					   "'%ls' - Generated mesh bindings between tet mesh index %d and skeletal mesh - stats:\n"
					   "    Skeletal transforms num: %d\n"
					   "    Transforms in tetrahedra: %d",
					   *(SkeletalMeshName.ToString()), TetMeshIdx, Indices.Num(), CurrentIndex);

				// Store bindings in the collection
				if (TetIndex.Num())
				{
					using namespace GeometryCollection::Facades;
					GeometryCollection::Facades::FTetrahedralSkeletalBindings TetBindings(OutCollection);
					FString MeshBindingsName = FTetrahedralSkeletalBindings::GenerateMeshGroupName(TetMeshIdx, SkeletalMeshName);
					TetBindings.SetBindings(MeshBindingsName, TetIndex, Weights, IndexKey);
				}
			} // end for TetMeshIdx
		}
		SetValue(Context, MoveTemp(OutCollection), &Collection);
	}
}
