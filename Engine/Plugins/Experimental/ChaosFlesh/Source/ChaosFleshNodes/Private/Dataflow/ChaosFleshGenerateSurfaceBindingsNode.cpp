// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshGenerateSurfaceBindingsNode.h"

#include "Dataflow/DataflowInputOutput.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Chaos/AABBTree.h"
#include "Chaos/BoundingVolumeHierarchy.h"
#include "Chaos/Tetrahedron.h"
#include "Chaos/TriangleMesh.h"
#include "ChaosFlesh/FleshCollectionEngineUtility.h"
#include "ChaosFlesh/TetrahedralCollection.h"
#include "Containers/Map.h"
#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#include "GeometryCollection/Facades/CollectionTetrahedralBindingsFacade.h"
#include "GeometryCollection/GeometryCollectionEngineConversion.h"
#include "IndexTypes.h"
#include "MeshDescription.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "UObject/PrimaryAssetId.h"
#include "Dataflow/ChaosFleshNodesUtility.h"
#include "UDynamicMesh.h"
#include "DynamicMesh/InfoTypes.h"
#include "DynamicMesh/MeshNormals.h"
#include "Chaos/TriangleCollisionPoint.h"
#include "Chaos/HierarchicalSpatialHash.h"

DEFINE_LOG_CATEGORY(LogMeshBindings);

void
BuildVertexToVertexAdjacencyBuffer(
	const UE::Geometry::FDynamicMesh3 DynamicMesh,
	TArray<TArray<uint32>>& OutNeighborNodes)
{
	OutNeighborNodes.SetNum(DynamicMesh.VertexCount());
	for (int32 TriIdx = 0; TriIdx < DynamicMesh.TriangleCount(); ++TriIdx)
	{
		const UE::Geometry::FIndex3i& Tri = DynamicMesh.GetTriangle(TriIdx);
		TArray<uint32>& N0 = OutNeighborNodes[Tri[0]];
		N0.AddUnique(Tri[1]);
		N0.AddUnique(Tri[2]);
		TArray<uint32>& N1 = OutNeighborNodes[Tri[1]];
		N1.AddUnique(Tri[0]);
		N1.AddUnique(Tri[2]);
		TArray<uint32>& N2 = OutNeighborNodes[Tri[2]];
		N2.AddUnique(Tri[0]);
		N2.AddUnique(Tri[1]);
	}
}

void UnloadMeshDescription(const FMeshDescription& SourceMesh,
	TArray<FVector3f>& OutVertices, TArray<FIntVector>& OutTriangles, TArray<TArray<uint32>>& OutNeighborNodes)
{
	FDynamicMesh3 DynamicMesh;
	FMeshDescriptionToDynamicMesh Converter;
	Converter.Convert(&SourceMesh, DynamicMesh);

	OutVertices.SetNumUninitialized(DynamicMesh.VertexCount());
	for (int32 VertexIdx = 0; VertexIdx < DynamicMesh.VertexCount(); ++VertexIdx)
	{
		const FVector3d& Pos = DynamicMesh.GetVertex(VertexIdx);
		OutVertices[VertexIdx].Set(Pos[0], Pos[1], Pos[2]);
	}

	OutTriangles.SetNumUninitialized(DynamicMesh.TriangleCount());
	for (int32 TriIdx = 0; TriIdx < DynamicMesh.TriangleCount(); ++TriIdx)
	{
		const UE::Geometry::FIndex3i& Tri = DynamicMesh.GetTriangle(TriIdx);
		OutTriangles[TriIdx] = FIntVector(Tri[0], Tri[1], Tri[2]);
	}

	BuildVertexToVertexAdjacencyBuffer(DynamicMesh, OutNeighborNodes);
}

void
BuildVertexToVertexAdjacencyBuffer(
	const FSkeletalMeshLODRenderData& LodRenderData,
	TArray<TArray<uint32>>& OutNeighborNodes)
{
	const FRawStaticIndexBuffer16or32Interface* IndexBuffer = LodRenderData.MultiSizeIndexContainer.GetIndexBuffer();
	const uint32 IndexCount = IndexBuffer->Num();

	const FPositionVertexBuffer& VertexBuffer = LodRenderData.StaticVertexBuffers.PositionVertexBuffer;
	const uint32 VertexCount = LodRenderData.StaticVertexBuffers.PositionVertexBuffer.GetNumVertices();

	OutNeighborNodes.SetNum(0); // clear, to init clean
	OutNeighborNodes.SetNum(VertexCount);

	int32 BaseTriangle = 0;
	int32 BaseVertex = 0;
	for (int32 SectionIndex = 0; SectionIndex < LodRenderData.RenderSections.Num(); ++SectionIndex)
	{
		FSkelMeshRenderSection const& RenderSection = LodRenderData.RenderSections[SectionIndex];
		int32 NumTriangles = RenderSection.NumTriangles;
		int32 NumVertices = RenderSection.NumVertices;

		TArray<uint32> RedirectionArray;
		RedirectionArray.SetNum(VertexCount);
		TMap<FVector, int32 /*UniqueVertexIndex*/> UniqueIndexMap;

		for (int32 TriangleIt = BaseTriangle; TriangleIt < BaseTriangle + NumTriangles; ++TriangleIt)
		{
			const uint32 V[3] =
			{
				IndexBuffer->Get(TriangleIt * 3 + 0),
				IndexBuffer->Get(TriangleIt * 3 + 1),
				IndexBuffer->Get(TriangleIt * 3 + 2)
			};

			const FVector P[3] =
			{
				(FVector)VertexBuffer.VertexPosition(V[0]),
				(FVector)VertexBuffer.VertexPosition(V[1]),
				(FVector)VertexBuffer.VertexPosition(V[2])
			};

			for (int32 i = 0; i < 3; ++i)
			{
				const uint32 VertexIndex = RedirectionArray[V[i]] = UniqueIndexMap.FindOrAdd(P[i], V[i]);
				TArray<uint32>& AdjacentVertices = OutNeighborNodes[VertexIndex];
				for (int32 a = 1; a < 3; ++a)
				{
					const uint32 AdjacentVertexIndex = V[(i + a) % 3];
					if (VertexIndex != AdjacentVertexIndex)
					{
						AdjacentVertices.AddUnique(AdjacentVertexIndex);
					}
				}
			}
		}

		for (int32 VertexIt = BaseVertex + 1; VertexIt < BaseVertex + NumVertices; ++VertexIt)
		{
			// if this vertex has a sibling we copy the data over
			const int32 SiblingIndex = RedirectionArray[VertexIt];
			if (SiblingIndex != VertexIt)
			{
				for (int32 i = 0; i < OutNeighborNodes[SiblingIndex].Num(); i++)
				{
					const uint32 OtherNode = OutNeighborNodes[SiblingIndex][i];
					if (OtherNode != VertexIt)
					{
						OutNeighborNodes[VertexIt].AddUnique(OtherNode);
					}
				}
			}
		}

		BaseTriangle += NumTriangles;
		BaseVertex += NumVertices;
	}
}

void
BuildTriangles(
	const FSkeletalMeshLODRenderData& LodRenderData,
	TArray<FIntVector>& TriangleMesh)
{
	const FRawStaticIndexBuffer16or32Interface* IndexBuffer = LodRenderData.MultiSizeIndexContainer.GetIndexBuffer();
	const uint32 IndexCount = IndexBuffer->Num();

	TriangleMesh.Empty(); // clear, to init clean

	int32 BaseTriangle = 0;
	for (int32 SectionIndex = 0; SectionIndex < LodRenderData.RenderSections.Num(); ++SectionIndex)
	{
		FSkelMeshRenderSection const& RenderSection = LodRenderData.RenderSections[SectionIndex];
		int32 NumTriangles = RenderSection.NumTriangles;
		TriangleMesh.SetNum(BaseTriangle + NumTriangles);
		for (int32 TriangleIt = BaseTriangle; TriangleIt < BaseTriangle + NumTriangles; ++TriangleIt)
		{
			TriangleMesh[TriangleIt] = FIntVector
			(
				IndexBuffer->Get(TriangleIt * 3 + 0),
				IndexBuffer->Get(TriangleIt * 3 + 1),
				IndexBuffer->Get(TriangleIt * 3 + 2)
			);
		}
		BaseTriangle += NumTriangles;
	}
}

void
FGenerateSurfaceBindings::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;
	auto FloatVert = [](FVector3d V) { return FVector3f(V.X, V.Y, V.Z); };
	auto DoubleVert = [](FVector3f V) { return FVector3d(V.X, V.Y, V.Z); };
	auto TVec3Vert = [](FVector3f V) { return Chaos::TVec3<Chaos::FRealDouble>(V.X, V.Y, V.Z); };
	
	if (Out->IsA(&Collection) || Out->IsA(&SKMDynamicMesh))
	{
		TUniquePtr<FTetrahedralCollection> InCollection(GetValue(Context, &Collection).NewCopy<FTetrahedralCollection>());
		const TManagedArray<FIntVector4>* Tetrahedron = 
			InCollection->FindAttribute<FIntVector4>(
				FTetrahedralCollection::TetrahedronAttribute, FTetrahedralCollection::TetrahedralGroup);
		const TManagedArray<int32>* TetrahedronStart =
			InCollection->FindAttribute<int32>(
				FTetrahedralCollection::TetrahedronStartAttribute, FGeometryCollection::GeometryGroup);
		const TManagedArray<int32>* TetrahedronCount =
			InCollection->FindAttribute<int32>(
				FTetrahedralCollection::TetrahedronCountAttribute, FGeometryCollection::GeometryGroup);
		const TManagedArray<TArray<int32>>* IncidentElements =
			InCollection->FindAttribute<TArray<int32>>(
				FTetrahedralCollection::IncidentElementsAttribute, FGeometryCollection::VerticesGroup);

		const TManagedArray<FIntVector>* Triangle =
			InCollection->FindAttribute<FIntVector>(
				"Indices", FGeometryCollection::FacesGroup);
		const TManagedArray<int32>* FacesStart =
			InCollection->FindAttribute<int32>(
				"FaceStart", FGeometryCollection::GeometryGroup);
		const TManagedArray<int32>* FacesCount =
			InCollection->FindAttribute<int32>(
				"FaceCount", FGeometryCollection::GeometryGroup);

		const TManagedArray<FVector3f>* Vertex = 
			InCollection->FindAttribute<FVector3f>(
				"Vertex", "Vertices");

		TObjectPtr<const USkeletalMesh> SkeletalMesh = GetValue(Context, &SkeletalMeshIn);
		TObjectPtr<const UStaticMesh> StaticMesh = GetValue(Context, &StaticMeshIn);
		TObjectPtr<UDynamicMesh> OutSKMDynamicMesh = NewObject<UDynamicMesh>();
		OutSKMDynamicMesh->Reset();
		FDynamicMesh3& OutSKMDynamicMesh3 = OutSKMDynamicMesh->GetMeshRef();

		const bool UseSkeletalMesh = SkeletalMesh != nullptr;
		const bool UseStaticMesh = StaticMesh != nullptr;
		if (IsConnected(&Collection) && (UseSkeletalMesh || UseStaticMesh) &&
			Tetrahedron && TetrahedronStart && TetrahedronCount &&
			Triangle && FacesStart && FacesCount &&
			Vertex)
		{
			// Extract positions to bind
			FString MeshId;
			TArray<TArray<FVector3f>> RenderMeshVertices;
			TArray<TArray<FIntVector>> RenderMeshTriangles;
			TArray<TArray<TArray<uint32>>> RenderMeshNeighborNodes;
			if (UseSkeletalMesh)
			{
				FPrimaryAssetId Id = SkeletalMesh->GetPrimaryAssetId();
				MeshId = ChaosFlesh::GetMeshId(SkeletalMesh, bUseSkeletalMeshImportModel);

				if (!bUseSkeletalMeshImportModel)
				{
					FSkeletalMeshRenderData* RenderData = SkeletalMesh->GetResourceForRendering();

					RenderMeshVertices.SetNum(RenderData->LODRenderData.Num());
					RenderMeshTriangles.SetNum(RenderData->LODRenderData.Num());
					RenderMeshNeighborNodes.SetNum(RenderData->LODRenderData.Num());
					for (int32 i = 0; i < RenderData->LODRenderData.Num(); i++)
					{
						FSkeletalMeshLODRenderData* LODRenderData = &RenderData->LODRenderData[i];
						const FPositionVertexBuffer& PositionVertexBuffer =
							LODRenderData->StaticVertexBuffers.PositionVertexBuffer;

						TArray<FVector3f>& Vertices = RenderMeshVertices[i];
						Vertices.SetNumUninitialized(PositionVertexBuffer.GetNumVertices());
						for (uint32 j = 0; j < PositionVertexBuffer.GetNumVertices(); j++)
						{
							const FVector3f& Pos = PositionVertexBuffer.VertexPosition(j);
							Vertices[j] = Pos;
						}

						TArray<TArray<uint32>>& NeighborNodes = RenderMeshNeighborNodes[i];
						BuildVertexToVertexAdjacencyBuffer(*LODRenderData, NeighborNodes);
						BuildTriangles(*LODRenderData, RenderMeshTriangles[i]);
					}
				}
#if WITH_EDITOR
				else // Import Model
				{
					const int32 LODIndex = 0;
					RenderMeshVertices.SetNum(1);
					RenderMeshTriangles.SetNum(1);
					RenderMeshNeighborNodes.SetNum(1);

					// Check first if we have bulk data available and non-empty.
					FMeshDescription SourceMesh;
#if WITH_EDITORONLY_DATA
					if (SkeletalMesh->HasMeshDescription(LODIndex))
					{
						SkeletalMesh->CloneMeshDescription(LODIndex, SourceMesh);
					}
					else
#endif
					{
						// Fall back on the LOD model directly if no bulk data exists. When we commit
						// the mesh description, we override using the bulk data. This can happen for older
						// skeletal meshes, from UE 4.24 and earlier.
						const FSkeletalMeshModel* SkeletalMeshModel = SkeletalMesh->GetImportedModel();
						if (SkeletalMeshModel && SkeletalMeshModel->LODModels.IsValidIndex(LODIndex))
						{
							SkeletalMeshModel->LODModels[LODIndex].GetMeshDescription(SkeletalMesh, LODIndex, SourceMesh);
						}
					}
					UnloadMeshDescription(SourceMesh, RenderMeshVertices[LODIndex], RenderMeshTriangles[LODIndex], RenderMeshNeighborNodes[LODIndex]);
				}
#endif
			}
			else // StaticMesh
			{
				MeshId = ChaosFlesh::GetMeshId(StaticMesh);
				const int32 LODIndex = 0;
				RenderMeshVertices.SetNum(1);
				RenderMeshTriangles.SetNum(1);
				RenderMeshNeighborNodes.SetNum(1);
				FMeshDescription* MeshDescription = FGeometryCollectionEngineConversion::GetMaxResMeshDescriptionWithNormalsAndTangents(StaticMesh);

				if (MeshDescription)
				{
					UnloadMeshDescription(*MeshDescription, RenderMeshVertices[LODIndex], RenderMeshTriangles[LODIndex], RenderMeshNeighborNodes[LODIndex]);
				}
				else
				{
					Context.Warning(FString::Printf(
						TEXT("No MeshDescription found in Static Mesh [%s]."),
						*StaticMesh->GetName()),
						this, Out);
				}
			}
			TArray<FString> GeometryGroupGuidsLocal = GetValue(Context, &GeometryGroupGuidsIn);
			const TManagedArray<FString>* Guids = InCollection->FindAttribute<FString>("Guid", FGeometryCollection::GeometryGroup);

			// Build Tetrahedra
			TArray<Chaos::TTetrahedron<Chaos::FReal>> Tets;			// Index 0 == TetMeshStart
			TArray<Chaos::TTetrahedron<Chaos::FReal>*> BVHTetPtrs;

			//
			// Init boundary mesh for projections.
			//
			TArray<FIntVector> Triangles;
			Chaos::FTriangleMesh SurfaceMesh;
			Chaos::FTriangleMesh::TBVHType<Chaos::FRealDouble> TetBoundaryBVH;
			TArray<Chaos::FVec3> VertexD;
			TConstArrayView<Chaos::TVec3<Chaos::FRealDouble>> VertexDView(VertexD);
			TArray<Chaos::FVec3> PointNormals;

			Chaos::THierarchicalSpatialHash<int32, Chaos::FRealDouble> SpatialHash;

			TArray<int32> GeometryGroupSelected;
			if (IsConnected(&TransformSelection))
			{
				FDataflowTransformSelection InTransformSelection = GetValue(Context, &TransformSelection);
				if (InTransformSelection.Num() == InCollection->NumElements(FGeometryCollection::TransformGroup))
				{
					GeometryGroupSelected = InCollection->TransformSelectionToGeometryIndices(InTransformSelection.AsArray());
				}
				else
				{
					UE_LOGF(LogMeshBindings, Error,
						"TransformSelection size: %d is different from Collection Transform group size: %d",
						InTransformSelection.Num(),
						InCollection->NumElements(FGeometryCollection::TransformGroup));
					return;
				}
			}
			else
			{
				for (int32 GeometryIdx = 0; GeometryIdx < TetrahedronStart->Num(); ++GeometryIdx)
				{
					GeometryGroupSelected.Add(GeometryIdx);
				}
			}
			TArray<int32> UsedTetsIndexToGlobalTetIndex;
			TArray<int32> GlobalTetIndexToUsedTetsIndex;
			GlobalTetIndexToUsedTetsIndex.Init(INDEX_NONE, Tetrahedron->Num());
			int32 NumGuidHit = 0;
			for (const int32& GeometryIdx: GeometryGroupSelected)
			{
				if (GeometryGroupGuidsLocal.Num() && Guids)
				{
					if (!(*Guids)[GeometryIdx].IsEmpty() && !GeometryGroupGuidsLocal.Contains((*Guids)[GeometryIdx]))
					{
						continue;
					}
					else
					{
						NumGuidHit++;
					}
				}
				const int32 TetMeshStart = (*TetrahedronStart)[GeometryIdx];
				const int32 TetMeshCount = (*TetrahedronCount)[GeometryIdx];
				Tets.Reserve(Tets.Num() + TetMeshCount);
				UsedTetsIndexToGlobalTetIndex.Reserve(UsedTetsIndexToGlobalTetIndex.Num() + TetMeshCount);
				for (int32 i = 0; i < TetMeshCount; i++)
				{
					const int32 Idx = TetMeshStart + i;
					const FIntVector4& Tet = (*Tetrahedron)[Idx];
					GlobalTetIndexToUsedTetsIndex[Idx] = Tets.Num();
					Tets.Add(Chaos::TTetrahedron<Chaos::FReal>(
						(*Vertex)[Tet[0]],
						(*Vertex)[Tet[1]],
						(*Vertex)[Tet[2]],
						(*Vertex)[Tet[3]]));
					UsedTetsIndexToGlobalTetIndex.Add(Idx);
				}
				if (bDoSurfaceProjection)
				{
					Triangles.Reserve(Triangles.Num() + (*FacesCount)[GeometryIdx]);
					for (int32 FaceIdx = (*FacesStart)[GeometryIdx]; FaceIdx < (*FacesStart)[GeometryIdx] + (*FacesCount)[GeometryIdx]; ++FaceIdx)
					{
						Triangles.Add((*Triangle)[FaceIdx]);
					}
				}
			}
			if (GeometryGroupGuidsLocal.Num() && NumGuidHit == 0)
			{
				UE_LOGF(LogMeshBindings, Error,
					"GeometryGroupGuids contains %d guids but none was matched (empty guids are ignored).",
					GeometryGroupGuidsLocal.Num());
				return;
			}
			// Init BVH for tetrahedra.
			BVHTetPtrs.SetNumUninitialized(Tets.Num());
			for (int32 TetIdx = 0; TetIdx < Tets.Num(); ++TetIdx)
			{
				BVHTetPtrs[TetIdx] = &Tets[TetIdx];
			}
			Chaos::TBoundingVolumeHierarchy<
				TArray<Chaos::TTetrahedron<Chaos::FReal>*>,
				TArray<int32>,
				Chaos::FReal,
				3> TetBVH(BVHTetPtrs);

			// Init BVH for surface triangle mesh.
			if (bDoSurfaceProjection)
			{
				//Todo(Chaosflesh): refactor reinterpret_cast in case the memory layout of the vector types not the same
				SurfaceMesh.Init(
					reinterpret_cast<const TArray<Chaos::TVec3<int32>>&>(Triangles), 0, -1, false);

				// Promote vertices to double because that's what FTriangleMesh wants.
				VertexD.SetNumUninitialized(Vertex->Num());
				for (int32 i = 0; i < VertexD.Num(); i++)
				{
					VertexD[i] = Chaos::FVec3((*Vertex)[i][0], (*Vertex)[i][1], (*Vertex)[i][2]);
				}
				VertexDView = TConstArrayView<Chaos::TVec3<Chaos::FRealDouble>>(VertexD);

				PointNormals = SurfaceMesh.GetPointNormals(VertexDView, false, true);
				SurfaceMesh.BuildBVH(VertexDView, TetBoundaryBVH);


				SurfaceMesh.BuildSpatialHash(VertexDView, SpatialHash, Chaos::FRealDouble(SurfaceProjectionSearchRadius));
			}

			//
			// Do intersection tests against tets, then the surface.
			//

			TArray<TArray<FIntVector4>> Parents; Parents.SetNum(RenderMeshVertices.Num());
			TArray<TArray<FVector4f>> Weights;	 Weights.SetNum(RenderMeshVertices.Num());
			TArray<TArray<FVector3f>> Offsets;	 Offsets.SetNum(RenderMeshVertices.Num());
			TArray<TArray<float>> Masks;		 Masks.SetNum(RenderMeshVertices.Num());
			TArray<int32> LOD0Orphans;

			for (int32 LOD = 0; LOD < RenderMeshVertices.Num(); LOD++)
			{
				Parents[LOD].SetNumUninitialized(RenderMeshVertices[LOD].Num());
				Weights[LOD].SetNumUninitialized(RenderMeshVertices[LOD].Num());
				Offsets[LOD].SetNumUninitialized(RenderMeshVertices[LOD].Num());
				Masks[LOD].SetNumUninitialized(RenderMeshVertices[LOD].Num());
				for (int32 i = 0; i < RenderMeshVertices[LOD].Num(); i++)
				{
					Parents[LOD][i] = FIntVector4(INDEX_NONE);
					Weights[LOD][i] = FVector4f(0);
					Offsets[LOD][i] = FVector3f(0);
					Masks[LOD][i] = 0.0; // Shader does skinning for this vertex
				}
				if (SkeletalMeshLODList.Num() && !SkeletalMeshLODList.Contains(LOD))
				{
					continue;
				}
				TArray<int32> Orphans;
				int32 TetHits = 0;
				int32 TriHits = 0;
				int32 Adoptions = 0;
				int32 NumOrphans = 0;
				int32 NumTetNotCollocated = 0;

				TArray<int32> TetIntersections; TetIntersections.Reserve(64);
				for (int32 i = 0; i < RenderMeshVertices[LOD].Num(); i++)
				{
					const FVector3f& Pos = RenderMeshVertices[LOD][i];
					Chaos::TVec3<Chaos::FReal> PosD(Pos[0], Pos[1], Pos[2]);
					TetIntersections = TetBVH.FindAllIntersections(PosD);
					int32 j = 0;
					for (; j < TetIntersections.Num(); j++)
					{
						const int32 TetIdx = TetIntersections[j];
						if (!Tets[TetIdx].Outside(Pos, 0.f)) // includes boundary
						{
							Chaos::TVector<Chaos::FReal, 4> WeightsD = Tets[TetIdx].GetBarycentricCoordinates(Pos);
							const int32 GlobalTetIndex = UsedTetsIndexToGlobalTetIndex[TetIdx];
							FVector3f EmbeddedPos =
								(*Vertex)[(*Tetrahedron)[GlobalTetIndex][0]] * WeightsD[0] +
								(*Vertex)[(*Tetrahedron)[GlobalTetIndex][1]] * WeightsD[1] +
								(*Vertex)[(*Tetrahedron)[GlobalTetIndex][2]] * WeightsD[2] +
								(*Vertex)[(*Tetrahedron)[GlobalTetIndex][3]] * WeightsD[3];
							if ((Pos - EmbeddedPos).SquaredLength() < UE_SMALL_NUMBER)
							{
								TetHits++;
								Parents[LOD][i] = (*Tetrahedron)[GlobalTetIndex];
								Weights[LOD][i] = FVector4f(WeightsD[0], WeightsD[1], WeightsD[2], WeightsD[3]);
								Offsets[LOD][i] = FVector3f(0);
								Masks[LOD][i] = 1.0; // Shader does sim for this vertex
								break;
							}
							else
							{
								NumTetNotCollocated++;
								if (NumTetNotCollocated == 1)
								{
									UE_LOGF(LogMeshBindings, Error,
										"Vertex position does not collocate with interpolated position, for example LOD %d, SKM vertex %d, tetrahedron %d, distance = %.4f)",
										LOD,
										i,
										TetIdx,
										(Pos - EmbeddedPos).Length());
								}
							}
						}
					}
					if (j == TetIntersections.Num())
					{
						bool bSuccess = false;
						if (bDoSurfaceProjection)
						{
							TArray<Chaos::TTriangleCollisionPoint<Chaos::FRealDouble>> Result;
							//PointClosestTriangleQuery instead of SmoothProject
							if (SurfaceMesh.PointClosestTriangleQuery(SpatialHash, VertexDView, i, TVec3Vert(Pos), Chaos::FRealDouble(SurfaceProjectionSearchRadius), Chaos::FRealDouble(SurfaceProjectionSearchRadius),
								[](const int32 PointIndex, const int32 TriangleIndex)->bool
								{
									// use all nearby triangles
									return true;
								},
								Result))
							{
								if (Result.Num() > 0)
								{
									const Chaos::TTriangleCollisionPoint<Chaos::FRealDouble>& CollisionPoint = Result[0];
									const FIntVector& Tri = Triangles[CollisionPoint.Indices[1]];
									TriHits++;
									Parents[LOD][i][0] = Tri[0];
									Parents[LOD][i][1] = Tri[1];
									Parents[LOD][i][2] = Tri[2];
									Parents[LOD][i][3] = INDEX_NONE;

									Weights[LOD][i][0] = CollisionPoint.Bary[1];
									Weights[LOD][i][1] = CollisionPoint.Bary[2];
									Weights[LOD][i][2] = CollisionPoint.Bary[3];
									Weights[LOD][i][3] = 0.0;

									const FVector3f EmbeddedPos =
										Weights[LOD][i][0] * Vertex->GetConstArray()[Tri[0]] +
										Weights[LOD][i][1] * Vertex->GetConstArray()[Tri[1]] +
										Weights[LOD][i][2] * Vertex->GetConstArray()[Tri[2]];
									Offsets[LOD][i] = EmbeddedPos - Pos;
									Masks[LOD][i] = 1.0; // Shader does sim for this vertex
									bSuccess = true;
								}
							}
						}
						if (!bSuccess)
						{
							// Despair...
							Orphans.Add(i);

							Parents[LOD][i][0] = INDEX_NONE;
							Parents[LOD][i][1] = INDEX_NONE;
							Parents[LOD][i][2] = INDEX_NONE;
							Parents[LOD][i][3] = INDEX_NONE;

							Weights[LOD][i][0] = 0.0;
							Weights[LOD][i][1] = 0.0;
							Weights[LOD][i][2] = 0.0;
							Weights[LOD][i][3] = 0.0;

							Offsets[LOD][i][0] = 0.0;
							Offsets[LOD][i][1] = 0.0;
							Offsets[LOD][i][2] = 0.0;

							Masks[LOD][i] = 0.0; // Shader does skinning for this vertex
						}
					} // if !TetIntersections
				} // end for all vertices

				// 
				// Advancing front orphan reparenting
				//
				if (!RenderMeshNeighborNodes.IsValidIndex(LOD))
				{
					continue;
				}
				const TArray<TArray<uint32>>& NeighborNodes = RenderMeshNeighborNodes[LOD];
				if (LOD == 0)
				{
					LOD0Orphans = Orphans;
				}
				TBitArray<> IsOrphan(false, RenderMeshVertices[LOD].Num());
				while (bDoOrphanReparenting && Orphans.Num())
				{
					for (int32 Orphan : Orphans)
					{
						IsOrphan[Orphan] = true;
					}
					// Find the orphan with the fewest number of orphan neighbors, and the 
					// most non-orphans in their 1 ring.
					int32 Orphan = INDEX_NONE;
					int32 NumOrphanNeighbors = TNumericLimits<int32>::Max();
					int32 NumNonOrphanNeighbors = 0;
					for (int32 i = 0; i < Orphans.Num(); i++)
					{
						int32 CurrOrphan = Orphans[i];
						if (!NeighborNodes.IsValidIndex(CurrOrphan))
						{
							continue;
						}
						const TArray<uint32>& Neighbors = NeighborNodes[CurrOrphan];
						int32 OrphanCount = 0;
						int32 NonOrphanCount = 0;
						for (int32 j = 0; j < Neighbors.Num(); j++)
						{
							if (IsOrphan[Neighbors[j]])
							{
								OrphanCount++;
							}
							else
							{
								NonOrphanCount++;
							}
						}
						if (OrphanCount <= NumOrphanNeighbors && NonOrphanCount > NumNonOrphanNeighbors)
						{
							Orphan = CurrOrphan;
							NumOrphanNeighbors = OrphanCount;
							NumNonOrphanNeighbors = NonOrphanCount;
						}
					}
					if (Orphan == INDEX_NONE)
					{
						// We only have orphans with no neighbors left.
						break;
					}
					const FVector3f& Pos = RenderMeshVertices[LOD][Orphan];
					Chaos::TVec3<Chaos::FReal> PosD(Pos[0], Pos[1], Pos[2]);

					// Use the parent simplices of non-orphan neighbors as test candidates.
					Chaos::FReal CurrDist = TNumericLimits<Chaos::FReal>::Max();
					const TArray<uint32>& Neighbors = NeighborNodes[Orphan];
					bool FoundBinding = false;
					for (int32 i = 0; i < Neighbors.Num(); i++)
					{
						const uint32 Neighbor = Neighbors[i];
						if (IsOrphan[Neighbor])
						{
							continue;
						}

						const FIntVector4& P = Parents[LOD][Neighbor];
						int32 NumValid = 0;
						for (int32 j = 0; j < 4; j++)
						{
							NumValid += P[j] != INDEX_NONE ? 1 : 0;
						}

						if (NumValid == 0)
						{
							continue;
						}
						else
						{
							// Find tets that share parent indices
							for (int32 j = 0; j < 4; j++)
							{
								const int32 ParentIdx = P[j];
								if (IncidentElements->GetConstArray().IsValidIndex(ParentIdx))
								{
									const TArray<int32>& NeighborTets = (*IncidentElements)[ParentIdx];
									for (int32 k = 0; k < NeighborTets.Num(); k++)
									{
										const int32 TetIdx = NeighborTets[k];
										const int32 UsedTetIdx = GlobalTetIndexToUsedTetsIndex[TetIdx];
										if (ensure(Tets.IsValidIndex(UsedTetIdx)))
										{
											const Chaos::FTetrahedron& Tet = Tets[UsedTetIdx];

											Chaos::TVec4<Chaos::FReal> W;
											Chaos::TVec3<Chaos::FReal> EmbeddedPos = Tet.FindClosestPointAndBary(PosD, W, 0 /*Tolerance*/); //Tolerance should be small negative number or zero
											Chaos::TVec3<Chaos::FReal> O = EmbeddedPos - PosD;
											Chaos::FReal Dist = O.SquaredLength();
											if (Dist < CurrDist) // Closest neighbor tet
											{
												CurrDist = Dist;
												Parents[LOD][Orphan] = (*Tetrahedron)[TetIdx];
												Weights[LOD][Orphan] = FVector4f(W[0], W[1], W[2], W[3]);
												Offsets[LOD][Orphan] = FVector3f(O[0], O[1], O[2]);
												Masks[LOD][i] = 1.0; // Shader does sim for this vertex
												FoundBinding = true;
											}
										}
									}
								}
							}
						}
					} // end for all neighbors

					// Whether or not we successfully reparented, remove the orphan from the list.
					IsOrphan[Orphan] = false;
					Orphans.Remove(Orphan);
					if (FoundBinding)
					{
						Adoptions++;
					}
					else
					{
						NumOrphans++;
					}
				} // end while(Orphans)
				NumOrphans += Orphans.Num();

				if (Orphans.Num() > 0)
				{
					UE_LOGF(LogMeshBindings, Error,
						"'%ls' - Generated mesh bindings between tet mesh and %ls mesh of '%ls' LOD %d - stats:\n"
							"    Render vertices num: %d\n"
							"    Vertices in tetrahedra: %d\n"
							"    Vertices bound to tet surface: %d\n"
							"    Orphaned vertices reparented: %d\n"
							"    Vertices orphaned: %d",
						*GetName().ToString(),
						bUseSkeletalMeshImportModel ? TEXT("import") : TEXT("render"),
						*MeshId, LOD,
						RenderMeshVertices[LOD].Num(), TetHits, TriHits, Adoptions, NumOrphans);
					Context.Warning(FString::Printf(
						TEXT("GenerateSurfaceBindings Node: There are %d orphans."),
						Orphans.Num()),
						this, Out);
				}
				else
				{
					UE_LOGF(LogMeshBindings, Display,
						"'%ls' - Generated mesh bindings between tet mesh and %ls mesh of '%ls' LOD %d - stats:\n"
							"    Render vertices num: %d\n"
							"    Vertices in tetrahedra: %d\n"
							"    Vertices bound to tet surface: %d\n"
							"    Orphaned vertices reparented: %d\n"
							"    Vertices orphaned: %d",
						*GetName().ToString(),
						bUseSkeletalMeshImportModel ? TEXT("import") : TEXT("render"),
						* MeshId, LOD,
						RenderMeshVertices[LOD].Num(), TetHits, TriHits, Adoptions, NumOrphans);
				}
				if (NumTetNotCollocated)
				{
					UE_LOGF(LogMeshBindings, Error,
						"%d vertex positions do not collocate with interpolated position for LOD %d",
						NumTetNotCollocated,
						LOD);
				}

			} // end for all LOD

			// Stash bindings in the geometry collection
			GeometryCollection::Facades::FTetrahedralBindings TetBindings(*InCollection);
			TetBindings.DefineSchema();
			FName MeshName(*MeshId, MeshId.Len());
			for (int32 LOD = 0; LOD < RenderMeshVertices.Num(); LOD++)
			{
				TetBindings.AddBindingsGroup(/*TetMeshIdx = */ 0, MeshName, LOD);
				TetBindings.SetBindingsData(Parents[LOD], Weights[LOD], Offsets[LOD], Masks[LOD]);
			}

			//Write DynamicMesh			
			if (RenderMeshVertices.Num())
			{
				OutSKMDynamicMesh3.EnableAttributes();
				OutSKMDynamicMesh3.EnableVertexColors(FVector3f(1, 0, 0));
				TBitArray<> WasOrphan(false, RenderMeshVertices[0].Num());
				for (int32 OrphanIdx : LOD0Orphans)
				{
					WasOrphan[OrphanIdx] = true;
				}
				for (int32 VertexIndex = 0; VertexIndex < RenderMeshVertices[0].Num(); ++VertexIndex)
				{
					FVertexInfo VertexInfo;
					VertexInfo.Position = DoubleVert(RenderMeshVertices[0][VertexIndex]);
					VertexInfo.bHaveC = true;
					if (Parents[0][VertexIndex][0] == INDEX_NONE)
					{
						VertexInfo.Color = FVector3f(1, 0, 0); //red if orphan
					}
					else
					{
						if (Parents[0][VertexIndex][3] == INDEX_NONE)
						{
							VertexInfo.Color = FVector3f(0, 0, 1); //blue if on surface
						}
						else
						{
							VertexInfo.Color = FVector3f(0, 1, 0); //green if in tet
						}
						if (WasOrphan[VertexIndex])
						{
							VertexInfo.Color += FVector3f(1, 0, 0); //add red if was orphan
						}
					}
					OutSKMDynamicMesh3.AppendVertex(VertexInfo);
				}
				
				for (int32 TriangleIndex = 0; TriangleIndex < RenderMeshTriangles[0].Num(); ++TriangleIndex)
				{
					OutSKMDynamicMesh3.AppendTriangle(FIndex3i(
						RenderMeshTriangles[0][TriangleIndex][0],
						RenderMeshTriangles[0][TriangleIndex][1],
						RenderMeshTriangles[0][TriangleIndex][2])
					);
				}
				// Compute normals
				OutSKMDynamicMesh3.EnableVertexNormals(FVector3f(1, 0, 0));
				FMeshNormals MeshNormals(&OutSKMDynamicMesh3);
				MeshNormals.ComputeVertexNormals();
				for (int32 VertexIndex = 0; VertexIndex < RenderMeshVertices[0].Num(); ++VertexIndex)
				{
					OutSKMDynamicMesh3.SetVertexNormal(VertexIndex, FloatVert(MeshNormals[VertexIndex]));
				}
			}
		}
		SetValue<const FManagedArrayCollection&>(Context, *InCollection, &Collection);
		SetValue(Context, OutSKMDynamicMesh, &SKMDynamicMesh);
	}
}