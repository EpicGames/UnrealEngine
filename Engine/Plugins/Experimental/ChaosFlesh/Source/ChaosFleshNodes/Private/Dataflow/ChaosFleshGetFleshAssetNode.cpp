// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshGetFleshAssetNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFleshGetFleshAssetNode)

void FGetFleshAssetDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Output))
	{
		FManagedArrayCollection Collection;
		SetValue(Context, MoveTemp(Collection), &Output);

		const UFleshAsset* FleshAssetValue = FleshAsset;
		if (!FleshAssetValue)
		{
			if (const UE::Dataflow::FEngineContext* EngineContext = Context.AsType<UE::Dataflow::FEngineContext>())
			{
				FleshAssetValue = Cast<UFleshAsset>(EngineContext->Owner);
			}
		}

		if (FleshAssetValue)
		{
			if (TSharedPtr<const FFleshCollection> AssetCollection = FleshAssetValue->GetFleshCollection())
			{
				SetValue(Context, (const FManagedArrayCollection&)(*AssetCollection), &Output);
			}
		}
	}
}
