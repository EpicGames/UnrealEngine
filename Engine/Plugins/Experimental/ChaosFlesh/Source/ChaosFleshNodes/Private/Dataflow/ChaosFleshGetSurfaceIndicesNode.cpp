// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshGetSurfaceIndicesNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFleshGetSurfaceIndicesNode)

void FGetSurfaceIndicesNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	const FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
	if (const TManagedArray<FVector3f>* Vertex = InCollection.FindAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup))
	{
		FDataflowVertexSelection SurfaceVertexSelectionOut;
		SurfaceVertexSelectionOut.Initialize(Vertex->Num(), false);
		TArray<int32> SurfaceIndicesLocal;
		if (const TManagedArray<FIntVector>* Indices = InCollection.FindAttribute<FIntVector>("Indices", FGeometryCollection::FacesGroup))
		{
			if (FindInput(&GeometryGroupGuidsIn) && FindInput(&GeometryGroupGuidsIn)->GetConnection())
			{
				const TArray<FString> GeometryGroupGuidsLocal = GetValue<TArray<FString>>(Context, &GeometryGroupGuidsIn);
				const TManagedArray<int32>* IndicesStart = InCollection.FindAttribute<int32>("FaceStart", FGeometryCollection::GeometryGroup);
				const TManagedArray<int32>* IndicesCount = InCollection.FindAttribute<int32>("FaceCount", FGeometryCollection::GeometryGroup);
				if (const TManagedArray<FString>* Guids = InCollection.FindAttribute<FString>("Guid", FGeometryCollection::GeometryGroup))
				{
					for (int32 Idx = 0; Idx < IndicesStart->Num(); Idx++)
					{
						if (GeometryGroupGuidsLocal.Num() && Guids)
						{
							if (GeometryGroupGuidsLocal.Contains((*Guids)[Idx]))
							{
								for (int32 i = (*IndicesStart)[Idx]; i < (*IndicesStart)[Idx] + (*IndicesCount)[Idx]; i++)
								{
									for (int32 j = 0; j < 3; j++)
									{
										SurfaceIndicesLocal.AddUnique((*Indices)[i][j]);
									}
								}
							}
						}
					}
				}
			}
			else
			{
				for (int32 i = 0; i < Indices->Num(); i++)
				{
					for (int32 j = 0; j < 3; j++)
					{
						SurfaceIndicesLocal.AddUnique((*Indices)[i][j]);
					}
				}
			}
			SurfaceVertexSelectionOut.SetFromArray(SurfaceIndicesLocal);
		}
		SetValue(Context, MoveTemp(SurfaceVertexSelectionOut), &SurfaceVertexSelection);
	}
	else
	{
		SetValue(Context, FDataflowVertexSelection(), &SurfaceVertexSelection);
	}
}
