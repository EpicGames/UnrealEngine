// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshIsolateComponentNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFleshIsolateComponentNode)

void FIsolateComponentNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{	
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		TArray<int32> DeleteList;
		TManagedArray<FIntVector>* Indices = InCollection.FindAttribute<FIntVector>("Indices", FGeometryCollection::FacesGroup);
		TManagedArray<bool>*FaceVisibility = InCollection.FindAttribute<bool>("Visible", FGeometryCollection::FacesGroup);
		TManagedArray<int32>* FaceStart = InCollection.FindAttribute<int32>("FaceStart", FGeometryCollection::GeometryGroup);
		TManagedArray<int32>* FaceCount = InCollection.FindAttribute<int32>("FaceCount", FGeometryCollection::GeometryGroup);
		if (Indices && FaceVisibility && FaceStart && FaceCount)
		{
			FaceVisibility->Fill(false);
			TSet<int32> GeometrySet;
			TArray<FString> StrArray;
			TargetGeometryIndex.ParseIntoArray(StrArray, *FString(" "));
			for (FString GeometryIdx : StrArray)
			{
				if (GeometryIdx.Len() && FCString::IsNumeric(*GeometryIdx))
				{
					GeometrySet.Add(FCString::Atoi(*GeometryIdx));
				}
			}
			for (TSet<int32>::TConstIterator It = GeometrySet.CreateConstIterator(); It; ++It)
			{
				for (int32 FaceIdx = (*FaceStart)[*It]; FaceIdx < (*FaceStart)[*It] + (*FaceCount)[*It]; FaceIdx++)
				{
					(*FaceVisibility)[FaceIdx] = true;
					DeleteList.Add(FaceIdx);
				}
			}
			if (bDeleteHiddenFaces)
			{
				InCollection.RemoveElements(FGeometryCollection::FacesGroup, DeleteList);
			}
		}
		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}
