// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshKinematicConstraintNode.h"

#include "Engine/SkeletalMesh.h"
#include "GeometryCollection/Facades/CollectionKinematicBindingFacade.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFleshKinematicConstraintNode)


void FKinematicInitializationDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<DataType>(&Collection))
	{
		DataType InCollection = GetValue<DataType>(Context, &Collection);

		if (TManagedArray<FVector3f>* Vertices = InCollection.FindAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup))
		{
			if (FindInput(&VertexIndicesIn) && FindInput(&VertexIndicesIn)->GetConnection())
			{
				GeometryCollection::Facades::FVertexBoneWeightsFacade VertexBoneWeightsFacade(InCollection);
				for (int32 SelectionIndex : GetValue<TArray<int32>>(Context, &VertexIndicesIn))
				{
					if (Vertices->IsValidIndex(SelectionIndex))
					{
						VertexBoneWeightsFacade.SetVertexKinematic(SelectionIndex);
					}
				}
			}
			else if (TObjectPtr<USkeletalMesh> SkeletalMesh = GetValue<TObjectPtr<USkeletalMesh>>(Context, &SkeletalMeshIn))
			{
				int32 IndexValue = GetValue<int32>(Context, &BoneIndexIn);
				if (IndexValue != INDEX_NONE)
				{
					TArray<FTransform> ComponentPose;
					UE::Dataflow::Animation::GlobalTransforms(SkeletalMesh->GetRefSkeleton(), ComponentPose);

					TArray<int32> BranchIndices;
					if (SkeletalSelectionMode == ESkeletalSeletionMode::Dataflow_SkeletalSelection_Branch)
					{
						TArray<int32> ToProcess;
						int32 CurrentIndex = IndexValue;
						while (SkeletalMesh->GetRefSkeleton().IsValidIndex(CurrentIndex))
						{
							TArray<int32> Buffer;
							SkeletalMesh->GetRefSkeleton().GetDirectChildBones(CurrentIndex, Buffer);

							if (Buffer.Num())
							{
								ToProcess.Append(Buffer);
							}

							BranchIndices.Add(CurrentIndex);

							CurrentIndex = INDEX_NONE;
							if (ToProcess.Num())
							{
								CurrentIndex = ToProcess.Pop();
							}
						}
					}
					else // ESkeletalSeletionMode::Dataflow_SkeletalSelection_Single
					{
						BranchIndices.Add(IndexValue);
					}

					TSet<int32> ProcessedVertices;
					for (const int32 Index : BranchIndices)
					{
						TArray<int32> BoundVerts;
						TArray<float> BoundWeights;

						if (0 <= Index && Index < ComponentPose.Num())
						{
							FVector3f BonePosition(ComponentPose[Index].GetTranslation());

							int NumVertices = Vertices->Num();
							for (int i = Vertices->Num() - 1; i > 0; i--)
							{
								if ((BonePosition - (*Vertices)[i]).Length() < Radius)
								{
									if (!ProcessedVertices.Contains(i))
									{
										ProcessedVertices.Add(i);
										BoundVerts.Add(i);
										BoundWeights.Add(1.0);
									}
								}
							}

							if (BoundVerts.Num())
							{
								GeometryCollection::Facades::FKinematicBindingFacade Kinematics(InCollection);
								Kinematics.AddKinematicBinding(Kinematics.SetBoneBindings(Index, BoundVerts, BoundWeights));
							}
						}
					}
					GeometryCollection::Facades::FVertexBoneWeightsFacade(InCollection).AddBoneWeightsFromKinematicBindings();
				}
			}
		}
		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}
