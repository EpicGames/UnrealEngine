// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshKinematicMuscleAttachmentsNode.h"

#include "Engine/SkeletalMesh.h"
#include "GeometryCollection/Facades/CollectionKinematicBindingFacade.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"
#include "GeometryCollection/Facades/CollectionTransformFacade.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "BoneWeights.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFleshKinematicMuscleAttachmentsNode)


void FKinematicMuscleAttachmentsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<DataType>(&Collection))
	{
		DataType InCollection = GetValue<DataType>(Context, &Collection);

		if (TManagedArray<FVector3f>* Vertices = InCollection.FindAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup))
		{
			if (FindInput(&OriginVertexIndicesIn) && FindInput(&OriginVertexIndicesIn)->GetConnection() && FindInput(&InsertionVertexIndicesIn) && FindInput(&InsertionVertexIndicesIn)->GetConnection())
			{
				TArray<int32> BoundVerts;

				for (int32 SelectionIndex : GetValue<TArray<int32>>(Context, &OriginVertexIndicesIn))
				{
					if (0 <= SelectionIndex && SelectionIndex < Vertices->Num())
					{
						BoundVerts.AddUnique(SelectionIndex);
					}
				}
				for (int32 SelectionIndex : GetValue<TArray<int32>>(Context, &InsertionVertexIndicesIn))
				{
					if (0 <= SelectionIndex && SelectionIndex < Vertices->Num())
					{
						BoundVerts.AddUnique(SelectionIndex);
					}
				}
				if (TObjectPtr<USkeletalMesh> SkeletalMesh = GetValue<TObjectPtr<USkeletalMesh>>(Context, &SkeletalMeshIn))
				{
					FSkeletalMeshRenderData* RenderData = SkeletalMesh->GetResourceForRendering();
					if (RenderData->LODRenderData.Num())
					{
						//Grab vertices only, no elements
						FSkeletalMeshLODRenderData* LODRenderData = &RenderData->LODRenderData[0];
						const FPositionVertexBuffer& PositionVertexBuffer =
							LODRenderData->StaticVertexBuffers.PositionVertexBuffer;
						//Grab skin weights
						const FSkinWeightVertexBuffer* SkinWeightVertexBuffer = LODRenderData->GetSkinWeightVertexBuffer();
						const int32 MaxBoneInfluences = SkinWeightVertexBuffer->GetMaxBoneInfluences();
						TArray<FTransform> ComponentPose;
						UE::Dataflow::Animation::GlobalTransforms(SkeletalMesh->GetRefSkeleton(), ComponentPose);
						TArray<TArray<int32>> BoneBoundVerts;
						TArray<TArray<float>> BoneBoundWeights;
						BoneBoundVerts.SetNum(ComponentPose.Num());
						BoneBoundWeights.SetNum(ComponentPose.Num());
						if (!PositionVertexBuffer.GetNumVertices())
						{
							return;
						}
						GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(InCollection);
						const TMap<FString, int32> BoneNameToIndex = TransformFacade.BoneNameIndexMap();
						auto DoubleVert = [](FVector3f V) { return FVector3d(V.X, V.Y, V.Z); };
						//ToDo(Chaosflesh): change this to particle bounding box query
						for (int32 i = 0; i < BoundVerts.Num(); i++)
						{
							int32 ClosestPointIndex = 0;
							double MinDistance = DBL_MAX;
							for (uint32 j = 0; j < PositionVertexBuffer.GetNumVertices(); j++)
							{
								const FVector3f& Pos = PositionVertexBuffer.VertexPosition(j);
								double Distance = FVector::Distance(DoubleVert((*Vertices)[BoundVerts[i]]), DoubleVert(Pos));
								if (Distance < MinDistance)
								{
									ClosestPointIndex = j;
									MinDistance = Distance;
								}
							}
							int32 SectionIndex;
							int32 VertIndex;
							LODRenderData->GetSectionFromVertexIndex(ClosestPointIndex, SectionIndex, VertIndex);

							check(SectionIndex < LODRenderData->RenderSections.Num());
							const FSkelMeshRenderSection& Section = LODRenderData->RenderSections[SectionIndex];
							int32 BufferVertIndex = Section.GetVertexBufferIndex() + VertIndex;
							for (int32 InfluenceIndex = 0; InfluenceIndex < MaxBoneInfluences; InfluenceIndex++)
							{
								const int32 BoneIndex = Section.BoneMap[SkinWeightVertexBuffer->GetBoneIndex(BufferVertIndex, InfluenceIndex)];
								const float	Weight = (float)SkinWeightVertexBuffer->GetBoneWeight(BufferVertIndex, InfluenceIndex) * UE::AnimationCore::InvMaxRawBoneWeightFloat;
								if (Weight > float(0) && 0 <= BoneIndex && BoneIndex < ComponentPose.Num())
								{
									FString BoneName = SkeletalMesh->GetRefSkeleton().GetBoneName(BoneIndex).ToString();
									BoneBoundVerts[BoneIndex].Add(BoundVerts[i]);
									BoneBoundWeights[BoneIndex].Add(Weight);
								}
							}
						}
						for (int32 BoneIndex = 0; BoneIndex < ComponentPose.Num(); ++BoneIndex)
						{
							if (BoneBoundVerts[BoneIndex].Num())
							{
								FString BoneName = SkeletalMesh->GetRefSkeleton().GetBoneName(BoneIndex).ToString();
								//get local coords of bound verts
								typedef GeometryCollection::Facades::FKinematicBindingFacade FKinematics;
								FKinematics Kinematics(InCollection); Kinematics.DefineSchema();
								if (Kinematics.IsValid() && BoneNameToIndex.Contains(BoneName))
								{
									FKinematics::FBindingKey Binding = Kinematics.SetBoneBindings(BoneNameToIndex[BoneName], BoneBoundVerts[BoneIndex], BoneBoundWeights[BoneIndex]);
									TManagedArray<TArray<FVector3f>>& LocalPos = InCollection.AddAttribute<TArray<FVector3f>>("LocalPosition", Binding.GroupName);
									Kinematics.AddKinematicBinding(Binding);

									auto FloatVert = [](FVector3d V) { return FVector3f(V.X, V.Y, V.Z); };
									LocalPos[Binding.Index].SetNum(BoneBoundVerts[BoneIndex].Num());
									for (int32 i = 0; i < BoneBoundVerts[BoneIndex].Num(); i++)
									{
										FVector3f Temp = (*Vertices)[BoneBoundVerts[BoneIndex][i]];
										LocalPos[Binding.Index][i] = FloatVert(ComponentPose[BoneIndex].InverseTransformPosition(DoubleVert(Temp)));
									}
								}
							}
						}
						GeometryCollection::Facades::FVertexBoneWeightsFacade(InCollection).AddBoneWeightsFromKinematicBindings();
					}
				}
				else
				{
					if (BoundVerts.Num())
					{
						TArray<float> BoundWeights;
						BoundWeights.Init(1.0, BoundVerts.Num());
						GeometryCollection::Facades::FKinematicBindingFacade Kinematics(InCollection);
						Kinematics.AddKinematicBinding(Kinematics.SetBoneBindings(INDEX_NONE, BoundVerts, BoundWeights));
					}
				}
			}
		}
		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}
