// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshKinematicSkeletonConstraintNode.h"

#include "ChaosFlesh/ChaosFleshCollectionFacade.h"
#include "ChaosFlesh/TetrahedralCollection.h"
#include "Animation/Skeleton.h"
#include "GeometryCollection/Facades/CollectionKinematicBindingFacade.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/Facades/CollectionTransformFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFleshKinematicSkeletonConstraintNode)

void FKinematicSkeletonConstraintDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<DataType>(&Collection))
	{
		TUniquePtr<FFleshCollection> InCollection(GetValue<DataType>(Context, &Collection).NewCopy<FFleshCollection>());
		Chaos::FFleshCollectionFacade TetCollection(*InCollection.Get());

		TObjectPtr<USkeleton> Skeleton = GetValue<TObjectPtr<USkeleton>>(Context, &SkeletonIn);
		if (Skeleton && TetCollection.IsTetrahedronValid() )
		{
			//parse exclusion list to find bones to skip
			TArray<FString> StrArray;
			ExclusionList.ParseIntoArray(StrArray, *FString(" "));				
			int32 NumTets = TetCollection.Tetrahedron.Num();

			TArray<FTransform> ComponentPose;
			UE::Dataflow::Animation::GlobalTransforms(Skeleton->GetReferenceSkeleton(), ComponentPose);

			TArray<FVector3f> Vertex = TetCollection.Vertex.Get().GetConstArray();
			TetCollection.ComponentSpaceVertices(Vertex);

			TArray<bool> VertAdded;
			VertAdded.Init(false, Vertex.Num());
			auto DoSkipBoneIndex = [&StrArray, &Skeleton](int32 BoneIndex)
			{
				bool Skip = false;
				FString BoneName = Skeleton->GetReferenceSkeleton().GetBoneName(BoneIndex).ToString();
				for (FString Elem : StrArray)
				{
					if (BoneName.Contains(Elem))
					{
						Skip = true;
						break;
					}
				}
				return Skip;
			};
			GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(*InCollection);
			const TMap<FString, int32> BoneNameMap = TransformFacade.BoneNameIndexMap();

			for (int32 b = 0; b < Skeleton->GetReferenceSkeleton().GetNum(); ++b)
			{
				FVector3f BonePosition(ComponentPose[b].GetTranslation());
				int32 ParentIndex=Skeleton->GetReferenceSkeleton().GetParentIndex(b);
				if (!(ParentIndex == INDEX_NONE || DoSkipBoneIndex(b) || DoSkipBoneIndex(ParentIndex)))
				{
					FVector3f ParentPosition(ComponentPose[ParentIndex].GetTranslation());
					FVector3f RayDir = ParentPosition - BonePosition;
					Chaos::FReal Length = RayDir.Length();
					RayDir.Normalize();

					if (Length > Chaos::FReal(1e-8)) 
					{
						TSet<int32> BoneVertSet;
						for (int32 t = 0; t < NumTets; t++) 
						{
							int32 i = TetCollection.Tetrahedron[t][0];
							int32 j = TetCollection.Tetrahedron[t][1];
							int32 k = TetCollection.Tetrahedron[t][2];
							int32 l = TetCollection.Tetrahedron[t][3];

							TArray<Chaos::TVec3<Chaos::FRealSingle>> InVertices;
							InVertices.SetNum(4);
							InVertices[0][0] = Vertex[i].X; InVertices[0][1] = Vertex[i].Y; InVertices[0][2] = Vertex[i].Z;
							InVertices[1][0] = Vertex[j].X; InVertices[1][1] = Vertex[j].Y; InVertices[1][2] = Vertex[j].Z;
							InVertices[2][0] = Vertex[k].X; InVertices[2][1] = Vertex[k].Y; InVertices[2][2] = Vertex[k].Z;
							InVertices[3][0] = Vertex[l].X; InVertices[3][1] = Vertex[l].Y; InVertices[3][2] = Vertex[l].Z;
							Chaos::FConvex ConvexTet(InVertices, Chaos::FReal(0));
							Chaos::FReal OutTime;
							Chaos::FVec3 OutPosition, OutNormal;
							int32 OutFaceIndex;
							bool KeepTet = ConvexTet.Raycast(BonePosition, RayDir, Length, Chaos::FReal(0), OutTime, OutPosition, OutNormal, OutFaceIndex);
							if (KeepTet) 
							{	
								for (int32 c = 0; c < 4; ++c)
								{	
									if (!VertAdded[TetCollection.Tetrahedron[t][c]])
									{
										VertAdded[TetCollection.Tetrahedron[t][c]] = true;
										BoneVertSet.Add(TetCollection.Tetrahedron[t][c]);
									}
								}
							}
						}

						TArray<int32> BoundVerts = BoneVertSet.Array();
						TArray<float> BoundWeights;
						BoundWeights.Init(float(1), BoundVerts.Num());
						if (BoundVerts.Num())
						{
							//get local coords of bound verts
							typedef GeometryCollection::Facades::FKinematicBindingFacade FKinematics;
							FKinematics Kinematics(*InCollection.Get()); Kinematics.DefineSchema();
							FString ParentBoneName = Skeleton->GetReferenceSkeleton().GetBoneName(ParentIndex).ToString();
							if (Kinematics.IsValid() && BoneNameMap.Contains(ParentBoneName))
							{
								FKinematics::FBindingKey Binding = Kinematics.SetBoneBindings(BoneNameMap[ParentBoneName], BoundVerts, BoundWeights);
								TManagedArray<TArray<FVector3f>>& LocalPos = InCollection->AddAttribute<TArray<FVector3f>>("LocalPosition", Binding.GroupName);
								Kinematics.AddKinematicBinding(Binding);
								auto DoubleVert = [](FVector3f V) { return FVector3d(V.X, V.Y, V.Z); };
								auto FloatVert = [](FVector3d V) { return FVector3f(V.X, V.Y, V.Z); };
								LocalPos[Binding.Index].SetNum(BoundVerts.Num());
								for (int32 i = 0; i < BoundVerts.Num(); i++)
								{
									FVector3f Temp = Vertex[BoundVerts[i]];
									LocalPos[Binding.Index][i] = FloatVert(ComponentPose[ParentIndex].InverseTransformPosition(DoubleVert(Temp)));
								}
							}
						}
					}
				}
			}
			GeometryCollection::Facades::FVertexBoneWeightsFacade(*InCollection.Get()).AddBoneWeightsFromKinematicBindings();
		}
		FManagedArrayCollection& InBaseCollection = *InCollection.Release();
		SetValue(Context, MoveTemp(InBaseCollection), &Collection);
	}
}
