// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshMakeFleshAssetNode.h"

#include "ChaosFlesh/ChaosFleshCollectionFacade.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFleshMakeFleshAssetNode)

#define LOCTEXT_NAMESPACE "ChaosFleshMakeFleshAssetNode"

FMakeFleshAssetNode::FMakeFleshAssetNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	// Start with Lod0
	for (int32 Index = 0; Index < NumInitialCollectionLods; ++Index)
	{
		AddPins();
	}
	check(GetNumInputs() == NumRequiredInputs + NumInitialCollectionLods);  // Update NumRequiredInputs if you add more Inputs. This is used by Serialize.
	RegisterOutputConnection(&Asset);
}

void FMakeFleshAssetNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput*) const
{		
	TArray<TUniquePtr<FFleshCollection>> InClothCollections = GetCleanedCollectionLodValues(Context);
	const int32 LODIndex = 0; //TODO: support multiple LODs

	TObjectPtr<UFleshAsset> OutFleshAsset = NewObject<UFleshAsset>();
	if (InClothCollections.Num() > 0)
	{
		OutFleshAsset->SetFleshCollection(MoveTemp(InClothCollections[LODIndex]));
		SetValue(Context, OutFleshAsset, &Asset);
		return;
	}
	SetValue(Context, OutFleshAsset, &Asset);
}

TArray<UE::Dataflow::FPin> FMakeFleshAssetNode::AddPins()
{
	const int32 Index = CollectionLods.AddDefaulted();
	const FDataflowInput& Input = RegisterInputArrayConnection(GetConnectionReference(Index));
	return { { UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() } };
}

TArray<UE::Dataflow::FPin> FMakeFleshAssetNode::GetPinsToRemove() const
{
	const int32 Index = CollectionLods.Num() - 1;
	check(CollectionLods.IsValidIndex(Index));
	if (const FDataflowInput* const Input = FindInput(GetConnectionReference(Index)))
	{
		return { { UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() } };
	}
	return Super::GetPinsToRemove();
}

void FMakeFleshAssetNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	const int32 Index = CollectionLods.Num() - 1;
	check(CollectionLods.IsValidIndex(Index));
#if DO_CHECK
	const FDataflowInput* const Input = FindInput(GetConnectionReference(Index));
	check(Input);
	check(Input->GetName() == Pin.Name);
	check(Input->GetType() == Pin.Type);
#endif
	CollectionLods.SetNum(Index);

	return Super::OnPinRemoved(Pin);
}

TArray<TUniquePtr<FFleshCollection>> FMakeFleshAssetNode::GetCleanedCollectionLodValues(UE::Dataflow::FContext& Context) const
{
	TArray<TUniquePtr<FFleshCollection>> CollectionLodValues;
	CollectionLodValues.Reserve(CollectionLods.Num());

	int32 LastValidLodIndex = INDEX_NONE;
	for (int32 LodIndex = 0; LodIndex < CollectionLods.Num(); ++LodIndex)
	{
		TUniquePtr<FFleshCollection> FleshCollection(GetValue<FManagedArrayCollection>(Context, GetConnectionReference(LodIndex)).NewCopy<FFleshCollection>());
		Chaos::FFleshCollectionFacade FleshFacade(*FleshCollection); 
		if (FleshFacade.IsValid())
		{
			LastValidLodIndex = LodIndex;
			CollectionLodValues.Emplace(MoveTemp(FleshCollection));
			return CollectionLodValues;
		}
		else if (LastValidLodIndex >= 0)
		{
			TUniquePtr<FFleshCollection> FleshCollectionCopy(CollectionLodValues[LastValidLodIndex]->NewCopy<FFleshCollection>());
			CollectionLodValues.Emplace(MoveTemp(FleshCollectionCopy));
		}
		else
		{
			break;
		}	
	}
	return CollectionLodValues;
}

UE::Dataflow::TConnectionReference<FManagedArrayCollection> FMakeFleshAssetNode::GetConnectionReference(int32 Index) const
{
	return { &CollectionLods[Index], Index, &CollectionLods };
}

void FMakeFleshAssetNode::PostSerialize(const FArchive& Ar)
{
	// because we add pins we need to make sure we restore them when loading
	// to make sure they can get properly reconnected

	if (Ar.IsLoading())
	{
		if (CollectionLods.Num() < NumInitialCollectionLods)
		{
			CollectionLods.SetNum(NumInitialCollectionLods);  // In case the FManagedArrayCollection wasn't serialized with the node (pre the WithSerializer trait)
		}
		for (int32 Index = 0; Index < NumInitialCollectionLods; ++Index)
		{
			check(FindInput(GetConnectionReference(Index)));
		}

		for (int32 Index = NumInitialCollectionLods; Index < CollectionLods.Num(); ++Index)
		{
			FindOrRegisterInputArrayConnection(GetConnectionReference(Index));
		}
		if (Ar.IsTransacting())
		{
			const int32 OrigNumRegisteredInputs = GetNumInputs();
			check(OrigNumRegisteredInputs >= NumRequiredInputs + NumInitialCollectionLods);
			const int32 OrigNumCollections = CollectionLods.Num();
			const int32 OrigNumRegisteredCollections = (OrigNumRegisteredInputs - NumRequiredInputs);
			if (OrigNumRegisteredCollections > OrigNumCollections)
			{
				// Inputs have been removed.
				// Temporarily expand Collections so we can get connection references.
				CollectionLods.SetNum(OrigNumRegisteredCollections);
				for (int32 Index = OrigNumCollections; Index < CollectionLods.Num(); ++Index)
				{
					UnregisterInputConnection(GetConnectionReference(Index));
				}
				CollectionLods.SetNum(OrigNumCollections);
			}
		}
		else
		{
			ensureAlways(CollectionLods.Num() + NumRequiredInputs == GetNumInputs());
		}
	}
}

#undef LOCTEXT_NAMESPACE
