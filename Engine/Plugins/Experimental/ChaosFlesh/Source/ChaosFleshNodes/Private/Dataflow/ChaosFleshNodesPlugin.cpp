// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshNodesPlugin.h"

#include "Dataflow/ChaosFleshAddKinematicParticlesNode.h"
#include "Dataflow/ChaosFleshAppendTetrahedralCollectionNode.h"
#include "Dataflow/ChaosFleshAuthorSceneCollisionCandidatesNode.h"
#include "Dataflow/ChaosFleshCalculateTetrehedralMetricsNode.h"
#include "Dataflow/ChaosFleshCollisionBodyConstraintNode.h"
#include "Dataflow/ChaosFleshComputeFiberFieldNode.h"
#include "Dataflow/ChaosFleshComputeIslandsNode.h"
#include "Dataflow/ChaosFleshComputeMuscleActivationNode.h"
#include "Dataflow/ChaosFleshCreateTetrahedronNode.h"
#include "Dataflow/ChaosFleshFleshAssetTerminalNode.h"
#include "Dataflow/ChaosFleshGenerateOriginInsertionNode.h"
#include "Dataflow/ChaosFleshGenerateSkeletalBindingsNode.h"
#include "Dataflow/ChaosFleshGenerateSurfaceBindingsNode.h"
#include "Dataflow/ChaosFleshGetFleshAssetNode.h"
#include "Dataflow/ChaosFleshGetSurfaceIndicesNode.h"
#include "Dataflow/ChaosFleshIsolateComponentNode.h"
#include "Dataflow/ChaosFleshKinematicConstraintNode.h"
#include "Dataflow/ChaosFleshKinematicMuscleAttachmentsNode.h"
#include "Dataflow/ChaosFleshKinematicSkeletonConstraintNode.h"
#include "Dataflow/ChaosFleshMakeFleshAssetNode.h"
#include "Dataflow/ChaosFleshRadialTetrahedronNode.h"
#include "Dataflow/ChaosFleshTriangleMeshSimulationPropertiesNode.h"
#include "Dataflow/ChaosFleshSkeletalMeshConstraintNode.h"
#include "Dataflow/ChaosFleshSkinSimulationPropertiesNode.h"
#include "Dataflow/ChaosFleshSetFleshBonePositionTargetBindingNode.h"
#include "Dataflow/ChaosFleshSetFleshDefaultPropertiesNode.h"
#include "Dataflow/ChaosFleshSetVertexTetrahedraPositionTargetBindingNode.h"
#include "Dataflow/ChaosFleshSetVertexTrianglePositionTargetBindingNode.h"
#include "Dataflow/ChaosFleshSetVertexVertexPositionTargetBindingNode.h"
#include "Dataflow/ChaosFleshVertexConstraintNode.h"
#include "Dataflow/ChaosFleshVisualizeFiberFieldNode.h"
#include "Dataflow/GeometryCollectionAppendCollectionTransformNode.h"
#include "Dataflow/DataflowCategoryRegistry.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "ChaosFleshNodes"

void IChaosFleshNodesPlugin::StartupModule()
{
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAddKinematicParticlesDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAppendTetrahedralCollectionDataflowNode_v2);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAppendToCollectionTransformAttributeDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAuthorSceneCollisionCandidates);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCalculateTetMetrics);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FComputeFiberFieldNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FComputeFiberStreamlineNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FComputeIslandsNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FComputeMuscleActivationDataNode_v2);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCreateAirTetrahedralConstraintDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCreateAirVolumeConstraintDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCreateTetrahedronDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCurveSamplingAnimationAssetTerminalNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDeleteFleshVerticesDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDeleteVertexTrianglePositionTargetBindingDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FFleshAssetTerminalDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGenerateOriginInsertionNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGenerateSkeletalBindings);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGenerateSurfaceBindings);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetFleshAssetDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetSurfaceIndicesNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FIsolateComponentNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FKinematicBodySetupInitializationDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FKinematicInitializationDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FKinematicMuscleAttachmentsDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FKinematicSkeletonConstraintDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FKinematicSkeletalMeshInitializationDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeFleshAssetNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FRadialTetrahedronDataflowNodes);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FReadSkeletalMeshCurvesDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSetCollidableVerticesDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSetFleshBonePositionTargetBindingDataflowNode_v2);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSetFleshDefaultPropertiesNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSetMuscleActivationParameterNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSetVerticesKinematicDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSetVertexTetrahedraPositionTargetBindingDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSetVertexTrianglePositionTargetBindingDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSetVertexVertexPositionTargetBindingDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSkinSimulationPropertiesDataflowNodes);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FTriangleMeshSimulationPropertiesDataflowNodes);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FVisualizeFiberFieldNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FVisualizePositionTargetsNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FVisualizeKinematicFacesNode);

	// Deprecated nodes
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAppendTetrahedralCollectionDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FComputeMuscleActivationDataNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSetFleshBonePositionTargetBindingDataflowNode);

	// register node category for flesh asset
	UE_DATAFLOW_REGISTER_CATEGORY_FORASSET_TYPE("Flesh", UFleshAsset);
	UE_DATAFLOW_REGISTER_CATEGORY_FORASSET_TYPE("GeometryCollection", UFleshAsset);
}

void IChaosFleshNodesPlugin::ShutdownModule()
{
}


IMPLEMENT_MODULE(IChaosFleshNodesPlugin, ChaosFleshNodes)


#undef LOCTEXT_NAMESPACE
