// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshNodesUtility.h"

#include "Chaos/Deformable/Utilities.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"

namespace UE::Dataflow
{
	// Helper to get the boundary of a tet mesh, useful for debugging / verifying output
	TArray<FIntVector3> GetSurfaceTriangles(const TArray<FIntVector4>& Tets, const bool bKeepInterior)
	{
		// Rotate the vector so the first element is the smallest
		auto RotVec = [](const FIntVector3& F) -> FIntVector3
		{
			int32 MinIdx = F.X < F.Y ? (F.X < F.Z ? 0 : 2) : (F.Y < F.Z ? 1 : 2);
			return FIntVector3(F[MinIdx], F[(MinIdx + 1) % 3], F[(MinIdx + 2) % 3]);
		};
		// Reverse the winding while keeping the first element unchanged
		auto RevVec = [](const FIntVector3& F) -> FIntVector3
		{
			return FIntVector3(F.X, F.Z, F.Y);
		};

		TSet<FIntVector3> FacesSet;
		for (int32 TetIdx = 0; TetIdx < Tets.Num(); ++TetIdx)
		{
			FIntVector3 TetF[4];
			Chaos::Utilities::GetTetFaces(Tets[TetIdx], TetF[0], TetF[1], TetF[2], TetF[3], false);
			for (int32 SubIdx = 0; SubIdx < 4; ++SubIdx)
			{
				// A face can be shared between a maximum of 2 tets, so no need worrying about 
				// re-adding removed faces.
				FIntVector3 Key = RotVec(TetF[SubIdx]);
				if (FacesSet.Contains(Key))
				{
					if (!bKeepInterior)
					{
						FacesSet.Remove(Key);
					}
				}
				else
				{
					FacesSet.Add(RevVec(Key));
				}
			}
		}
		return FacesSet.Array();
	}



	TArray<int32> GetMatchingMeshIndices(const TArray<FString>& MeshNames, const FManagedArrayCollection* InCollection)
	{
		TArray<int32> GeometryIndices;

		// TransformGroup
		int32 NumTransforms = InCollection->NumElements(FTransformCollection::TransformGroup);
		const TManagedArray<FString>* TransformName = InCollection->FindAttribute<FString>("BoneName", FTransformCollection::TransformGroup);
		const TManagedArray<int32>* TransformToGeometryIndex = InCollection->FindAttribute<int32>("TransformToGeometryIndex", FTransformCollection::TransformGroup);

		// Geometry Group
		int32 NumGeometry = InCollection->NumElements(FGeometryCollection::GeometryGroup);

		if (MeshNames.Num())
		{
			for (int32 Tdx = 0; Tdx < NumTransforms; Tdx++)
			{
				if (MeshNames.Contains((*TransformName)[Tdx]))
				{
					if (0 <= (*TransformToGeometryIndex)[Tdx] && (*TransformToGeometryIndex)[Tdx] < NumGeometry)
					{
						GeometryIndices.Add((*TransformToGeometryIndex)[Tdx]);
					}
				}
			}
		}
		else
		{
			GeometryCollectionAlgo::ContiguousArray(GeometryIndices, NumGeometry);
		}

		return GeometryIndices;
	}
}