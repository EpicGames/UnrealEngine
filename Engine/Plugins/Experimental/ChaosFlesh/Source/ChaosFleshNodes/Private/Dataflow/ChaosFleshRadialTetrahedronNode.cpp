// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshRadialTetrahedronNode.h"
#include "Meshing/ChaosFleshRadialMeshing.h"

#include "Chaos/Deformable/Utilities.h"
#include "ChaosFlesh/ChaosFlesh.h"
#include "Chaos/Utilities.h"
#include "Chaos/UniformGrid.h"
#include "ChaosFlesh/FleshCollection.h"
#include "ChaosFlesh/FleshCollectionUtility.h"
#include "ChaosLog.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Dataflow/ChaosFleshNodesUtility.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Engine/StaticMesh.h"
#include "FTetWildWrapper.h"
#include "Generate/IsosurfaceStuffing.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "Spatial/FastWinding.h"
#include "Spatial/MeshAABBTree3.h"

//=============================================================================
// FRadialTetrahedronDataflowNodes
//=============================================================================


void FRadialTetrahedronDataflowNodes::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<DataType>(&Collection))
	{
		TUniquePtr<FFleshCollection> InCollection(GetValue<DataType>(Context, &Collection).NewCopy<FFleshCollection>());
		TArray<FIntVector4> TetElements;
		TArray<FVector> TetVertices;
		RadialTetMesh(InnerRadius, OuterRadius, Height, RadialSample, AngularSample, VerticalSample, BulgeRatio * (OuterRadius - InnerRadius) / (Chaos::FReal)2., TetElements, TetVertices);
		TArray<FIntVector3> SurfaceElements = UE::Dataflow::GetSurfaceTriangles(TetElements, !bDiscardInteriorTriangles);
		TUniquePtr<FTetrahedralCollection> RadialCollection(FTetrahedralCollection::NewTetrahedralCollection(TetVertices, SurfaceElements, TetElements));
		InCollection->AppendGeometry(*RadialCollection.Get());

		SetValue<const DataType&>(Context, *InCollection, &Collection);
	}
}

