// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshSetFleshBonePositionTargetBindingNode.h"

#include "Engine/SkeletalMesh.h"
#include "Chaos/BoundingVolumeHierarchy.h"
#include "Chaos/Utilities.h"
#include "GeometryCollection/Facades/CollectionKinematicBindingFacade.h"
#include "GeometryCollection/Facades/CollectionPositionTargetFacade.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"
#include "GeometryCollection/Facades/CollectionTransformFacade.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "SkeletalMeshAttributes.h"
#include "ChaosFlesh/ChaosFlesh.h"
#include "Chaos/HierarchicalSpatialHash.h"
#include "Chaos/TriangleCollisionPoint.h"

#include "BoneWeights.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFleshSetFleshBonePositionTargetBindingNode)

DEFINE_LOG_CATEGORY_STATIC(ChaosFleshSetFleshBonePositionTargetBindingNodeLog, Log, All);

void FSetFleshBonePositionTargetBindingDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue(Context, &Collection);
#if WITH_EDITORONLY_DATA
		if (TObjectPtr<USkeletalMesh> BoneSkeletalMesh = GetValue<TObjectPtr<USkeletalMesh>>(Context, &SkeletalMeshIn))
		{
			const TManagedArray<FIntVector>* Indices = InCollection.FindAttribute<FIntVector>("Indices", FGeometryCollection::FacesGroup);
			const TManagedArray<FVector3f>* Vertices = InCollection.FindAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
			const TManagedArray<FTransform3f>* Transform = InCollection.FindAttribute<FTransform3f>("Transform", FTransformCollection::TransformGroup);
			const TManagedArray<FString>* TransformBoneName = InCollection.FindAttribute<FString>("BoneName", FTransformCollection::TransformGroup);
			GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(InCollection);
			const FReferenceSkeleton& ReferenceSkeleton = BoneSkeletalMesh->GetRefSkeleton();
			const int32 LODIndex = 0;
			if (Indices && Vertices && Transform && TransformBoneName)
			{
				FSkeletalMeshRenderData* RenderData = BoneSkeletalMesh->GetResourceForRendering();
				FMeshDescription MeshDescription;
				if (BoneSkeletalMesh->CloneMeshDescription(LODIndex, MeshDescription) && RenderData->LODRenderData.Num())
				{
					FSkeletalMeshLODRenderData* LODRenderData = &RenderData->LODRenderData[LODIndex];
					FSkeletalMeshAttributes MeshAttribs(MeshDescription);
					FSkinWeightsVertexAttributesRef VertexSkinWeights = MeshAttribs.GetVertexSkinWeights();
					int32 NumSkeletonVertices = MeshDescription.Vertices().Num();

					Chaos::FReal SphereRadius = (Chaos::FReal)0.;

					Chaos::FVec3f CoordMaxs(-FLT_MAX);
					Chaos::FVec3f CoordMins(FLT_MAX);
					for (int32 i = 0; i < NumSkeletonVertices; i++)
					{
						CoordMaxs = CoordMaxs.ComponentwiseMax(MeshDescription.GetVertexPosition(i));
						CoordMins = CoordMins.ComponentwiseMin(MeshDescription.GetVertexPosition(i));
					}
					Chaos::FVec3f CoordDiff = (CoordMaxs - CoordMins) * VertexRadiusRatio;
					SphereRadius = Chaos::FReal(FGenericPlatformMath::Max(CoordDiff[0], FGenericPlatformMath::Max(CoordDiff[1], CoordDiff[2])));

					TArray<Chaos::FSphere*> VertexSpherePtrs;
					TArray<Chaos::FSphere> VertexSpheres;

					VertexSpheres.Init(Chaos::FSphere(Chaos::TVec3<Chaos::FReal>(0), SphereRadius), NumSkeletonVertices);
					VertexSpherePtrs.SetNum(NumSkeletonVertices);

					for (int32 i = 0; i < int32(NumSkeletonVertices); i++)
					{
						Chaos::TVec3<Chaos::FReal> SphereCenter(MeshDescription.GetVertexPosition(i));
						Chaos::FSphere VertexSphere(SphereCenter, SphereRadius);
						VertexSpheres[i] = Chaos::FSphere(SphereCenter, SphereRadius);
						VertexSpherePtrs[i] = &VertexSpheres[i];
					}
					Chaos::TBoundingVolumeHierarchy<
						TArray<Chaos::FSphere*>,
						TArray<int32>,
						Chaos::FReal,
						3> VertexBVH(VertexSpherePtrs);

					GeometryCollection::Facades::FVertexBoneWeightsFacade VertexBoneWeightsFacade(InCollection);



					//Mapping from SKM bone index to Collection bone index
					const TMap<FString, int32> BoneNameIndexMap = TransformFacade.BoneNameIndexMap();
					TMap<int32, int32> SKMBoneIndexToCollectionBoneIndex;
					for (int32 BoneIndex = 0; BoneIndex < ReferenceSkeleton.GetNum(); ++BoneIndex)
					{
						FString SKMBoneName = ReferenceSkeleton.GetRefBoneInfo()[BoneIndex].Name.ToString();
						if (BoneNameIndexMap.Contains(SKMBoneName))
						{
							SKMBoneIndexToCollectionBoneIndex.Add(BoneIndex, BoneNameIndexMap[SKMBoneName]);
						}
					}

					if (SkeletalBindingMode == ESkeletalBindingMode::Dataflow_SkeletalBinding_Kinematic)
					{
						for (int32 i = 0; i < Vertices->Num(); ++i)
						{
							//only work on particles that are not kinematic:
							if (!VertexBoneWeightsFacade.IsKinematicVertex(i))
							{
								TArray<int32> ParticleIntersection = VertexBVH.FindAllIntersections((*Vertices)[i]);
								int32 MinIndex = -1;
								float MinDis = SphereRadius;
								for (int32 j = 0; j < ParticleIntersection.Num(); j++)
								{
									Chaos::FRealSingle CurrentDistance = ((*Vertices)[i] - MeshDescription.GetVertexPosition(ParticleIntersection[j])).Size();
									if (CurrentDistance < MinDis)
									{
										MinDis = CurrentDistance;
										MinIndex = ParticleIntersection[j];
									}
								}

								if (MinIndex != -1)
								{
									FVertexBoneWeights BoneWeights = VertexSkinWeights.Get(FVertexID(MinIndex));
									const int32 InfluenceCount = BoneWeights.Num();
									TArray<int32> VertexBoneIndex;
									TArray<float> VertexBoneWeight;
									for (int32 InfluenceIndex = 0; InfluenceIndex < InfluenceCount; ++InfluenceIndex)
									{
										if (SKMBoneIndexToCollectionBoneIndex.Contains(BoneWeights[InfluenceIndex].GetBoneIndex()))
										{
											VertexBoneIndex.Add(SKMBoneIndexToCollectionBoneIndex[BoneWeights[InfluenceIndex].GetBoneIndex()]);
											VertexBoneWeight.Add(BoneWeights[InfluenceIndex].GetWeight());
										}
									}
									VertexBoneWeightsFacade.ModifyBoneWeight(i, VertexBoneIndex, VertexBoneWeight);
									VertexBoneWeightsFacade.SetVertexKinematic(i);
								}
							}
						}
					}
					else
					{
						GeometryCollection::Facades::FPositionTargetFacade PositionTargets(InCollection);
						PositionTargets.DefineSchema();

						for (int32 i = 0; i < Vertices->Num(); ++i)
						{
							//only work on particles that are not kinematic:
							if (!VertexBoneWeightsFacade.IsKinematicVertex(i))
							{
								TArray<int32> ParticleIntersection = VertexBVH.FindAllIntersections((*Vertices)[i]);
								int32 MinIndex = -1;
								float MinDis = SphereRadius;
								for (int32 j = 0; j < ParticleIntersection.Num(); j++)
								{
									Chaos::FRealSingle CurrentDistance = ((*Vertices)[i] - MeshDescription.GetVertexPosition(ParticleIntersection[j])).Size();
									if (CurrentDistance < MinDis)
									{
										MinDis = CurrentDistance;
										MinIndex = ParticleIntersection[j];
									}
								}

								if (MinIndex != -1)
								{
									// add kinematic particles
									int32 ParticleIndex = InCollection.AddElements(1, FGeometryCollection::VerticesGroup);
									TManagedArray<FVector3f>& CurrentVertices = InCollection.ModifyAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);

									CurrentVertices[ParticleIndex] = MeshDescription.GetVertexPosition(MinIndex);

									FVertexBoneWeights BoneWeights = VertexSkinWeights.Get(FVertexID(MinIndex));
									const int32 InfluenceCount = BoneWeights.Num();
									TArray<int32> VertexBoneIndex;
									TArray<float> VertexBoneWeight;
									for (int32 InfluenceIndex = 0; InfluenceIndex < InfluenceCount; ++InfluenceIndex)
									{
										if (SKMBoneIndexToCollectionBoneIndex.Contains(BoneWeights[InfluenceIndex].GetBoneIndex()))
										{
											VertexBoneIndex.Add(SKMBoneIndexToCollectionBoneIndex[BoneWeights[InfluenceIndex].GetBoneIndex()]);
											VertexBoneWeight.Add(BoneWeights[InfluenceIndex].GetWeight());
										}
									}
									VertexBoneWeightsFacade.ModifyBoneWeight(ParticleIndex, VertexBoneIndex, VertexBoneWeight);
									VertexBoneWeightsFacade.SetVertexKinematic(ParticleIndex);

									GeometryCollection::Facades::FPositionTargetsData DataPackage;
									DataPackage.TargetIndex.Init(ParticleIndex, 1);
									DataPackage.TargetWeights.Init(1.f, 1);
									DataPackage.SourceWeights.Init(1.f, 1);
									DataPackage.SourceIndex.Init(i, 1);
									if (const TManagedArray<float>* Mass = InCollection.FindAttribute<float>("Mass", FGeometryCollection::VerticesGroup))
									{
										//Target is kinematic, only compute stiffness from source
										DataPackage.Stiffness = PositionTargetStiffness * (*Mass)[i];
									}
									else
									{
										DataPackage.Stiffness = PositionTargetStiffness;
									}
									PositionTargets.AddPositionTarget(DataPackage);
								}
							}
						}
					}
				}
			}
		}
#else
		ensureMsgf(false, TEXT("SetFleshBonePositionTargetBinding is an editor only node."));
#endif
		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}

void FSetFleshBonePositionTargetBindingDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue(Context, &Collection);
#if WITH_EDITORONLY_DATA
		if (TObjectPtr<USkeletalMesh> BoneSkeletalMesh = GetValue<TObjectPtr<USkeletalMesh>>(Context, &SkeletalMeshIn))
		{
			const TManagedArray<FVector3f>* Vertices = InCollection.FindAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
			const TManagedArray<FTransform3f>* Transform = InCollection.FindAttribute<FTransform3f>("Transform", FTransformCollection::TransformGroup);
			const TManagedArray<FString>* TransformBoneName = InCollection.FindAttribute<FString>("BoneName", FTransformCollection::TransformGroup);
			GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(InCollection);
			const FReferenceSkeleton& ReferenceSkeleton = BoneSkeletalMesh->GetRefSkeleton();
			static int32 LODIndex = 0;
			if (Vertices && Transform && TransformBoneName)
			{
				FMeshDescription MeshDescription;
				if (BoneSkeletalMesh->CloneMeshDescription(LODIndex, MeshDescription))
				{
					FSkeletalMeshAttributes MeshAttribs(MeshDescription);
					const FSkinWeightsVertexAttributesRef VertexSkinWeights = MeshAttribs.GetVertexSkinWeights();
					const int32 NumSkeletonVertices = MeshDescription.Vertices().Num();
					const int32 NumSkeletonTriangles = MeshDescription.Triangles().Num();

					TArray<Chaos::TVec3<Chaos::FReal>> SkeletonVerticesTVec3;
					SkeletonVerticesTVec3.SetNum(NumSkeletonVertices);
					// Get Vertex Position
					for (const FVertexID& VertexID : MeshDescription.Vertices().GetElementIDs())
					{
						const int32 VertexIndex = VertexID.GetValue();			
						SkeletonVerticesTVec3[VertexIndex] = Chaos::TVec3<Chaos::FReal>(MeshDescription.GetVertexPosition(VertexID));
					}
					TConstArrayView<Chaos::TVec3<Chaos::FReal>> ConstSkeletonVerticesTVec3(SkeletonVerticesTVec3);
					// Get Triangles
					Chaos::FTriangleMesh TriangleMesh;
					TArray<Chaos::TVec3<int32>> TriangleElements;
					TriangleElements.SetNum(NumSkeletonTriangles);
					for (const FTriangleID& TriangleID : MeshDescription.Triangles().GetElementIDs())
					{
						const int32 TriangleIndex = TriangleID.GetValue();
						const TArrayView<const FVertexID>& Triangle = MeshDescription.GetTriangleVertices(TriangleID);
						TriangleElements[TriangleIndex] = Chaos::TVec3<int32>(Triangle[0], Triangle[1], Triangle[2]);
					}
					TriangleMesh.Init(TriangleElements);
					Chaos::FTriangleMesh::TSpatialHashType<Chaos::FReal> SpatialHash;
					Chaos::FReal SphereRadius = (Chaos::FReal)SearchRadius;
					TriangleMesh.BuildSpatialHash(ConstSkeletonVerticesTVec3, SpatialHash, SphereRadius);

					GeometryCollection::Facades::FVertexBoneWeightsFacade VertexBoneWeightsFacade(InCollection);
					GeometryCollection::Facades::FPositionTargetFacade PositionTargets(InCollection);

					//Mapping from SKM bone index to Collection bone index
					const TMap<FString, int32> BoneNameIndexMap = TransformFacade.BoneNameIndexMap();
					TMap<int32, int32> SKMBoneIndexToCollectionBoneIndex;
					for (int32 BoneIndex = 0; BoneIndex < ReferenceSkeleton.GetNum(); ++BoneIndex)
					{
						FString SKMBoneName = ReferenceSkeleton.GetRefBoneInfo()[BoneIndex].Name.ToString();
						if (BoneNameIndexMap.Contains(SKMBoneName))
						{
							SKMBoneIndexToCollectionBoneIndex.Add(BoneIndex, BoneNameIndexMap[SKMBoneName]);
						}
					}

					// Only keep boundary vertices within VertexSelection
					TArray<int32> SelectedVertices;
					if (IsConnected(&VertexSelection))
					{
						FDataflowVertexSelection InDataflowVertexSelection = GetValue<FDataflowVertexSelection>(Context, &VertexSelection);
						if (InDataflowVertexSelection.Num() != Vertices->Num())
						{
							Context.Error(FString::Printf(
								TEXT("VertexSelection size [%d] is not equal to the collection's vertex count [%d]"),
								InDataflowVertexSelection.Num(), Vertices->Num()),
								this, Out);
							return;
						}
						SelectedVertices = InDataflowVertexSelection.AsArray();
					}
					else
					{
						SelectedVertices.SetNum(Vertices->Num());
						for (int32 VertIdx = 0; VertIdx < Vertices->Num(); ++VertIdx)
						{
							SelectedVertices[VertIdx] = VertIdx;
						}
					}

					//Transfer skin weights
					TSet<int32> MissingSourceBones;
					for (int32 VertIdx : SelectedVertices)
					{
						//only work on particles that are not kinematic:
						if (!VertexBoneWeightsFacade.IsKinematicVertex(VertIdx))
						{
							TArray<Chaos::TTriangleCollisionPoint<Chaos::FReal>> Result;
							if (TriangleMesh.PointClosestTriangleQuery(SpatialHash, ConstSkeletonVerticesTVec3,
								VertIdx, Chaos::TVec3<Chaos::FReal>((*Vertices)[VertIdx]), SphereRadius / 2.f, SphereRadius / 2.f,
								[](const int32 PointIndex, const int32 TriangleIndex)->bool { return true; }, Result))
							{
								if (Result.Num() > 0)
								{
									const Chaos::TTriangleCollisionPoint<Chaos::FReal>& CollisionPoint = Result[0];
									TMap<int32, float> BoneWeightBucket;
									FVector3f InterpSkeletonVertexPosition = FVector3f(0);
									for (int32 LocalTriIdx = 0; LocalTriIdx < 3; ++LocalTriIdx)
									{
										const int32 TriVertexIndex = TriangleElements[CollisionPoint.Indices[1]][LocalTriIdx];
										const float TriInterpWeight = CollisionPoint.Bary[LocalTriIdx + 1];
										InterpSkeletonVertexPosition += TriInterpWeight * MeshDescription.GetVertexPosition(TriVertexIndex);
										FVertexBoneWeights BoneWeights = VertexSkinWeights.Get(FVertexID(TriVertexIndex));
										const int32 InfluenceCount = BoneWeights.Num();
										for (int32 InfluenceIndex = 0; InfluenceIndex < InfluenceCount; ++InfluenceIndex)
										{
											const int32 SKMBoneIndex = BoneWeights[InfluenceIndex].GetBoneIndex();
											if (SKMBoneIndexToCollectionBoneIndex.Contains(SKMBoneIndex))
											{
												const int32 CollectionBoneIndex = SKMBoneIndexToCollectionBoneIndex[SKMBoneIndex];
												const float InterpBoneWeight = TriInterpWeight * BoneWeights[InfluenceIndex].GetWeight();
												if (BoneWeightBucket.Contains(CollectionBoneIndex))
												{
													BoneWeightBucket[CollectionBoneIndex] += InterpBoneWeight;
												}
												else
												{
													BoneWeightBucket.Add(CollectionBoneIndex, InterpBoneWeight);
												}
											}
											else if (!MissingSourceBones.Contains(SKMBoneIndex))
											{
												MissingSourceBones.Add(SKMBoneIndex);
												UE_LOGF(ChaosFleshSetFleshBonePositionTargetBindingNodeLog, Error, "Collection does not contain bone[%ls].", *BoneSkeletalMesh->GetRefSkeleton().GetBoneName(SKMBoneIndex).ToString());
											}
										}
									}
									TArray<int32> VertexBoneIndex;
									TArray<float> VertexBoneWeight;
									for (const TPair<int32, float>& Pair : BoneWeightBucket)
									{
										VertexBoneIndex.Add(Pair.Key);
										VertexBoneWeight.Add(Pair.Value);
									}
									if (SkeletalBindingMode == ESkeletalBindingMode::Dataflow_SkeletalBinding_Kinematic)
									{
										VertexBoneWeightsFacade.ModifyBoneWeight(VertIdx, VertexBoneIndex, VertexBoneWeight);
										VertexBoneWeightsFacade.SetVertexKinematic(VertIdx);
									}
									else // Dataflow_SkeletalBinding_PositionTarget
									{
										const int32 ParticleIndex = InCollection.AddElements(1, FGeometryCollection::VerticesGroup);
										TManagedArray<FVector3f>& CurrentVertices = InCollection.ModifyAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
										CurrentVertices[ParticleIndex] = InterpSkeletonVertexPosition;

										VertexBoneWeightsFacade.ModifyBoneWeight(ParticleIndex, VertexBoneIndex, VertexBoneWeight);
										VertexBoneWeightsFacade.SetVertexKinematic(ParticleIndex);

										GeometryCollection::Facades::FPositionTargetsData DataPackage;
										DataPackage.TargetIndex.Init(ParticleIndex, 1);
										DataPackage.TargetWeights.Init(1.f, 1);
										DataPackage.SourceWeights.Init(1.f, 1);
										DataPackage.SourceIndex.Init(VertIdx, 1);
										if (const TManagedArray<float>* Mass = InCollection.FindAttribute<float>("Mass", FGeometryCollection::VerticesGroup))
										{
											//Target is kinematic, only compute stiffness from source
											DataPackage.Stiffness = PositionTargetStiffness * (*Mass)[VertIdx];
										}
										else
										{
											DataPackage.Stiffness = PositionTargetStiffness;
										}
										PositionTargets.AddPositionTarget(DataPackage);
									}
								}
							}
						}
					}
				}
			}
		}
#else
		ensureMsgf(false, TEXT("SetFleshBonePositionTargetBinding is an editor only node."));
#endif
		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}