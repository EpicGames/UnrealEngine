// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshSetFleshDefaultPropertiesNode.h"

#include "Async/ParallelFor.h"
#include "Chaos/Deformable/Utilities.h"
#include "ChaosFlesh/ChaosFlesh.h"
#include "Chaos/Tetrahedron.h"
#include "Chaos/Utilities.h"
#include "Chaos/UniformGrid.h"
#include "ChaosFlesh/FleshCollection.h"
#include "ChaosFlesh/FleshCollectionUtility.h"
#include "ChaosLog.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "MeshDescription.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "Spatial/FastWinding.h"
#include "Spatial/MeshAABBTree3.h"

template <class T> using MType = FManagedArrayCollection::TManagedType<T>;

void FSetFleshDefaultPropertiesNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		
		TManagedArray<float>& ParticleStiffness = InCollection.AddAttribute<float>("Stiffness", FGeometryCollection::VerticesGroup);
		TManagedArray<float>& ParticleDamping = InCollection.AddAttribute<float>("Damping", FGeometryCollection::VerticesGroup);
		TManagedArray<float>& ParticleIncompressibility = InCollection.AddAttribute<float>("Incompressibility", FGeometryCollection::VerticesGroup);
		TManagedArray<float>& ParticleInflation = InCollection.AddAttribute<float>("Inflation", FGeometryCollection::VerticesGroup);

		

		if (InCollection.HasAttributes({
			MType< float >("Mass", FGeometryCollection::VerticesGroup),
			MType< float >("Stiffness", FGeometryCollection::VerticesGroup),
			MType< float >("Damping", FGeometryCollection::VerticesGroup),
			MType< float >("Incompressibility", FGeometryCollection::VerticesGroup),
			MType< float >("Inflation", FGeometryCollection::VerticesGroup),
			MType< FIntVector4 >(FTetrahedralCollection::TetrahedronAttribute, FTetrahedralCollection::TetrahedralGroup),
			MType< FVector3f >("Vertex", "Vertices"),
			MType< TArray<int32> >(FTetrahedralCollection::IncidentElementsAttribute, FGeometryCollection::VerticesGroup),
			MType< TArray<int32> >(FTetrahedralCollection::IncidentElementsLocalIndexAttribute, FGeometryCollection::VerticesGroup) }))
		{

			int32 VertsNum = InCollection.NumElements(FGeometryCollection::VerticesGroup);
			int32 TetsNum = InCollection.NumElements(FTetrahedralCollection::TetrahedralGroup);
			if (VertsNum)
			{
				TManagedArray<float>& Mass = InCollection.ModifyAttribute<float>("Mass", FGeometryCollection::VerticesGroup);
				TManagedArray<float>& Stiffness = InCollection.ModifyAttribute<float>("Stiffness", FGeometryCollection::VerticesGroup);
				TManagedArray<float>& Damping = InCollection.ModifyAttribute<float>("Damping", FGeometryCollection::VerticesGroup);
				TManagedArray<float>& Incompressibility = InCollection.ModifyAttribute<float>("Incompressibility", FGeometryCollection::VerticesGroup);
				TManagedArray<float>& Inflation = InCollection.ModifyAttribute<float>("Inflation", FGeometryCollection::VerticesGroup);
				const TManagedArray<FIntVector4>& Tetrahedron = InCollection.GetAttribute<FIntVector4>(FTetrahedralCollection::TetrahedronAttribute, FTetrahedralCollection::TetrahedralGroup);
				const TManagedArray<FVector3f>& Vertex = InCollection.GetAttribute<FVector3f>("Vertex", "Vertices");
				const TManagedArray<TArray<int32>>& IncidentElements = InCollection.GetAttribute<TArray<int32>>(FTetrahedralCollection::IncidentElementsAttribute, FGeometryCollection::VerticesGroup);
				const TManagedArray<TArray<int32>>& IncidentElementsLocalIndex = InCollection.GetAttribute<TArray<int32>>(FTetrahedralCollection::IncidentElementsLocalIndexAttribute, FGeometryCollection::VerticesGroup);
				
				Mass.Fill(0.f);
				float MinV = TNumericLimits<float>::Max();
				float MaxV = -TNumericLimits<float>::Max();
				int32 NegativeElementVolumeCount = 0;
				int32 SmallElementVolumeCount = 0;
				double AvgV = 0.0;
				TSet<int32> Visited;
				int32 NumSet = 0;
				if (TetsNum)
				{
					double TotalVolume = 0.0;
					TArray<float> ElementMass;
					TArray<float> ElementVolume;
					ElementMass.Init(0.f, TetsNum);
					ElementVolume.Init(0.f, TetsNum);
					for (int e = 0; e < TetsNum; e++)
					{
						FVector3f X0 = Vertex[Tetrahedron[e][0]];
						FVector3f X1 = Vertex[Tetrahedron[e][1]];
						FVector3f X2 = Vertex[Tetrahedron[e][2]];
						FVector3f X3 = Vertex[Tetrahedron[e][3]];
						ElementVolume[e] = ((X3 - X0).Dot(FVector3f::CrossProduct(X1 - X0, X2 - X0))) / 6.f;
						if (ElementVolume[e] < 0.f)
						{
							ElementVolume[e] = -ElementVolume[e];
							NegativeElementVolumeCount++;
						}
						if (FMath::Abs(ElementVolume[e]) < UE_SMALL_NUMBER)
						{
							SmallElementVolumeCount++;
							if (SmallElementVolumeCount == 1)
							{
								UE_LOGF(LogChaosFlesh, Error,
									"'%ls' - Example: tetrahedron %d has volume %f < %e.",
									*GetName().ToString(), e, ElementVolume[e], UE_SMALL_NUMBER);
							}
						}
						TotalVolume += ElementVolume[e];
					}
					if (NegativeElementVolumeCount)
					{
						UE_LOGF(LogChaosFlesh, Warning,
							"'%ls' - Flipped negative volume for %d tetrahedra.",
							*GetName().ToString(), NegativeElementVolumeCount);
					}
					if (SmallElementVolumeCount)
					{
						UE_LOGF(LogChaosFlesh, Error,
							"'%ls' - %d tetrahedra have volume < %e.",
							*GetName().ToString(), SmallElementVolumeCount, UE_SMALL_NUMBER);
					}
					for (int32 e = 0; e < TetsNum; e++)
					{
						ElementMass[e] = Density * ElementVolume[e];
					}

					//
					// Set per-node mass by connected volume
					//

					for (int32 i = 0; i < IncidentElements.Num(); i++)
					{
						const TArray<int32>& IncidentElems = IncidentElements[i];
						for (int32 j = 0; j < IncidentElems.Num(); j++)
						{
							const int32 TetIndex = IncidentElems[j];
							if (Tetrahedron.GetConstArray().IsValidIndex(TetIndex))
							{
								for (int32 k = 0; k < 4; k++)
								{
									const int32 MassIndex = Tetrahedron[TetIndex][k];
									if (Mass.GetConstArray().IsValidIndex(MassIndex))
									{
										Mass[MassIndex] += ElementMass[TetIndex] / 4;
										Visited.Add(MassIndex);
									}
								}
							}
						}
					}

					if (Visited.Num())
					{
						NumSet = Visited.Num();
						for (TSet<int32>::TConstIterator It = Visited.CreateConstIterator(); It; ++It)
						{
							const int32 MassIndex = *It;
							AvgV += Mass[MassIndex];
							MinV = MinV < Mass[MassIndex] ? MinV : Mass[MassIndex];
							MaxV = MaxV > Mass[MassIndex] ? MaxV : Mass[MassIndex];
						}
						AvgV /= NumSet;
					}
					else
					{
						MinV = MaxV = 0.0;
					}
					if (!Visited.Num() && Vertex.Num())
					{
						Mass.Fill(Density * TotalVolume / Vertex.Num());
						NumSet = Mass.Num();
						Chaos::Utilities::GetMinAvgMax(Mass.GetConstArray(), MinV, AvgV, MaxV);
					}
				}
				

				if (const TManagedArray<int32>* TriangleMeshIndices = InCollection.FindAttribute<int32>("ObjectIndices", "TriangleMesh"))
				{
					//sets stiffness and damping for nontriangle mesh particles
					if (const TManagedArray<int32>* VertexStarts = InCollection.FindAttribute<int32>("VertexStart", FGeometryCollection::GeometryGroup))
					{
						if (const TManagedArray<int32>* VertexCounts = InCollection.FindAttribute<int32>("VertexCount", FGeometryCollection::GeometryGroup))
						{
							for (int32 i = 0; i < VertexStarts->Num(); i++)
							{
								if (!TriangleMeshIndices->Contains(i))
								{
									const int32 VertexStartIndex = (*VertexStarts)[i];
									const int32 VertexNum = (*VertexCounts)[i];
									for (int32 ParticleIndex = VertexStartIndex; ParticleIndex < VertexStartIndex + VertexNum; ParticleIndex++)
									{
										Stiffness[ParticleIndex] = VertexStiffness;
										Damping[ParticleIndex] = VertexDamping;
									}
								}
							}
						}
					}
				}
				else
				{
					Stiffness.Fill(VertexStiffness);
					Damping.Fill(VertexDamping);
				}
				
				Incompressibility.Fill(.5f * VertexIncompressibility);
				Inflation.Fill(VertexInflation * 2.f);

				UE_LOGF(LogChaosFlesh, Display,
					"'%ls' - Set mass on %d nodes:\n"
						"    method: %ls\n"
						"    min, avg, max: %f, %f, %f",
					*GetName().ToString(), NumSet, 
					(Visited.Num() > 0 ? TEXT("connected tet volume") : TEXT("uniform")),
					MinV, AvgV, MaxV);
			}
		}
		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}