// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshSetVertexTetrahedraPositionTargetBindingNode.h"

#include "Chaos/BoundingVolumeHierarchy.h"
#include "GeometryCollection/Facades/CollectionPositionTargetFacade.h"
#include "Chaos/Tetrahedron.h"
#include "ChaosFlesh/TetrahedralCollection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFleshSetVertexTetrahedraPositionTargetBindingNode)

//DEFINE_LOG_CATEGORY_STATIC(ChaosFleshSetVertexTetrahedraPositionTargetBindingNodeLog, Log, All);


void FSetVertexTetrahedraPositionTargetBindingDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<DataType>(&Collection))
	{
		DataType InCollection = GetValue<DataType>(Context, &Collection);
		if (FindInput(&TargetIndicesIn) && FindInput(&TargetIndicesIn)->GetConnection()) 
		{
			if (TManagedArray<FVector3f>* Vertices = InCollection.FindAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup))
			{
				if (TManagedArray<FIntVector4>* Tetrahedron = InCollection.FindAttribute<FIntVector4>(FTetrahedralCollection::TetrahedronAttribute, FTetrahedralCollection::TetrahedralGroup))
				{
					TManagedArray<int32>* TetrahedronStart = InCollection.FindAttribute<int32>(FTetrahedralCollection::TetrahedronStartAttribute, FGeometryCollection::GeometryGroup);
					TManagedArray<int32>* TetrahedronCount = InCollection.FindAttribute<int32>(FTetrahedralCollection::TetrahedronCountAttribute, FGeometryCollection::GeometryGroup);
					if (TetrahedronStart && TetrahedronCount)
					{	
						TArray<FString> GeometryGroupGuidsLocal;
						if (FindInput(&GeometryGroupGuidsIn) && FindInput(&GeometryGroupGuidsIn)->GetConnection())
						{
							GeometryGroupGuidsLocal = GetValue<TArray<FString>>(Context, &GeometryGroupGuidsIn);
						}
						TManagedArray<FString>* Guids = InCollection.FindAttribute<FString>("Guid", FGeometryCollection::GeometryGroup);
						for (int32 TetMeshIdx = 0; TetMeshIdx < TetrahedronStart->Num(); TetMeshIdx++)
						{
							if (GeometryGroupGuidsLocal.Num() && Guids)
							{
								if (!GeometryGroupGuidsLocal.Contains((*Guids)[TetMeshIdx]))
								{
									continue;
								}
							}
							const int32 TetMeshStart = (*TetrahedronStart)[TetMeshIdx];
							const int32 TetMeshCount = (*TetrahedronCount)[TetMeshIdx];

							// Build Tetrahedra
							TArray<Chaos::TTetrahedron<Chaos::FReal>> Tets;			// Index 0 == TetMeshStart
							TArray<Chaos::TTetrahedron<Chaos::FReal>*> BVHTetPtrs;
							Tets.SetNumUninitialized(TetMeshCount);
							BVHTetPtrs.SetNumUninitialized(TetMeshCount);
							for (int32 i = 0; i < TetMeshCount; i++)
							{
								const int32 Idx = TetMeshStart + i;
								const FIntVector4& Tet = (*Tetrahedron)[Idx];
								Tets[i] = Chaos::TTetrahedron<Chaos::FReal>(
									(*Vertices)[Tet[0]],
									(*Vertices)[Tet[1]],
									(*Vertices)[Tet[2]],
									(*Vertices)[Tet[3]]);
								BVHTetPtrs[i] = &Tets[i];
							}

							Chaos::TBoundingVolumeHierarchy<
								TArray<Chaos::TTetrahedron<Chaos::FReal>*>,
								TArray<int32>,
								Chaos::FReal,
								3> TetBVH(BVHTetPtrs);

							TArray<int32> TargetIndicesLocal = GetValue<TArray<int32>>(Context, &TargetIndicesIn);
							TArray<int32> SourceIndices;
							SourceIndices.Init(-1, TargetIndicesLocal.Num());

							GeometryCollection::Facades::FPositionTargetFacade PositionTargets(InCollection);
							PositionTargets.DefineSchema();

							for (int32 i = 0; i < TargetIndicesLocal.Num(); i++)
							{
								if (TargetIndicesLocal[i] > -1 && TargetIndicesLocal[i] < Vertices->Num())
								{
									FVector3f ParticlePos = (*Vertices)[TargetIndicesLocal[i]];
									TArray<int32> TetIntersections = TetBVH.FindAllIntersections(ParticlePos);
									for (int32 j = 0; j < TetIntersections.Num(); j++)
									{
										const int32 TetIdx = TetIntersections[j];
										if (!Tets[TetIdx].Outside(ParticlePos, 0))
										{
											Chaos::TVector<Chaos::FReal, 4> WeightsD = Tets[TetIdx].GetBarycentricCoordinates(ParticlePos);
											GeometryCollection::Facades::FPositionTargetsData DataPackage;
											DataPackage.TargetIndex.Init(TargetIndicesLocal[i], 1);
											DataPackage.TargetWeights.Init(1.f, 1);
											DataPackage.SourceWeights.Init(1.f, 4);
											DataPackage.SourceIndex.Init(-1, 4);
											DataPackage.SourceIndex[0] = (*Tetrahedron)[TetIdx][0];
											DataPackage.SourceIndex[1] = (*Tetrahedron)[TetIdx][1];
											DataPackage.SourceIndex[2] = (*Tetrahedron)[TetIdx][2];
											DataPackage.SourceIndex[3] = (*Tetrahedron)[TetIdx][3];
											DataPackage.SourceWeights[0] = WeightsD[0];
											DataPackage.SourceWeights[1] = WeightsD[1];
											DataPackage.SourceWeights[2] = WeightsD[2];
											DataPackage.SourceWeights[3] = WeightsD[3];
											if (TManagedArray<float>* Mass = InCollection.FindAttribute<float>("Mass", FGeometryCollection::VerticesGroup))
											{
												DataPackage.Stiffness = 0.f;
												for (int32 k = 0; k < 4; k++)
												{
													DataPackage.Stiffness += DataPackage.SourceWeights[k] * PositionTargetStiffness * (*Mass)[DataPackage.SourceIndex[0]];
												}
											}
											else
											{
												DataPackage.Stiffness = PositionTargetStiffness;
											}
											PositionTargets.AddPositionTarget(DataPackage);
											break;
										}
									}
								}
							}
						}
					}
				}
			}
		}
		SetValue(Context, MoveTemp(InCollection), &Collection);
	}


}
