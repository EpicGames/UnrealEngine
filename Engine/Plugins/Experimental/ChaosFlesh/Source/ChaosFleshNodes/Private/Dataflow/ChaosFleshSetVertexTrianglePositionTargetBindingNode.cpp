// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshSetVertexTrianglePositionTargetBindingNode.h"

#include "Chaos/HierarchicalSpatialHash.h"
#include "Chaos/TriangleCollisionPoint.h"
#include "Chaos/Utilities.h"
#include "ChaosFlesh/ChaosFlesh.h"
#include "ChaosFlesh/ChaosFleshCollectionFacade.h"
#include "ChaosFlesh/TetrahedralCollection.h"
#include "Dataflow/ChaosFleshNodesUtility.h"
#include "DynamicMesh/InfoTypes.h"
#include "DynamicMesh/MeshNormals.h"
#include "GeometryCollection/Facades/CollectionCollisionFacade.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "GeometryCollection/Facades/CollectionPositionTargetFacade.h"
#include "GeometryCollection/Facades/CollectionVolumeConstraintFacade.h"

#include "UDynamicMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFleshSetVertexTrianglePositionTargetBindingNode)

static TArray<Chaos::TVector<int32, 3>> RemoveInvalidIndices(const TManagedArray<FIntVector>* Indices)
{
	TArray<Chaos::TVector<int32, 3>> IndicesArray;
	for (int32 FaceIdx = 0; FaceIdx < Indices->Num(); ++FaceIdx)
	{
		Chaos::TVector<int32, 3> CurrentIndices(0);
		for (int32 LocalIdx = 0; LocalIdx < 3; ++LocalIdx)
		{
			CurrentIndices[LocalIdx] = (*Indices)[FaceIdx][LocalIdx];
		}
		if (CurrentIndices[0] != INDEX_NONE
			&& CurrentIndices[1] != INDEX_NONE
			&& CurrentIndices[2] != INDEX_NONE)
		{
			IndicesArray.Add(CurrentIndices);
		}
	}
	return IndicesArray;
}

static TArray<int32> ComputeBoundaryVertices(const TArray<Chaos::TVector<int32, 3>>& IndicesArray)
{
	TArray<TArray<int32>> LocalIndex;
	TArray<TArray<int32>>* LocalIndexPtr = &LocalIndex;
	TArray<TArray<int>> IncidentElements = Chaos::Utilities::ComputeIncidentElements(IndicesArray, LocalIndexPtr);
	TArray<int32> BoundaryVertices;
	for (int32 VertIdx = 0; VertIdx < IncidentElements.Num(); ++VertIdx)
	{
		if (IncidentElements[VertIdx].Num() > 0)
		{
			BoundaryVertices.Add(VertIdx);
		}
	}
	return BoundaryVertices;
}

static void FilterBoundaryVertices(TArray<int32>& OutBoundaryVertices, const FDataflowVertexSelection& InDataflowVertexSelection)
{
	TArray<int32> BoundaryVerticesNew;
	for (int32 VertIdx : OutBoundaryVertices)
	{
		if (InDataflowVertexSelection.IsSelected(VertIdx))
		{
			BoundaryVerticesNew.Add(VertIdx);
		}
	}
	OutBoundaryVertices = BoundaryVerticesNew;
}

void FSetVertexTrianglePositionTargetBindingDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue(Context, &Collection);

		TUniquePtr<FFleshCollection> InFleshCollection(GetValue(Context, &Collection).NewCopy<FFleshCollection>());

		Chaos::FFleshCollectionFacade TetCollection(*InFleshCollection);
		if (TManagedArray<FVector3f>* Vertices = InCollection.FindAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup))
		{
			if (TManagedArray<FIntVector>* Indices = InCollection.FindAttribute<FIntVector>("Indices", FGeometryCollection::FacesGroup))
			{
				if (TetCollection.IsTetrahedronValid())
				{
					TArray<FVector3f> Vertex = TetCollection.Vertex.Get().GetConstArray();
					TetCollection.ComponentSpaceVertices(Vertex);
					GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(InCollection);
					const TArray<int32> ComponentIndex = MeshFacade.GetGeometryGroupIndexArray();
					const TArray<Chaos::TVector<int32, 3>> IndicesArray = RemoveInvalidIndices(Indices);
					TArray<int32> BoundaryVertices = ComputeBoundaryVertices(IndicesArray);		
					// Only keep boundary vertices within VertexSelection
					if (IsConnected(&VertexSelection))
					{
						FDataflowVertexSelection InDataflowVertexSelection = GetValue<FDataflowVertexSelection>(Context, &VertexSelection);
						if (InDataflowVertexSelection.Num() != Vertices->Num())
						{
							Context.Error(FString::Printf(
								TEXT("VertexSelection size [%d] is not equal to the collection's vertex count [%d]"),
								InDataflowVertexSelection.Num(), Vertices->Num()),
								this, Out);
							return;
						}
						FilterBoundaryVertices(BoundaryVertices, InDataflowVertexSelection);
					}

					GeometryCollection::Facades::FPositionTargetFacade PositionTargets(InCollection);
					PositionTargets.DefineSchema();
#if WITH_EDITOR
					TArray<float> MassRatioArray;
#endif 
					TArray<Chaos::TVec3<Chaos::FReal>> VertexTVec3;
					VertexTVec3.SetNum(Vertex.Num());
					for (int32 VertexIdx = 0; VertexIdx < Vertex.Num(); VertexIdx++)
					{
						VertexTVec3[VertexIdx] = Chaos::TVec3<Chaos::FReal>(Vertex[VertexIdx]);
					}
					Chaos::FTriangleMesh TriangleMesh;
					TriangleMesh.Init(IndicesArray);
					Chaos::FTriangleMesh::TSpatialHashType<Chaos::FReal> SpatialHash;
					TConstArrayView<Chaos::TVec3<Chaos::FReal>> ConstArrayViewVertex(VertexTVec3);
					TriangleMesh.BuildSpatialHash(ConstArrayViewVertex, SpatialHash, Chaos::FReal(SearchRadius));
					for (int32 PointIndex: BoundaryVertices)
					{
						TArray<Chaos::TTriangleCollisionPoint<Chaos::FReal>> Result;
						if (TriangleMesh.PointClosestTriangleQuery(SpatialHash, ConstArrayViewVertex,
							PointIndex, VertexTVec3[PointIndex], Chaos::FReal(SearchRadius) / 2.f, Chaos::FReal(SearchRadius) / 2.f,
							[this, &ComponentIndex, &IndicesArray](const int32 PointIndex, const int32 TriangleIndex)->bool
							{
								return ComponentIndex[PointIndex] != ComponentIndex[IndicesArray[TriangleIndex][0]];
							},
							Result))
						{
							for (const Chaos::TTriangleCollisionPoint<Chaos::FReal>& CollisionPoint : Result)
							{
								GeometryCollection::Facades::FPositionTargetsData DataPackage;
								DataPackage.TargetIndex.Init(PointIndex, 1);
								DataPackage.TargetWeights.Init(1.f, 1);
								DataPackage.SourceWeights.Init(1.f, 3);
								DataPackage.SourceIndex.Init(-1, 3);
								DataPackage.SourceIndex[0] = IndicesArray[CollisionPoint.Indices[1]][0];
								DataPackage.SourceIndex[1] = IndicesArray[CollisionPoint.Indices[1]][1];
								DataPackage.SourceIndex[2] = IndicesArray[CollisionPoint.Indices[1]][2];
								DataPackage.SourceWeights[0] = CollisionPoint.Bary[1]; //convention: Bary[0] point, Bary[1:3] triangle
								DataPackage.SourceWeights[1] = CollisionPoint.Bary[2];
								DataPackage.SourceWeights[2] = CollisionPoint.Bary[3];
								if (TManagedArray<float>* Mass = InCollection.FindAttribute<float>("Mass", FGeometryCollection::VerticesGroup))
								{
									float MinMass = (*Mass)[DataPackage.TargetIndex[0]];
									float MaxMass = (*Mass)[DataPackage.TargetIndex[0]];
									DataPackage.Stiffness = 0.f;
									for (int32 k = 0; k < 3; k++)
									{
										MinMass = FGenericPlatformMath::Min(MinMass, (*Mass)[DataPackage.SourceIndex[k]]);
										MaxMass = FGenericPlatformMath::Max(MaxMass, (*Mass)[DataPackage.SourceIndex[k]]);
										DataPackage.Stiffness += DataPackage.SourceWeights[k] * PositionTargetStiffness * (*Mass)[DataPackage.SourceIndex[k]];
									}
#if WITH_EDITOR
									MassRatioArray.Add(MaxMass / MinMass);
#endif
									DataPackage.Stiffness += DataPackage.TargetWeights[0] * PositionTargetStiffness * (*Mass)[DataPackage.TargetIndex[0]];
								}
								else
								{
									DataPackage.Stiffness = PositionTargetStiffness;
								}
								DataPackage.bIsAnisotropic = bAllowSliding;
								DataPackage.bIsZeroRestLength = bUseZeroRestLengthSprings;
								PositionTargets.AddPositionTarget(DataPackage);
							}
						}
					}
#if WITH_EDITOR
					MassRatioArray.Sort();
					if (MassRatioArray.Num())
					{
						UE_LOGF(LogChaosFlesh, Display, "SetVertexTrianglePositionTargetBinding: Max mass ratio = %f, median mass ratio = %f"
							, MassRatioArray[MassRatioArray.Num()-1], MassRatioArray[MassRatioArray.Num()/2]);
					}
#endif
				}
			}
		}
		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}

void FDeleteVertexTrianglePositionTargetBindingDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue(Context, &Collection);
		if (IsConnected(&VertexSelection1) && IsConnected(&VertexSelection2))
		{
			const FDataflowVertexSelection& InVertexSelection1 = GetValue(Context, &VertexSelection1);
			const FDataflowVertexSelection& InVertexSelection2 = GetValue(Context, &VertexSelection2);
			const int32 VertexCount = InCollection.NumElements(FGeometryCollection::VerticesGroup);
			if (InVertexSelection1.Num() != VertexCount)
			{
				Context.Error(FString::Printf(
					TEXT("VertexSelection1 size (%d) is not equal to the collection's vertex count (%d)"),
					InVertexSelection1.Num(), VertexCount),
					this, Out);
				return;
			}
			if (InVertexSelection2.Num() != VertexCount)
			{
				Context.Error(FString::Printf(
					TEXT("VertexSelection2 size (%d) is not equal to the collection's vertex count (%d)"),
					InVertexSelection2.Num(), VertexCount),
					this, Out);
				return;
			}
			GeometryCollection::Facades::FPositionTargetFacade PositionTargets(InCollection);
			const int32 NumRemoved = PositionTargets.RemovePositionTargetBetween(
				[&InVertexSelection1](const int32 VertexIdx) { return InVertexSelection1.IsSelected(VertexIdx); },
				[&InVertexSelection2](const int32 VertexIdx) { return InVertexSelection2.IsSelected(VertexIdx); });
			Context.Info(FString::Printf(
				TEXT("DeleteVertexTrianglePositionTargetBinding: removed %d springs between two VertexSelections"), 
				NumRemoved), this, Out);
		}
		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}

void FSetCollidableVerticesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue(Context, &Collection);
		if (IsConnected(&VertexSelection))
		{
			GeometryCollection::Facades::FCollisionFacade CollisionFacade(InCollection);
			const FDataflowVertexSelection& InDataflowVertexSelection = GetValue(Context, &VertexSelection);
			CollisionFacade.SetCollisionEnabled(InDataflowVertexSelection.AsArray());
		}
		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}

void FCreateAirTetrahedralConstraintDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;
	if (Out->IsA(&Collection) || Out->IsA(&DynamicMesh))
	{
		TUniquePtr<FFleshCollection> InFleshCollection(GetValue(Context, &Collection).NewCopy<FFleshCollection>());
		Chaos::FFleshCollectionFacade TetCollection(*InFleshCollection);
		TObjectPtr<UDynamicMesh> OutDynamicMesh = NewObject<UDynamicMesh>();
		if (const TManagedArray<FVector3f>* Vertices = InFleshCollection->FindAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup))
		{
			if (const TManagedArray<FIntVector>* Indices = InFleshCollection->FindAttribute<FIntVector>("Indices", FGeometryCollection::FacesGroup))
			{
				TManagedArray<FIntVector4>& Tetrahedron = InFleshCollection->AddAttribute<FIntVector4>(FTetrahedralCollection::TetrahedronAttribute, FTetrahedralCollection::TetrahedralGroup);
				TArray<FVector3f> Vertex = Vertices->GetConstArray();
				TetCollection.ComponentSpaceVertices(Vertex);
				GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(*InFleshCollection);
				const TArray<int32> ComponentIndex = MeshFacade.GetGeometryGroupIndexArray();
				const TArray<Chaos::TVector<int32, 3>> IndicesArray = RemoveInvalidIndices(Indices);
				TArray<int32> BoundaryVertices = ComputeBoundaryVertices(IndicesArray);
				// Only keep boundary vertices within VertexSelection
				if (IsConnected(&VertexSelection))
				{
					FDataflowVertexSelection InDataflowVertexSelection = GetValue<FDataflowVertexSelection>(Context, &VertexSelection);
					if (InDataflowVertexSelection.Num() != Vertices->Num())
					{
						Context.Error(FString::Printf(
							TEXT("VertexSelection size [%d] is not equal to the collection's vertex count [%d]"),
							InDataflowVertexSelection.Num(), Vertices->Num()),
							this, Out);
						return;
					}
					FilterBoundaryVertices(BoundaryVertices, InDataflowVertexSelection);
				}

				TArray<Chaos::TVec3<Chaos::FReal>> VertexTVec3;
				VertexTVec3.SetNum(Vertex.Num());
				for (int32 VertexIdx = 0; VertexIdx < Vertex.Num(); VertexIdx++)
				{
					VertexTVec3[VertexIdx] = Chaos::TVec3<Chaos::FReal>(Vertex[VertexIdx]);
				}
				Chaos::FTriangleMesh TriangleMesh;
				TriangleMesh.Init(IndicesArray);
				Chaos::FTriangleMesh::TSpatialHashType<Chaos::FReal> SpatialHash;
				TConstArrayView<Chaos::TVec3<Chaos::FReal>> ConstArrayViewVertex(VertexTVec3);
				TriangleMesh.BuildSpatialHash(ConstArrayViewVertex, SpatialHash, Chaos::FReal(SearchRadius));
				TArray<FIntVector4> NewTetrahedra;
				auto ComputeDihedralAngle = [](const FVector3f& A, const FVector3f& B, const FVector3f& C, const FVector3f& D)
					{
						FVector3f Normal1 = FVector3f::CrossProduct(B - A, C - A).GetSafeNormal();
						FVector3f Normal2 = FVector3f::CrossProduct(B - A, D - A).GetSafeNormal();

						float CosTheta = FVector3f::DotProduct(Normal1, Normal2);
						return FMath::RadiansToDegrees(FMath::Acos(CosTheta));
					};
				auto IfTetInverted = [](const FVector3f& A, const FVector3f& B, const FVector3f& C, const FVector3f& D)
					{
						return (D - A).Dot(FVector3f::CrossProduct(B - A, C - A)) < 0;
					};
				int32 NumSkippedDihedral = 0;
				for (int32 PointIndex : BoundaryVertices)
				{
					TArray<Chaos::TTriangleCollisionPoint<Chaos::FReal>> Result;
					if (TriangleMesh.PointClosestTriangleQuery(SpatialHash, ConstArrayViewVertex,
						PointIndex, VertexTVec3[PointIndex], Chaos::FReal(SearchRadius) / 2.f, Chaos::FReal(SearchRadius) / 2.f,
						[this, &ComponentIndex, &IndicesArray](const int32 PointIndex, const int32 TriangleIndex)->bool
						{
							return ComponentIndex[PointIndex] != ComponentIndex[IndicesArray[TriangleIndex][0]];
						},
						Result))
					{
						for (const Chaos::TTriangleCollisionPoint<Chaos::FReal>& CollisionPoint : Result)
						{
							int32 Tri0 = IndicesArray[CollisionPoint.Indices[1]][0];
							int32 Tri1 = IndicesArray[CollisionPoint.Indices[1]][1];
							int32 Tri2 = IndicesArray[CollisionPoint.Indices[1]][2];
							float DihedralAngle = ComputeDihedralAngle(Vertex[PointIndex], Vertex[Tri0], Vertex[Tri1], Vertex[Tri2]);
							if (DihedralAngle < 10 || DihedralAngle > 170)
							{
								NumSkippedDihedral++;
								continue;
							}
							if (IfTetInverted(Vertex[PointIndex], Vertex[Tri0], Vertex[Tri1], Vertex[Tri2]))
							{
								Swap(Tri0, Tri1);
							}
							NewTetrahedra.Add(FIntVector4(PointIndex,
								Tri0,
								Tri1,
								Tri2));
							break;					
						}
					}
				}
				int32 StartSize = InFleshCollection->AddElements(NewTetrahedra.Num(), FTetrahedralCollection::TetrahedralGroup);
				for (int32 Idx = 0; Idx < NewTetrahedra.Num(); ++Idx)
				{
					Tetrahedron[StartSize + Idx] = NewTetrahedra[Idx];
				}
				InFleshCollection->InitIncidentElements(); //recompute incident elements after appending new tet constraints
				UE_LOGF(LogChaosFlesh, Display, "CreateAirTetrahedralConstraint: Added %d volumetric constraints."
					, NewTetrahedra.Num());
				UE_LOGF(LogChaosFlesh, Display, "CreateAirTetrahedralConstraint: Skipped %d volumetric constraints due to extreme dihedral angles."
					, NumSkippedDihedral);
				//Draw dynamic mesh of new tet boundary
				auto FloatVert = [](FVector3d V) { return FVector3f(V.X, V.Y, V.Z); };
				auto DoubleVert = [](FVector3f V) { return FVector3d(V.X, V.Y, V.Z); };
				OutDynamicMesh->Reset();
				FDynamicMesh3& OutDynamicMesh3 = OutDynamicMesh->GetMeshRef();
				OutDynamicMesh3.EnableAttributes();

				// Compute boundary triangle mesh
				const TArray<FIntVector3> SurfaceElements = UE::Dataflow::GetSurfaceTriangles(NewTetrahedra, /*bKeepInterior = */ false);
				for (int32 TriangleIndex = 0; TriangleIndex < SurfaceElements.Num(); ++TriangleIndex)
				{
					int v0 = OutDynamicMesh3.AppendVertex(DoubleVert(Vertex[SurfaceElements[TriangleIndex][0]]));
					int v1 = OutDynamicMesh3.AppendVertex(DoubleVert(Vertex[SurfaceElements[TriangleIndex][1]]));
					int v2 = OutDynamicMesh3.AppendVertex(DoubleVert(Vertex[SurfaceElements[TriangleIndex][2]]));
					OutDynamicMesh3.AppendTriangle(FIndex3i(v0, v1, v2));
				}
				
				// Compute normals
				OutDynamicMesh3.EnableVertexNormals(FVector3f(1, 0, 0));
				FMeshNormals MeshNormals(&OutDynamicMesh3);
				MeshNormals.ComputeVertexNormals();
				for (int32 VertexIndex = 0; VertexIndex < OutDynamicMesh3.VertexCount(); ++VertexIndex)
				{
					OutDynamicMesh3.SetVertexNormal(VertexIndex, FloatVert(MeshNormals[VertexIndex]));
				}
			} // if Indices
		} // if Vertices
		SetValue(Context, MoveTemp(static_cast<FManagedArrayCollection&>(*InFleshCollection.Release())), &Collection);
		SetValue(Context, OutDynamicMesh, &DynamicMesh);
	}
}

void FCreateAirVolumeConstraintDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;
	if (Out->IsA(&Collection) || Out->IsA(&DynamicMesh))
	{
		TUniquePtr<FFleshCollection> InFleshCollection(GetValue(Context, &Collection).NewCopy<FFleshCollection>());
		Chaos::FFleshCollectionFacade TetCollection(*InFleshCollection);
		TObjectPtr<UDynamicMesh> OutDynamicMesh = NewObject<UDynamicMesh>();
		if (const TManagedArray<FVector3f>* Vertices = InFleshCollection->FindAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup))
		{
			if (const TManagedArray<FIntVector>* Indices = InFleshCollection->FindAttribute<FIntVector>("Indices", FGeometryCollection::FacesGroup))
			{
				TArray<FVector3f> Vertex = Vertices->GetConstArray();
				TetCollection.ComponentSpaceVertices(Vertex);
				GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(*InFleshCollection);
				const TArray<int32> ComponentIndex = MeshFacade.GetGeometryGroupIndexArray();
				const TArray<Chaos::TVector<int32, 3>> IndicesArray = RemoveInvalidIndices(Indices);
				TArray<int32> BoundaryVertices = ComputeBoundaryVertices(IndicesArray);
				// Only keep boundary vertices within VertexSelection
				if (IsConnected(&VertexSelection))
				{
					FDataflowVertexSelection InDataflowVertexSelection = GetValue<FDataflowVertexSelection>(Context, &VertexSelection);
					if (InDataflowVertexSelection.Num() != Vertices->Num())
					{
						Context.Error(FString::Printf(
							TEXT("VertexSelection size [%d] is not equal to the collection's vertex count [%d]"),
							InDataflowVertexSelection.Num(), Vertices->Num()),
							this, Out);
						return;
					}
					FilterBoundaryVertices(BoundaryVertices, InDataflowVertexSelection);
				}

				TArray<Chaos::TVec3<Chaos::FReal>> VertexTVec3;
				VertexTVec3.SetNum(Vertex.Num());
				for (int32 VertexIdx = 0; VertexIdx < Vertex.Num(); VertexIdx++)
				{
					VertexTVec3[VertexIdx] = Chaos::TVec3<Chaos::FReal>(Vertex[VertexIdx]);
				}
				Chaos::FTriangleMesh TriangleMesh;
				TriangleMesh.Init(IndicesArray);
				Chaos::FTriangleMesh::TSpatialHashType<Chaos::FReal> SpatialHash;
				TConstArrayView<Chaos::TVec3<Chaos::FReal>> ConstArrayViewVertex(VertexTVec3);
				TriangleMesh.BuildSpatialHash(ConstArrayViewVertex, SpatialHash, Chaos::FReal(SearchRadius));
				TArray<FIntVector4> NewTetrahedra;
				auto ComputeDihedralAngle = [](const FVector3f& A, const FVector3f& B, const FVector3f& C, const FVector3f& D)
					{
						FVector3f Normal1 = FVector3f::CrossProduct(B - A, C - A).GetSafeNormal();
						FVector3f Normal2 = FVector3f::CrossProduct(B - A, D - A).GetSafeNormal();

						float CosTheta = FVector3f::DotProduct(Normal1, Normal2);
						return FMath::RadiansToDegrees(FMath::Acos(CosTheta));
					};
				auto IfTetInverted = [](const FVector3f& A, const FVector3f& B, const FVector3f& C, const FVector3f& D)
					{
						return (D - A).Dot(FVector3f::CrossProduct(B - A, C - A)) < 0;
					};
				int32 NumSkippedDihedral = 0;
				for (int32 PointIndex : BoundaryVertices)
				{
					TArray<Chaos::TTriangleCollisionPoint<Chaos::FReal>> Result;
					if (TriangleMesh.PointClosestTriangleQuery(SpatialHash, ConstArrayViewVertex,
						PointIndex, VertexTVec3[PointIndex], Chaos::FReal(SearchRadius) / 2.f, Chaos::FReal(SearchRadius) / 2.f,
						[this, &ComponentIndex, &IndicesArray](const int32 PointIndex, const int32 TriangleIndex)->bool
						{
							return ComponentIndex[PointIndex] != ComponentIndex[IndicesArray[TriangleIndex][0]];
						},
						Result))
					{
						for (const Chaos::TTriangleCollisionPoint<Chaos::FReal>& CollisionPoint : Result)
						{
							int32 Tri0 = IndicesArray[CollisionPoint.Indices[1]][0];
							int32 Tri1 = IndicesArray[CollisionPoint.Indices[1]][1];
							int32 Tri2 = IndicesArray[CollisionPoint.Indices[1]][2];
							float DihedralAngle = ComputeDihedralAngle(Vertex[PointIndex], Vertex[Tri0], Vertex[Tri1], Vertex[Tri2]);
							if (DihedralAngle < 10 || DihedralAngle > 170)
							{
								NumSkippedDihedral++;
								continue;
							}
							if (IfTetInverted(Vertex[PointIndex], Vertex[Tri0], Vertex[Tri1], Vertex[Tri2]))
							{
								Swap(Tri0, Tri1);
							}
							NewTetrahedra.Add(FIntVector4(PointIndex,
								Tri0,
								Tri1,
								Tri2));
							break;
						}
					}
				}
				// Add volume constraints
				GeometryCollection::Facades::FVolumeConstraintFacade VolumeConstraint(*InFleshCollection);
				for (const FIntVector4& NewTetrahedron : NewTetrahedra)
				{
					VolumeConstraint.AddVolumeConstraint(NewTetrahedron, Stiffness);
				}

				UE_LOGF(LogChaosFlesh, Display, "CreateAirVolumeConstraint: Added %d volumetric constraints."
					, NewTetrahedra.Num());
				UE_LOGF(LogChaosFlesh, Display, "CreateAirVolumeConstraint: Skipped %d volumetric constraints due to extreme dihedral angles."
					, NumSkippedDihedral);
				//Draw dynamic mesh of new tet boundary
				auto FloatVert = [](FVector3d V) { return FVector3f(V.X, V.Y, V.Z); };
				auto DoubleVert = [](FVector3f V) { return FVector3d(V.X, V.Y, V.Z); };
				OutDynamicMesh->Reset();
				FDynamicMesh3& OutDynamicMesh3 = OutDynamicMesh->GetMeshRef();
				OutDynamicMesh3.EnableAttributes();

				// Compute boundary triangle mesh
				const TArray<FIntVector3> SurfaceElements = UE::Dataflow::GetSurfaceTriangles(NewTetrahedra, /*bKeepInterior = */ false);
				for (int32 TriangleIndex = 0; TriangleIndex < SurfaceElements.Num(); ++TriangleIndex)
				{
					int v0 = OutDynamicMesh3.AppendVertex(DoubleVert(Vertex[SurfaceElements[TriangleIndex][0]]));
					int v1 = OutDynamicMesh3.AppendVertex(DoubleVert(Vertex[SurfaceElements[TriangleIndex][1]]));
					int v2 = OutDynamicMesh3.AppendVertex(DoubleVert(Vertex[SurfaceElements[TriangleIndex][2]]));
					OutDynamicMesh3.AppendTriangle(FIndex3i(v0, v1, v2));
				}

				// Compute normals
				OutDynamicMesh3.EnableVertexNormals(FVector3f(1, 0, 0));
				FMeshNormals MeshNormals(&OutDynamicMesh3);
				MeshNormals.ComputeVertexNormals();
				for (int32 VertexIndex = 0; VertexIndex < OutDynamicMesh3.VertexCount(); ++VertexIndex)
				{
					OutDynamicMesh3.SetVertexNormal(VertexIndex, FloatVert(MeshNormals[VertexIndex]));
				}
			} // if Indices
		} // if Vertices
		SetValue(Context, MoveTemp(static_cast<FManagedArrayCollection&>(*InFleshCollection.Release())), &Collection);
		SetValue(Context, OutDynamicMesh, &DynamicMesh);
	}
}