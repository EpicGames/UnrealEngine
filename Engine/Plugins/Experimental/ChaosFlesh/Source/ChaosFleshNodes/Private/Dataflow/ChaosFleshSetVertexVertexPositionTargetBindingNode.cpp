// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshSetVertexVertexPositionTargetBindingNode.h"

#include "Chaos/BoundingVolumeHierarchy.h"
#include "GeometryCollection/Facades/CollectionPositionTargetFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFleshSetVertexVertexPositionTargetBindingNode)

//DEFINE_LOG_CATEGORY_STATIC(ChaosFleshSetVertexVertexPositionTargetBindingNodeLog, Log, All);



void FSetVertexVertexPositionTargetBindingDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<DataType>(&Collection))
	{
		DataType InCollection = GetValue<DataType>(Context, &Collection);

		if (TManagedArray<FVector3f>* Vertices = InCollection.FindAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup))
		{
			if (FindInput(&TargetIndicesIn) && FindInput(&TargetIndicesIn)->GetConnection())
			{
				Chaos::FReal SphereRadius = (Chaos::FReal)0.;

				Chaos::TVec3<float> CoordMaxs(-FLT_MAX);
				Chaos::TVec3<float> CoordMins(FLT_MAX);
				for (int32 i = 0; i < Vertices->Num(); i++)
				{
					for (int32 j = 0; j < 3; j++) 
					{
						if ((*Vertices)[i][j] > CoordMaxs[j]) 
						{
							CoordMaxs[j] = (*Vertices)[i][j];
						}
						if ((*Vertices)[i][j] < CoordMins[j])
						{
							CoordMins[j] = (*Vertices)[i][j];
						}
					}
				}

				Chaos::TVec3<float> CoordDiff = (CoordMaxs - CoordMins) * RadiusRatio;

				SphereRadius = Chaos::FReal(FGenericPlatformMath::Min(CoordDiff[0], FGenericPlatformMath::Min(CoordDiff[1], CoordDiff[2])));

				TArray<Chaos::FSphere*> VertexSpherePtrs;
				TArray<Chaos::FSphere> VertexSpheres;

				VertexSpheres.Init(Chaos::FSphere(Chaos::TVec3<Chaos::FReal>(0), SphereRadius), Vertices->Num());
				VertexSpherePtrs.SetNum(Vertices->Num());

				for (int32 i = 0; i < Vertices->Num(); i++)
				{
					Chaos::TVec3<Chaos::FReal> SphereCenter((*Vertices)[i]);
					Chaos::FSphere VertexSphere(SphereCenter, SphereRadius);
					VertexSpheres[i] = Chaos::FSphere(SphereCenter, SphereRadius);
					VertexSpherePtrs[i] = &VertexSpheres[i];
				}
				Chaos::TBoundingVolumeHierarchy<
					TArray<Chaos::FSphere*>,
					TArray<int32>,
					Chaos::FReal,
					3> VertexBVH(VertexSpherePtrs);

			
				TArray<int32> TargetIndicesLocal = GetValue<TArray<int32>>(Context, &TargetIndicesIn);
				TSet<int32> TargetIndicesSet = TSet<int32>(TargetIndicesLocal);
				TArray<int32> SourceIndices;
				SourceIndices.Init(-1, TargetIndicesLocal.Num());
				
				for (int32 i = 0; i < TargetIndicesLocal.Num(); i++)
				{
					if (TargetIndicesLocal[i] > -1 && TargetIndicesLocal[i] < Vertices->Num()) 
					{
						FVector3f ParticlePos = (*Vertices)[TargetIndicesLocal[i]];
						TArray<int32> VertexIntersections = VertexBVH.FindAllIntersections(ParticlePos);
						float MinDistance = 10.f * (float)SphereRadius;
						int32 MinIndex = -1;
						for (int32 k = 0; k < VertexIntersections.Num(); k++)
						{
							if ((ParticlePos - (*Vertices)[VertexIntersections[k]]).Size() < MinDistance
								&& VertexIntersections[k] != TargetIndicesLocal[i] && !TargetIndicesSet.Contains(VertexIntersections[k]))
							{
								MinIndex = VertexIntersections[k];
								MinDistance = (ParticlePos - (*Vertices)[VertexIntersections[k]]).Size();
							}
						}
						ensure(MinIndex != -1);
						SourceIndices[i] = MinIndex;
					}
				}
				GeometryCollection::Facades::FPositionTargetFacade PositionTargets(InCollection);
				PositionTargets.DefineSchema();
				for (int32 i = 0; i < TargetIndicesLocal.Num(); i++)
				{
					if (SourceIndices[i] != -1)
					{
						GeometryCollection::Facades::FPositionTargetsData DataPackage;
						DataPackage.TargetIndex.Init(TargetIndicesLocal[i], 1);
						DataPackage.TargetWeights.Init(1.f, 1);
						DataPackage.SourceWeights.Init(1.f, 1);
						DataPackage.SourceIndex.Init(SourceIndices[i], 1);
						if (TManagedArray<float>* Mass = InCollection.FindAttribute<float>("Mass", FGeometryCollection::VerticesGroup))
						{
							if ((*Mass)[SourceIndices[i]] > 0.f)
							{
								DataPackage.Stiffness = PositionTargetStiffness * (*Mass)[SourceIndices[i]];
							}
							else
							{
								DataPackage.Stiffness = PositionTargetStiffness;
							}
						}
						else
						{
							DataPackage.Stiffness = PositionTargetStiffness;
						}

						PositionTargets.AddPositionTarget(DataPackage);
					}
				}
			}
			
		}
		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}

