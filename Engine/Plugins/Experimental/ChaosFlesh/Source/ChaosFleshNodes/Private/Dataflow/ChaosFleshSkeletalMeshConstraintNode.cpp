// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshSkeletalMeshConstraintNode.h"

#include "Engine/SkeletalMesh.h"
#include "GeometryCollection/Facades/CollectionKinematicBindingFacade.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "BoneWeights.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFleshSkeletalMeshConstraintNode)


void FKinematicSkeletalMeshInitializationDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<DataType>(&Collection) || Out->IsA<TArray<int32>>(&IndicesOut))
	{
		DataType InCollection = GetValue<DataType>(Context, &Collection);
		TArray<int32> Indices;
		if (TObjectPtr<USkeletalMesh> SkeletalMesh = GetValue<TObjectPtr<USkeletalMesh>>(Context, &SkeletalMeshIn))
		{
			FSkeletalMeshRenderData* RenderData = SkeletalMesh->GetResourceForRendering();
			if (RenderData->LODRenderData.Num())
			{
				//Grab vertices only, no elements
				FSkeletalMeshLODRenderData* LODRenderData = &RenderData->LODRenderData[0];
				const FPositionVertexBuffer& PositionVertexBuffer =
					LODRenderData->StaticVertexBuffers.PositionVertexBuffer;
				TManagedArray<FVector3f>& Vertices = InCollection.AddAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
				int32 index = InCollection.AddElements(PositionVertexBuffer.GetNumVertices(), FGeometryCollection::VerticesGroup);
				for (uint32 j = 0; j < PositionVertexBuffer.GetNumVertices(); j++)
				{
					const FVector3f& Pos = PositionVertexBuffer.VertexPosition(j);
					Vertices[index+j] = Pos;
					Indices.Add(index+j);
				}
				//Grab skin weights
				const FSkinWeightVertexBuffer* SkinWeightVertexBuffer = LODRenderData->GetSkinWeightVertexBuffer();
				const int32 MaxBoneInfluences = SkinWeightVertexBuffer->GetMaxBoneInfluences();
				TArray<FTransform> ComponentPose;
				UE::Dataflow::Animation::GlobalTransforms(SkeletalMesh->GetRefSkeleton(), ComponentPose);
				TArray<TArray<int32>> BoundVerts;
				TArray<TArray<float>> BoundWeights;
				BoundVerts.SetNum(ComponentPose.Num());
				BoundWeights.SetNum(ComponentPose.Num());
				for (uint32 j = 0; j < PositionVertexBuffer.GetNumVertices(); j++)
				{	
					
					int32 SectionIndex;
					int32 VertIndex;
					LODRenderData->GetSectionFromVertexIndex(j, SectionIndex, VertIndex);

					check(SectionIndex < LODRenderData->RenderSections.Num());
					const FSkelMeshRenderSection& Section = LODRenderData->RenderSections[SectionIndex];
					int32 BufferVertIndex = Section.GetVertexBufferIndex() + VertIndex;
					for (int32 InfluenceIndex = 0; InfluenceIndex < MaxBoneInfluences; InfluenceIndex++)
					{
						const int32 BoneIndex = Section.BoneMap[SkinWeightVertexBuffer->GetBoneIndex(BufferVertIndex, InfluenceIndex)];
						const float	Weight = (float)SkinWeightVertexBuffer->GetBoneWeight(BufferVertIndex, InfluenceIndex) * UE::AnimationCore::InvMaxRawBoneWeightFloat;
						if (Weight > float(0) && 0 <= BoneIndex && BoneIndex < ComponentPose.Num())
						{
							FString BoneName = SkeletalMesh->GetRefSkeleton().GetBoneName(BoneIndex).ToString();
							BoundVerts[BoneIndex].Add(index+j);
							BoundWeights[BoneIndex].Add(Weight);
						}
					}
				}
				for (int32 BoneIndex = 0; BoneIndex < ComponentPose.Num(); ++BoneIndex)
				{
					if (BoundVerts[BoneIndex].Num())
					{
						FString BoneName = SkeletalMesh->GetRefSkeleton().GetBoneName(BoneIndex).ToString();
						//get local coords of bound verts
						typedef GeometryCollection::Facades::FKinematicBindingFacade FKinematics;
						FKinematics Kinematics(InCollection); Kinematics.DefineSchema();
						if (Kinematics.IsValid())
						{
							FKinematics::FBindingKey Binding = Kinematics.SetBoneBindings(BoneIndex, BoundVerts[BoneIndex], BoundWeights[BoneIndex]);
							TManagedArray<TArray<FVector3f>>& LocalPos = InCollection.AddAttribute<TArray<FVector3f>>("LocalPosition", Binding.GroupName);
							Kinematics.AddKinematicBinding(Binding);

							auto FloatVert = [](FVector3d V) { return FVector3f(V.X, V.Y, V.Z); };
							auto DoubleVert = [](FVector3f V) { return FVector3d(V.X, V.Y, V.Z); };
							LocalPos[Binding.Index].SetNum(BoundVerts[BoneIndex].Num());
							for (int32 i = 0; i < BoundVerts[BoneIndex].Num(); i++)
							{
								FVector3f Temp = Vertices[BoundVerts[BoneIndex][i]];
								LocalPos[Binding.Index][i] = FloatVert(ComponentPose[BoneIndex].InverseTransformPosition(DoubleVert(Temp)));
							}
						}
					}
				}
				GeometryCollection::Facades::FVertexBoneWeightsFacade(InCollection).AddBoneWeightsFromKinematicBindings();
			}
		}
		SetValue(Context, MoveTemp(InCollection), &Collection);
		SetValue(Context, MoveTemp(Indices), &IndicesOut);
	}
}
