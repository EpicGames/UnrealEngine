// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshSkinSimulationPropertiesNode.h"

#include "Async/ParallelFor.h"
#include "Chaos/Deformable/Utilities.h"
#include "ChaosFlesh/ChaosFlesh.h"
#include "Chaos/Tetrahedron.h"
#include "Chaos/Utilities.h"
#include "Chaos/UniformGrid.h"
#include "ChaosFlesh/FleshCollection.h"
#include "ChaosFlesh/FleshCollectionUtility.h"
#include "ChaosLog.h"
#include "Dataflow/DataflowInputOutput.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "FTetWildWrapper.h"
#include "Generate/IsosurfaceStuffing.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/Facades/CollectionTetrahedralMetricsFacade.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "MeshDescription.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "Rendering/SkeletalMeshLODImporterData.h"
#include "Rendering/SkeletalMeshModel.h"
#include "SkeletalMeshLODModelToDynamicMesh.h" // MeshModelingBlueprints
#include "Spatial/FastWinding.h"
#include "Spatial/MeshAABBTree3.h"


void FSkinSimulationPropertiesDataflowNodes::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	TUniquePtr<FFleshCollection> InCollection(GetValue<DataType>(Context, &Collection).NewCopy<FFleshCollection>());
	if (Out->IsA<DataType>(&Collection))
	{
		const FName TriangleMeshGroup = "TriangleMesh";
		if (const TManagedArray<int32>* TriangleMeshIndices = InCollection->FindAttribute<int32>("ObjectIndices", "TriangleMesh"))
		{
			if (bSkinConstraints)
			{
				TManagedArray<bool>& IsSkinConstraint = InCollection->AddAttribute<bool>("SkinConstraints", TriangleMeshGroup);
				for (int32 i = 0; i < IsSkinConstraint.Num(); i++)
				{
					IsSkinConstraint[i] = true;
				}
			}
		}
		SetValue<const DataType&>(Context, *InCollection, &Collection);
	}
}