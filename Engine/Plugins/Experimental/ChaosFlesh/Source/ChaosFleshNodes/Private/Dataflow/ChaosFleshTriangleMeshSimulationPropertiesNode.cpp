// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshTriangleMeshSimulationPropertiesNode.h"
#include "Async/ParallelFor.h"
#include "Chaos/Deformable/Utilities.h"
#include "ChaosFlesh/ChaosFlesh.h"
#include "Chaos/Tetrahedron.h"
#include "Chaos/Utilities.h"
#include "Chaos/UniformGrid.h"
#include "ChaosFlesh/FleshCollection.h"
#include "ChaosFlesh/FleshCollectionUtility.h"
#include "ChaosLog.h"
#include "Dataflow/DataflowInputOutput.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "FTetWildWrapper.h"
#include "Generate/IsosurfaceStuffing.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/Facades/CollectionTetrahedralMetricsFacade.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "MeshDescription.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "Rendering/SkeletalMeshLODImporterData.h"
#include "Rendering/SkeletalMeshModel.h"
#include "SkeletalMeshLODModelToDynamicMesh.h" // MeshModelingBlueprints
#include "Spatial/FastWinding.h"
#include "Spatial/MeshAABBTree3.h"


template <class T> using MType = FManagedArrayCollection::TManagedType<T>;

void FTriangleMeshSimulationPropertiesDataflowNodes::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	TUniquePtr<FFleshCollection> InCollection(GetValue<DataType>(Context, &Collection).NewCopy<FFleshCollection>());
	if (Out->IsA<DataType>(&Collection))
	{

		TManagedArray<FString>* GroupNames = InCollection->FindAttribute<FString>("BoneName", FGeometryCollection::TransformGroup);
		TManagedArray<int32>* TransformIndex = InCollection->FindAttribute<int32>("TransformIndex", FGeometryCollection::GeometryGroup);
		TManagedArray<float>& ParticleStiffness = InCollection->AddAttribute<float>("Stiffness", FGeometryCollection::VerticesGroup);
		TManagedArray<float>& ParticleDamping = InCollection->AddAttribute<float>("Damping", FGeometryCollection::VerticesGroup);
		if (GroupNames && TransformIndex)
		{
			TSet<FString> TriangleMeshNameSet = TSet(MeshNames);
			TArray<int32> TriangleMeshIndices;

			for (int32 i = 0; i < TransformIndex->Num(); i++)
			{
				if (TriangleMeshNameSet.Contains((*GroupNames)[(*TransformIndex)[i]]))
				{
					TriangleMeshIndices.Add(i);
				}
			}

			if (TriangleMeshIndices.Num())
			{
				const FName TriangleMeshGroup = "TriangleMesh";
				int32 TriMeshIndex = InCollection->AddElements(TriangleMeshIndices.Num(), TriangleMeshGroup);
				if (TriMeshIndex == 0)
				{
					TManagedArray<int32>& TriangleMeshIndex = InCollection->AddAttribute<int32>("ObjectIndices", TriangleMeshGroup);
				}
				if (InCollection->HasAttributes({MType<int32>("ObjectIndices", TriangleMeshGroup) }))
				{
					TManagedArray<int32>* TriangleMeshIndexPtr = InCollection->FindAttribute<int32>("ObjectIndices", TriangleMeshGroup);
					for (int32 i = 0; i < TriangleMeshIndices.Num(); i++)
					{
						(*TriangleMeshIndexPtr)[i + TriMeshIndex] = TriangleMeshIndices[i];
					}
				}
			}
		}

		if (const TManagedArray<int32>* TriangleMeshIndices = InCollection->FindAttribute<int32>("ObjectIndices", "TriangleMesh"))
		{
			if (const TManagedArray<FIntVector>* Indices = InCollection->FindAttribute<FIntVector>("Indices", FGeometryCollection::FacesGroup))
			{
				if (const TManagedArray<int32>* FaceStarts = InCollection->FindAttribute<int32>("FaceStart", FGeometryCollection::GeometryGroup))
				{
					if (const TManagedArray<int32>* FaceCounts = InCollection->FindAttribute<int32>("FaceCount", FGeometryCollection::GeometryGroup))
					{
						if (const TManagedArray<FVector3f>* Vertex = InCollection->FindAttribute<FVector3f>("Vertex", "Vertices"))
						{
							if (TManagedArray<float>* Mass = InCollection->FindAttribute<float>("Mass", "Vertices"))
							{
								for (int32 i = 0; i < TriangleMeshIndices->Num(); i++)
								{
									const int32 ObjectIndex = (*TriangleMeshIndices)[i];
							
									const int32 FaceStartIndex = (*FaceStarts)[ObjectIndex];
									const int32 FaceNum = (*FaceCounts)[ObjectIndex];
									for (int32 e = FaceStartIndex; e < FaceStartIndex + FaceNum; e++)
									{
										const FVector3f X0 = (*Vertex)[(*Indices)[e][0]];
										const FVector3f X1 = (*Vertex)[(*Indices)[e][1]];
										const FVector3f X2 = (*Vertex)[(*Indices)[e][2]];
										float SingleElementMass = FVector3f::CrossProduct(X1 - X0, X2 - X0).Size() * TriangleMeshDensity/ 2.f;
										if (SingleElementMass < 0.f)
										{
											SingleElementMass = -SingleElementMass;
										}
										for (int32 k = 0; k < 3; k++)
										{
											const int32 MassIndex = (*Indices)[e][k];
											(*Mass)[MassIndex] += SingleElementMass/3.f;
										}
									}
								}
							}
						}	
					}
				}
			}


			if (const TManagedArray<int32>* VertexStarts = InCollection->FindAttribute<int32>("VertexStart", FGeometryCollection::GeometryGroup))
			{
				if (const TManagedArray<int32>* VertexCounts = InCollection->FindAttribute<int32>("VertexCount", FGeometryCollection::GeometryGroup))
				{
					for (int32 i = 0; i < TriangleMeshIndices->Num(); i++)
					{
						const int32 ObjectIndex = (*TriangleMeshIndices)[i];
							
						const int32 VertexStartIndex = (*VertexStarts)[ObjectIndex];
						const int32 VertexNum = (*VertexCounts)[ObjectIndex];
						for (int32 ParticleIndex = VertexStartIndex; ParticleIndex < VertexStartIndex + VertexNum; ParticleIndex++)
						{
							ParticleStiffness[ParticleIndex] = VertexTriangleMeshStiffness;
							ParticleDamping[ParticleIndex] = VertexTriangleMeshDamping;
						}
					}	
				}
			}
		}

		SetValue<const DataType&>(Context, *InCollection, &Collection);
	}
}