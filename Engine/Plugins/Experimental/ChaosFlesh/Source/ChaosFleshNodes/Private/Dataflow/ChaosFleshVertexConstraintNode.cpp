// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshVertexConstraintNode.h"

#include "GeometryCollection/Facades/CollectionKinematicBindingFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFleshVertexConstraintNode)

void FSetVerticesKinematicDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<DataType>(&Collection))
	{
		DataType InCollection = GetValue<DataType>(Context, &Collection);
		TArray<int32> BoundVerts;
		TArray<float> BoundWeights;
		if (FindInput(&VertexIndicesIn) && FindInput(&VertexIndicesIn)->GetConnection())
		{
			if (TManagedArray<FVector3f>* Vertices = InCollection.FindAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup))
			{

				for (int32 SelectionIndex : GetValue<TArray<int32>>(Context, &VertexIndicesIn))
				{
					if (0 <= SelectionIndex && SelectionIndex < Vertices->Num())
					{
						BoundVerts.Add(SelectionIndex);
					}
				}
				BoundWeights.Init(1.0, BoundVerts.Num());	
			}
		} 

		if (BoundVerts.Num() > 0)
		{
			GeometryCollection::Facades::FKinematicBindingFacade Kinematics(InCollection);
			Kinematics.AddKinematicBinding(Kinematics.SetBoneBindings(INDEX_NONE, BoundVerts, BoundWeights));
		}
		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}
