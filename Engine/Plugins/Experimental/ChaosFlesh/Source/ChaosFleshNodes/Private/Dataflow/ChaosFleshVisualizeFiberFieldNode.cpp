// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/ChaosFleshVisualizeFiberFieldNode.h"
#include "GeometryCollection/Facades/CollectionKinematicBindingFacade.h"
#include "GeometryCollection/Facades/CollectionPositionTargetFacade.h"
#include "ChaosFlesh/TetrahedralCollection.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosFleshVisualizeFiberFieldNode)

void FVisualizeFiberFieldNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FFieldCollection>(&VectorField))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		FFieldCollection OutVectorField = VectorField;
		
		if (TManagedArray<FVector3f>* Vertex = InCollection.FindAttribute<FVector3f>("Vertex", "Vertices"))
		{
			if (TManagedArray<FLinearColor>* Color = InCollection.FindAttribute<FLinearColor>("Color", "Vertices"))
			{
				if (TManagedArray<FIntVector4>* Elements = InCollection.FindAttribute<FIntVector4>(FTetrahedralCollection::TetrahedronAttribute, FTetrahedralCollection::TetrahedralGroup))
				{
					if (TManagedArray<FVector3f>* FiberDirections = InCollection.FindAttribute<FVector3f>("FiberDirection", FTetrahedralCollection::TetrahedralGroup))
					{
						ensureMsgf(Elements->Num() == FiberDirections->Num(), TEXT("Fiber direction has different size than elements"));
						for (int32 ElemIndex = 0; ElemIndex < Elements->Num(); ElemIndex++)
						{
							if ((*FiberDirections)[ElemIndex].Length() > UE_KINDA_SMALL_NUMBER) // This is supposed to be unit length
							{
								FVector3f VectorStart = { 0,0,0 };
								FLinearColor VectorColor(EForceInit::ForceInitToZero);

								for (int32 LocalIndex = 0; LocalIndex < 4; LocalIndex++)
								{
									VectorStart += (*Vertex)[(*Elements)[ElemIndex][LocalIndex]];
									VectorColor += (*Color)[(*Elements)[ElemIndex][LocalIndex]];
								}
								VectorStart /= float(4);
								VectorColor /= float(4);
								FVector3f VectorEnd = VectorStart + (*FiberDirections)[ElemIndex] * VectorScale;
								int32 VectorIndex = OutVectorField.AddVectorToField(VectorStart, VectorEnd);
								OutVectorField.SetColorOnVector(VectorIndex, VectorColor);
							}
						}
					}
				}
			}
		}
		
		Out->SetValue(MoveTemp(OutVectorField), Context);
	}
}

void FVisualizePositionTargetsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FFieldCollection>(&VectorField))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		FFieldCollection OutVectorField = VectorField;

		if (TManagedArray<FVector3f>* Vertex = InCollection.FindAttribute<FVector3f>("Vertex", "Vertices"))
		{
			GeometryCollection::Facades::FPositionTargetFacade PositionTargets(InCollection);

			// Read in position target info
			for (int i = 0; i < PositionTargets.NumPositionTargets(); i++)
			{
				GeometryCollection::Facades::FPositionTargetsData DataPackage = PositionTargets.GetPositionTarget(i);			
				FVector3f VectorStart = { 0,0,0 };
				for (int32 LocalIndex = 0; LocalIndex < DataPackage.SourceIndex.Num(); LocalIndex++)
				{
					VectorStart += DataPackage.SourceWeights[LocalIndex] * (*Vertex)[DataPackage.SourceIndex[LocalIndex]];
				}
				FVector3f VectorEnd = { 0,0,0 };
				for (int32 LocalIndex = 0; LocalIndex < DataPackage.TargetIndex.Num(); LocalIndex++)
				{
					VectorEnd += DataPackage.TargetWeights[LocalIndex] * (*Vertex)[DataPackage.TargetIndex[LocalIndex]];
				}
				OutVectorField.AddVectorToField(VectorStart, VectorEnd);
			}
		}

		Out->SetValue(MoveTemp(OutVectorField), Context);
	}
}

void FVisualizeKinematicFacesNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		TManagedArray<FIntVector>* Indices = InCollection.FindAttribute<FIntVector>("Indices", FGeometryCollection::FacesGroup);
		TManagedArray<bool>* FaceVisibility = InCollection.FindAttribute<bool>("Visible", FGeometryCollection::FacesGroup);
		TManagedArray<FVector3f>* Vertices = InCollection.FindAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
		GeometryCollection::Facades::FVertexBoneWeightsFacade VertexBoneWeightsFacade(InCollection);
		if (Indices && FaceVisibility && Vertices && VertexBoneWeightsFacade.IsValid())
		{
			FaceVisibility->Fill(false);
			for (int32 FaceIdx = 0; FaceIdx < Indices->Num(); ++FaceIdx)
			{
				bool IsElementKinematic = true;
				for (int32 j = 0; j < 3; j++)
				{
					if (!VertexBoneWeightsFacade.IsKinematicVertex((*Indices)[FaceIdx][j]))
					{
						IsElementKinematic = false;
						break;
					}
				}
				if (IsElementKinematic)
				{
					(*FaceVisibility)[FaceIdx] = true;
				}
			}
		}
		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}