// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionAppendCollectionTransformNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionAppendCollectionTransformNode)

void
FAppendToCollectionTransformAttributeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<DataType>(&Collection))
	{
		const FTransform& Transform = GetValue<FTransform>(Context, &TransformIn);
		FManagedArrayCollection CollectionValue = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TManagedArray<FTransform>* ConstTransformsPtr = CollectionValue.FindAttributeTyped<FTransform>(FName(AttributeName), FName(GroupName));
		if (!ConstTransformsPtr)
		{
			CollectionValue.AddAttribute<FTransform>(FName(AttributeName), FName(GroupName));
			ConstTransformsPtr = CollectionValue.FindAttributeTyped<FTransform>(FName(AttributeName), FName(GroupName));
		}

		if(ConstTransformsPtr)
		{
			int32 Index = CollectionValue.AddElements(1, FName(GroupName));
			TManagedArray<FTransform>& Transforms = CollectionValue.ModifyAttribute<FTransform>(FName(AttributeName), FName(GroupName));
			Transforms[Index] = Transform;
		}
		SetValue(Context, MoveTemp(CollectionValue), &Collection);
	}
}