// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/ChaosFleshNodesUtility.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "ChaosFleshAddKinematicParticlesNode.generated.h"

class USkeletalMesh;

USTRUCT(meta = (DataflowFlesh))
struct FAddKinematicParticlesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FAddKinematicParticlesDataflowNode, "AddKinematicParticles", "Flesh", "")

public:
	typedef FManagedArrayCollection DataType;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
		FTransform Transform;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
		ESkeletalSeletionMode SkeletalSelectionMode = ESkeletalSeletionMode::Dataflow_SkeletalSelection_Single;

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
		FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowInput, DisplayName = "SkeletalMesh"))
		TObjectPtr<USkeletalMesh> SkeletalMeshIn;

	UPROPERTY(meta = (DataflowInput, DisplayName = "SelectionSet"))
		TArray<int32> VertexIndicesIn;

	UPROPERTY(meta = (DataflowInput, DisplayName = "BoneIndex"))
		int32 BoneIndexIn = 0;

	UPROPERTY(meta = (DataflowOutput, DisplayName = "TargetIndices"))
		TArray<int32> TargetIndicesOut;



	FAddKinematicParticlesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterInputConnection(&SkeletalMeshIn);
		RegisterInputConnection(&VertexIndicesIn);
		RegisterInputConnection(&BoneIndexIn);
		RegisterOutputConnection(&TargetIndicesOut);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

