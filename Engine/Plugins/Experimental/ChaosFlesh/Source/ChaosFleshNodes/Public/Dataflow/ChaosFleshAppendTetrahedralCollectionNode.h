// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Dataflow/DataflowInputOutput.h"
#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArray.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollection.h"
#include "Dataflow/DataflowSelection.h"

#include "ChaosFleshAppendTetrahedralCollectionNode.generated.h"

/**
 * Append another Tetrahedral Collection to this collection. All attributes will be appended to the end.
 */
USTRUCT(meta = (DataflowFlesh))
struct FAppendTetrahedralCollectionDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FAppendTetrahedralCollectionDataflowNode_v2, "AppendTetrahedralCollection", "Flesh", "")
public:
	FAppendTetrahedralCollectionDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowRenderGroups = "Surface"));
	FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowInput));
	FManagedArrayCollection CollectionToAppend;

	/* If checked, non-geometry transforms from CollectionToAppend are merged into Collection by matching transform names. */
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DisplayName = "Merge Transform"));
	bool bMergeTransform = false;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowGeometrySelection CollectionGeometrySelection;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowGeometrySelection CollectionToAppendGeometrySelection;

	UPROPERTY(meta = (DataflowOutput))
	TArray<FString> CollectionGeometryGroupGuids;

	UPROPERTY(meta = (DataflowOutput))
	TArray<FString> CollectionToAppendGeometryGroupGuids;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Append another Tetrahedral Collection to this collection. All attributes will be copied.
 */
USTRUCT(meta = (Deprecated = "5.7"))
struct FAppendTetrahedralCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FAppendTetrahedralCollectionDataflowNode, "AppendTetrahedralCollection", "Flesh", "")

public:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection1", DataflowPassthrough = "Collection1", DataflowRenderGroups = "Surface"));
	FManagedArrayCollection Collection1;

	UPROPERTY(meta = (DataflowInput, DisplayName = "Collection2"));
	FManagedArrayCollection Collection2;

	/* if transforms from Collection2 are merged into Collection1 by matching transform name */
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DisplayName = "Merge Transform"));
	bool bMergeTransform = false;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowGeometrySelection GeometrySelection1;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowGeometrySelection GeometrySelection2;

	UPROPERTY(meta = (DataflowOutput, DisplayName = "GeometryGroupGuids1"))
	TArray<FString> GeometryGroupGuidsOut1;

	UPROPERTY(meta = (DataflowOutput, DisplayName = "GeometryGroupGuids2"))
	TArray<FString> GeometryGroupGuidsOut2;

	FAppendTetrahedralCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection1);
		RegisterInputConnection(&Collection2);
		RegisterOutputConnection(&Collection1, &Collection1);
		RegisterOutputConnection(&GeometryGroupGuidsOut1);
		RegisterOutputConnection(&GeometryGroupGuidsOut2);
		RegisterOutputConnection(&GeometrySelection1);
		RegisterOutputConnection(&GeometrySelection2);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Extract a Tetrahedral Collection from this collection based on selected vertex. Compatible attributes will be copied.
 */
USTRUCT(meta = (DataflowFlesh))
struct FDeleteFleshVerticesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDeleteFleshVerticesDataflowNode, "DeleteFleshVertices", "Flesh", "")

public:
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic, DataflowOutput, DataflowPassthrough = "Collection"));
	FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowVertexSelection VertexSelection;

	FDeleteFleshVerticesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&VertexSelection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};