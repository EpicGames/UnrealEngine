// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "ChaosFleshAuthorSceneCollisionCandidatesNode.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAuthorSceneCollisionCandidates, Verbose, All);

/**
* Marks mesh vertices as candidates for scene collision raycasts.  Each vertex has an associated 
* bone index to use as the origin of the raycast.  The runtime distance between the vertex and the
* bone designates the range of the scene query depth.
*/
USTRUCT(meta = (DataflowFlesh))
struct FAuthorSceneCollisionCandidates : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FAuthorSceneCollisionCandidates, "AuthorSceneCollisionCandidates", "Flesh", "")

public:
	typedef FManagedArrayCollection DataType;

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	//! Restricts vertices to only ones on the surface.  All vertices otherwise.
	UPROPERTY(EditAnywhere, Category = "Dataflow")
	bool bSurfaceVerticesOnly = true;

	//! Indices to use with environment collisions.  If this input is not connected, then all 
	//! indicies are used.
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowInput, DisplayName = "Vertex Indices"))
	TArray<int32> VertexIndices;

	//! Bone index to use as the world raycast origin.  -1 denotes the component transform.
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowInput, DisplayName = "Raycast Origin Bone Index"))
	int32 OriginBoneIndex = 0;

	FAuthorSceneCollisionCandidates(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterInputConnection(&VertexIndices);
		RegisterInputConnection(&OriginBoneIndex);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
