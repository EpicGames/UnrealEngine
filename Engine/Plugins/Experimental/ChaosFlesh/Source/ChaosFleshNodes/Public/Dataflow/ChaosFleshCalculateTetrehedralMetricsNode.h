// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "ChaosFleshCalculateTetrehedralMetricsNode.generated.h"


// Generate quality metrics
USTRUCT(meta = (DataflowFlesh))
struct FCalculateTetMetrics : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
		DATAFLOW_NODE_DEFINE_INTERNAL(FCalculateTetMetrics, "AuthorTetMetrics", "Flesh", "")
public:
	typedef FManagedArrayCollection DataType;

	// Passthrough geometry collection. Bindings are stored as standalone groups in the \p Collection, keyed by the name of the input render mesh and all available LOD's.
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DisplayName = "Collection"))
		FManagedArrayCollection Collection;

	FCalculateTetMetrics(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};




