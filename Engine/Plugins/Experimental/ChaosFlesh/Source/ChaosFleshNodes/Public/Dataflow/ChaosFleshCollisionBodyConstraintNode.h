// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "ChaosFleshCollisionBodyConstraintNode.generated.h"

class USkeletalMesh;

// @todo(deprecate), rename to FCollisionBodyConstraintDataflowNode

USTRUCT(meta = (DataflowFlesh))
struct FKinematicBodySetupInitializationDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FKinematicBodySetupInitializationDataflowNode, "KinematicBodySetupInitialization", "Flesh", "")

public:
	typedef FManagedArrayCollection DataType;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
		FTransform Transform;

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
		FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowInput, DisplayName = "SkeletalMesh"))
		TObjectPtr<USkeletalMesh> SkeletalMeshIn;

	FKinematicBodySetupInitializationDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterInputConnection(&SkeletalMeshIn);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

