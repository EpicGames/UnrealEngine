// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"
#include "Field/FieldSystemTypes.h"

#include "ChaosFleshComputeFiberFieldNode.generated.h"



/**
* Computes a muscle fiber direction per tetrahedron from a GeometryCollection containing tetrahedra, 
* vertices, and origin & insertion vertex fields.  Fiber directions should smoothly follow the geometry
* oriented from the origin vertices pointing to the insertion vertices.
*/
USTRUCT(meta = (DataflowFlesh))
struct FComputeFiberFieldNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FComputeFiberFieldNode, "ComputeFiberField", "Flesh", "")

public:
	//typedef FManagedArrayCollection DataType;

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowInput, DisplayName = "OriginVertexIndices"))
	TArray<int32> OriginIndices;

	UPROPERTY(meta = (DataflowInput, DisplayName = "InsertionVertexIndices"))
	TArray<int32> InsertionIndices;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	FString OriginInsertionGroupName = FString();

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	FString OriginVertexFieldName = FString("Origin");

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	FString InsertionVertexFieldName = FString("Insertion");

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	int32 MaxIterations = 100;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	float Tolerance = 1.0e-7f;

	//If muscles are colored by the flow from origins (blue) to insertions (red). This becomes optional in 5.6
	UPROPERTY(EditAnywhere, Category = "Dataflow")
	bool bShowMuscleColor = true;

	FComputeFiberFieldNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&OriginIndices);
		RegisterInputConnection(&InsertionIndices);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

	TArray<int32> GetNonZeroIndices(const TArray<uint8>& Map) const;

	void ComputeFiberField(
		const TManagedArray<FIntVector4>& Elements,
		const TManagedArray<FVector3f>& Vertex,
		const TManagedArray<TArray<int32>>& IncidentElements,
		const TManagedArray<TArray<int32>>& IncidentElementsLocalIndex,
		const TArray<int32>& Origin,
		const TArray<int32>& Insertion,
		TArray<FVector3f>& Directions,
		TArray<float>& ScalarField) const;
};

/**
* Computes fiber streamlines (line segments) flowing from muscle origins to insertions in the muscle fiber field.
*/
USTRUCT(meta = (DataflowFlesh))
struct FComputeFiberStreamlineNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FComputeFiberStreamlineNode, "ComputeFiberStreamline", "Flesh", "")

public:
	//typedef FManagedArrayCollection DataType;

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowInput, DisplayName = "OriginVertexIndices"))
	TArray<int32> OriginIndices;

	UPROPERTY(meta = (DataflowInput, DisplayName = "InsertionVertexIndices"))
	TArray<int32> InsertionIndices;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	FString OriginInsertionGroupName = FString();

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	FString OriginVertexFieldName = FString("Origin");

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	FString InsertionVertexFieldName = FString("Insertion");

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	int32 MaxStreamlineIterations = 500;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	int32 MaxPointsPerLine = 20;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	int32 NumLinesMultiplier = 1;

	UPROPERTY(meta = (DataflowOutput, DisplayName = "VectorField"))
	FFieldCollection VectorField;

	FComputeFiberStreamlineNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&OriginIndices);
		RegisterInputConnection(&InsertionIndices);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&VectorField);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};