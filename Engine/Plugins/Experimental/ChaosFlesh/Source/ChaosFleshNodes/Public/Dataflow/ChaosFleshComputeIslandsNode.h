// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "ChaosFleshComputeIslandsNode.generated.h"

//@todo(deprecate)
USTRUCT(meta = (DataflowFlesh))
struct FComputeIslandsNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FComputeIslandsNode, "ComputeIslands", "Flesh", "")

public:

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	FComputeIslandsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
