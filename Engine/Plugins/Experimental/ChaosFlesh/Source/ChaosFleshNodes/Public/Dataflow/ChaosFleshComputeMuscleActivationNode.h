// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"
#include "Dataflow/DataflowFunctionProperty.h"
#include "Curves/CurveFloat.h"

#include "ChaosFleshComputeMuscleActivationNode.generated.h"

class UAnimSequence;

/**
* Please use new version of this node, then use SetMuscleActivationParameter node to set up muscle activation parameters.
* Computes an orthogonal matrix for each element
* M = [v,w,u], where v is the fiber direction of that element, w and u are chosen to be orthogonal to v and each other.
*/
USTRUCT(meta = (Deprecated = "5.6"))
struct FComputeMuscleActivationDataNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FComputeMuscleActivationDataNode, "ComputeMuscleActivationData", "Flesh", "")

public:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowInput, DisplayName = "OriginVertexIndices"))
	TArray<int32> OriginIndicesIn;

	UPROPERTY(meta = (DataflowInput, DisplayName = "InsertionVertexIndices"))
	TArray<int32> InsertionIndicesIn;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	float ContractionVolumeScale = 1.f;

	FComputeMuscleActivationDataNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&OriginIndicesIn);
		RegisterInputConnection(&InsertionIndicesIn);
		RegisterOutputConnection(&Collection, &Collection);
	}
	
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
* Determines muscles that are eligible for activation and computes muscle activation data.
*/
USTRUCT(meta = (DataflowFlesh))
struct FComputeMuscleActivationDataNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FComputeMuscleActivationDataNode_v2, "ComputeMuscleActivationData", "Flesh", "")

public:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowInput, DisplayName = "OriginVertexIndices"))
	TArray<int32> OriginIndicesIn;

	UPROPERTY(meta = (DataflowInput, DisplayName = "InsertionVertexIndices"))
	TArray<int32> InsertionIndicesIn;

	FComputeMuscleActivationDataNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&OriginIndicesIn);
		RegisterInputConnection(&InsertionIndicesIn);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

class FLengthActivationUtils
{
public:
	/** Sets a default linear (0,0) -> (1,1) curve */
	static void SetDefaultLengthActivationCurve(FRuntimeFloatCurve& OutCurve)
	{
		FRichCurve* RichCurve = OutCurve.GetRichCurve();
		if (RichCurve)
		{
			RichCurve->Reset();
			RichCurve->AddKey(0.0f, 0.0f);
			RichCurve->AddKey(1.0f, 1.0f);
		}
	}
};

/**
* Struct data structure to store per-muscle activation parameters
*/
USTRUCT()
struct FPerMuscleParameter
{
	GENERATED_USTRUCT_BODY()
	UPROPERTY(EditAnywhere, meta = (InlineEditConditionToggle), Category = "Muscle Activation Parameters")
	bool bCanEditMuscleName = false;
	UPROPERTY(EditAnywhere, meta = (EditCondition = "bCanEditMuscleName"), Category = "Muscle Activation Parameters")
	FString MuscleName;
	UPROPERTY(EditAnywhere, Category = "Muscle Activation Parameters", meta = (ClampMin = "0"))
	float ContractionVolumeScale = 1.f;
	UPROPERTY(EditAnywhere, Category = "Muscle Activation Parameters", meta = (ClampMin = "0", ClampMax = "1"))
	float FiberLengthRatioAtMaxActivation = 0.5f;
	UPROPERTY(EditAnywhere, Category = "Muscle Activation Parameters", meta = (ClampMin = "0", ClampMax = "1"))
	float MuscleLengthRatioThresholdForMaxActivation = 0.75f;
	UPROPERTY(EditAnywhere, Category = "Muscle Activation Parameters", meta = (ClampMin = "0"))
	float InflationVolumeScale = 1.f;
	UPROPERTY(EditAnywhere, meta = (InlineEditConditionToggle), Category = "Muscle Activation Parameters")
	bool bUseLengthActivationCurve = false;
	UPROPERTY(EditAnywhere, Category = "Muscle Activation Parameters", meta = (EditCondition = "bUseLengthActivationCurve"))
	FRuntimeFloatCurve LengthActivationCurve;

	FPerMuscleParameter()
	{
		FLengthActivationUtils::SetDefaultLengthActivationCurve(LengthActivationCurve);
	}
};

UENUM()
enum EParameterMethod : int
{
	Global	UMETA(DisplayName = "Use global parameters"),
	Custom	UMETA(DisplayName = "Override with custom parameters"),
};

/**
* Sets per-muscle parameters for custom muscle contraction.
*/
USTRUCT(meta = (DataflowFlesh))
struct FSetMuscleActivationParameterNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSetMuscleActivationParameterNode, "SetMuscleActivationParameter", "Flesh", "")

public:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(EditAnywhere, Category = "Parameter Method", meta = (EditCondition = false))
	TEnumAsByte<EParameterMethod> ParameterMethod = EParameterMethod::Global;

	/** Click on this button to apply global parameters. */
	UPROPERTY(EditAnywhere, Category = "Global Parameters", meta = (ButtonImage = "Icons.Refresh"))
	FDataflowFunctionProperty ApplyGlobalParameters;

	/** Muscles gain volume during contraction if > 1. Volume-preserving if 1. Default: 1 */
	UPROPERTY(EditAnywhere, Category = "Global Parameters", DisplayName = "Global Contraction Volume Scale", meta = (ClampMin = "0"))
	float ContractionVolumeScale = 1.f;

	/** How much muscle fibers shorten at max activation 1. A smaller value means more contraction in the fiber direction. Default: 0.5 */
	UPROPERTY(EditAnywhere, Category = "Global Parameters", meta = (ClampMin = "0", ClampMax = "1"))
	float GlobalFiberLengthRatioAtMaxActivation = 0.5f;

	/** Muscle length ratio (defined by origin-insertion distance) below this threshold is considered to reach max activation 1. Default: 0.75 */
	UPROPERTY(EditAnywhere, Category = "Global Parameters", meta = (ClampMin = "0", ClampMax = "1"))
	float GlobalMuscleLengthRatioThresholdForMaxActivation = 0.75f;

	/** Inflates muscle rest volume if > 1 and deflates muscle rest volume if < 1. Default: 1 */
	UPROPERTY(EditAnywhere, Category = "Global Parameters", meta = (ClampMin = "0"))
	float GlobalInflationVolumeScale = 1.f;
	
	UPROPERTY(EditAnywhere, meta = (InlineEditConditionToggle), Category = "Global Parameters")
	bool bUseLengthActivationCurve = false;

	/** Curve editor for muscle length-activation curve. Default: linear
	* X-axis: normalized muscle length level from 0 (rest state) to 1 (muscle length ratio = Muscle Length Ratio Threshold For Max Activation)
	* Y-axis: muscle activation from 0 to 1 */
	UPROPERTY(EditAnywhere, Category = "Global Parameters", meta = (EditCondition = "bUseLengthActivationCurve"))
	FRuntimeFloatCurve GlobalLengthActivationCurve;

	/** Use minimum muscle lengths across this animation asset to infer MuscleLengthRatioThresholdForMaxActivation*/
	UPROPERTY(EditAnywhere, Category = "Animation Data", meta = (DataflowInput))
	TObjectPtr<UAnimSequence> AnimationAsset = nullptr;

	/** Skeletal mesh used to linear blend skin kinematic origins and insertions. 
	* This must be the same as the one used to create TransformSource group. 
	* Check geometry spreadsheet for more info.*/
	UPROPERTY(EditAnywhere, Category = "Animation Data", meta = (DataflowInput))
	TObjectPtr<USkeletalMesh> SkeletalMesh = nullptr;

	/** This percentage (usually between 0 and 100%) globally scales minimum muscle lengths computed from the animation asset.
	* For example, if minimum muscle length = 0.5 and ThresholdScalingPercent = 80%,
	* the final MuscleLengthRatioThresholdForMaxActivation for this muscle is 1 - (1 - 0.5) * 80% = 0.6 */
	UPROPERTY(EditAnywhere, Category = "Animation Data", meta = (ClampMin = "0.0", ClampMax = "200.0", UIMin = "0.0", UIMax = "200.0", Units = "Percent"))
	float ThresholdScalingPercent = 100.f;

	/** Click on this button to import the minimum muscle lengths across the animation asset as MuscleLengthRatioThresholdForMaxActivation*/
	UPROPERTY(EditAnywhere, Category = "Animation Data", meta = (ButtonImage = "Icons.Import"))
	FDataflowFunctionProperty ImportLowestMuscleLengthRatio;

	/** Click on this button to import muscle bone names (in the Transform group) from the collection. */
	UPROPERTY(EditAnywhere, Category = "Custom Parameters", meta = (ButtonImage = "Icons.Import"))
	FDataflowFunctionProperty ImportAllMuscleNames;

	/** Click on this button to reset per-muscle parameters to global parameters. */
	UPROPERTY(EditAnywhere, Category = "Custom Parameters", meta = (ButtonImage = "Icons.Import"))
	FDataflowFunctionProperty ResetToGlobalParameters;

	/** Click on this button to apply custom muscle parameters. */
	UPROPERTY(EditAnywhere, Category = "Custom Parameters", meta = (ButtonImage = "Icons.Refresh"))
	FDataflowFunctionProperty ApplyCustomParameters;

	UPROPERTY(EditAnywhere, Category = "Custom Parameters", DisplayName = "Per-muscle Parameters")
	TArray<FPerMuscleParameter> ParameterArray;

	FSetMuscleActivationParameterNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual bool ShouldInvalidateOnPropertyChanged(const FPropertyChangedEvent& InPropertyChangedEvent) const override;
};

/**
* Struct data structure to store curve names and muscle names
*/
USTRUCT()
struct FCurveMuscleName
{
	GENERATED_USTRUCT_BODY()
	UPROPERTY(EditAnywhere, Category = "Muscle Activation Parameters")
	FString CurveName;
	UPROPERTY(EditAnywhere, Category = "Muscle Activation Parameters")
	FString MuscleName;

	FCurveMuscleName() = default;
	FCurveMuscleName(const FString& InCurveName, const FString& InMuscleName)
		: CurveName(InCurveName), MuscleName(InMuscleName) {};
};

USTRUCT(meta = (DataflowFlesh))
struct FReadSkeletalMeshCurvesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FReadSkeletalMeshCurvesDataflowNode, "ReadSkeletalMeshCurves", "Flesh", "")

public:
	FReadSkeletalMeshCurvesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowInput, DisplayName = "SkeletalMesh"))
	TObjectPtr<const USkeletalMesh> SkeletalMesh = nullptr;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowGeometrySelection GeometrySelection;

	/** Click on this button to import curve names from the skeletal mesh. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh Curves", meta = (ButtonImage = "Icons.Import"))
	FDataflowFunctionProperty ImportSKMCurveNames;

	/** Click on this button to assign curve to muscles by name. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh Curves", meta = (ButtonImage = "Icons.Refresh"))
	FDataflowFunctionProperty AssignSKMCurveToMuscle;

	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh Curves", DisplayName = "Curve Name/Muscle Name")
	TArray<FCurveMuscleName> CurveMuscleNameArray;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual bool ShouldInvalidateOnPropertyChanged(const FPropertyChangedEvent& InPropertyChangedEvent) const override;

};