// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/ChaosFleshNodesUtility.h"

#include "ChaosFleshCreateTetrahedronNode.generated.h"


class FFleshCollection;
namespace UE {
	namespace Geometry {
		class FDynamicMesh3;
	}
}

// Note: disabled deprecation warnings due to deprecated member: IdealEdgeLength. Can remove these disable/enable pragmas once that member variable is removed.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
USTRUCT(meta = (DataflowFlesh))
struct FCreateTetrahedronDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	DATAFLOW_NODE_DEFINE_INTERNAL(FCreateTetrahedronDataflowNode, "CreateTetrahedron", "Flesh", "")

public:
	typedef FManagedArrayCollection DataType;

	// Tetrahedral meshing method (TetWild or ISO-stuffing)
	UPROPERTY(EditAnywhere, Category = "Dataflow")
	TEnumAsByte<TetMeshingMethod> Method = TetMeshingMethod::IsoStuffing;

	// Name of the mesh in the collection. This is defined from the name of the mesh's
	// parent transform. CollectionKey("BoneName","Vertices") 
	UPROPERTY(EditAnywhere, Category = "Dataflow")
	TArray<FString> Selection;

	//
	// IsoStuffing
	//

	// General control for density for the tetrahedron. 
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "1", EditCondition = "Method == TetMeshingMethod::IsoStuffing", EditConditionHides))
	int32 NumCells = 32;

	// Surface offset percentage to increase or decrease the surface alignment. 
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "-0.5", ClampMax = "0.5", EditCondition = "Method == TetMeshingMethod::IsoStuffing", EditConditionHides))
	double OffsetPercent = 0.05;

	//
	// TetWild
	//

	//! Desired relative edge length, as a fraction of bounding box size
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "0.0", EditCondition = "Method == TetMeshingMethod::TetWild", EditConditionHides))
	double IdealEdgeLengthRel = 0.05;

	UE_DEPRECATED(5.6, "Use IdealEdgeLengthRel instead, which is relative to the bounding box size")
	double IdealEdgeLength = 0.05;

	//! Maximum number of optimization iterations.
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "1", EditCondition = "Method == TetMeshingMethod::TetWild", EditConditionHides))
	int32 MaxIterations = 80;

	//! Energy at which to stop optimizing tet quality and accept the result.
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "0.0", EditCondition = "Method == TetMeshingMethod::TetWild", EditConditionHides))
	double StopEnergy = 10;

	//! Relative tolerance, controlling how closely the mesh must follow the input surface.
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "0.0", EditCondition = "Method == TetMeshingMethod::TetWild", EditConditionHides))
	double EpsRel = 1e-3;

	//! Coarsen the tet mesh result.
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (EditCondition = "Method == TetMeshingMethod::TetWild", EditConditionHides))
	bool bCoarsen = false;

	//! Enforce that the output boundary surface should be manifold.
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (EditCondition = "Method == TetMeshingMethod::TetWild", EditConditionHides))
	bool bExtractManifoldBoundarySurface = false;

	//! Skip the initial simplification step.
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (EditCondition = "Method == TetMeshingMethod::TetWild", EditConditionHides))
	bool bSkipSimplification = false;

	//! Invert tetrahedra.
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (EditCondition = "Method == TetMeshingMethod::TetWild", EditConditionHides))
	bool bInvertOutputTets = false;

	//
	// Common
	//
	UPROPERTY(EditAnywhere, Category = "Dataflow")
	bool bDiscardInteriorTriangles = true;

	// Input pass-through collection. When connected, the generated tetrahedron will be nested into
	// its associated parents transform.
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection"))
	FManagedArrayCollection Collection;

	// Source collection used to generate the tetrahedron from. Closed geometry within the collections geometry group will be used to 
	// generate tetrahedron.
	UPROPERTY(meta = (DataflowInput, DisplayName = "SourceCollection"))
	FManagedArrayCollection SourceCollection;

	FCreateTetrahedronDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection);
		RegisterInputConnection(&SourceCollection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

protected:
	void EvaluateIsoStuffing(UE::Dataflow::FContext& Context, FFleshCollection& InCollection, const UE::Geometry::FDynamicMesh3& DynamicMesh) const;
	void EvaluateTetWild(UE::Dataflow::FContext& Context, FFleshCollection& InCollection, const UE::Geometry::FDynamicMesh3& DynamicMesh) const;
};



