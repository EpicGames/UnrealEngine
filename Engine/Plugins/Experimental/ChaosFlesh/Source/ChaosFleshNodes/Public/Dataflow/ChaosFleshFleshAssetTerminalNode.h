// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "ChaosFlesh/FleshAsset.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "ChaosFleshFleshAssetTerminalNode.generated.h"

class UAnimSequence;
class USkeletalMesh;

USTRUCT(meta = (DataflowFlesh, DataflowTerminal))
struct FFleshAssetTerminalDataflowNode : public FDataflowTerminalNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FFleshAssetTerminalDataflowNode, "FleshAssetTerminal", "Terminal", "")

public:

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection"))
	FManagedArrayCollection Collection;


	FFleshAssetTerminalDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
	: FDataflowTerminalNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
	}
	
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DisplayName = "FleshAsset"))
	TObjectPtr<UFleshAsset> FleshAsset = nullptr;

	/** Return the terminal asset */
	virtual TObjectPtr<UObject> GetTerminalAsset() const override {return FleshAsset;}

	virtual void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/*
* terminal node to create an animation asset for muscle activation MLD training. 
* The animation remains in the rest pose with the curves spiking from 0 to 1 to 0 in each block of frames (Number of frames per muscle)
* Curves will stay at value 0 most of the time. Only one curve is active at a time.
* Total animation frames = Frame Rate * Number of frames per muscle
*/
USTRUCT(meta = (DataflowFlesh, DataflowTerminal))
struct FCurveSamplingAnimationAssetTerminalNode : public FDataflowTerminalNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCurveSamplingAnimationAssetTerminalNode, "CurveSamplingAnimationAssetTerminal", "Terminal", "")

private:
	UPROPERTY(EditAnywhere, Category = Asset, meta = (DataflowInput))
	TObjectPtr<USkeletalMesh> SkeletalMeshAsset = nullptr;

	UPROPERTY(EditAnywhere, Category = Asset, meta = (DataflowInput))
	TObjectPtr<UAnimSequence> AnimationAsset = nullptr;

	/** Frame rate of the animation */
	UPROPERTY(EditAnywhere, Category = "Record Parameter")
	int32 FrameRate = 30;

	/** Number of frames created for each curve. Within this window of frames, curve value will go from 0 to 1 to 0. */
	UPROPERTY(EditAnywhere, Category = "Record Parameter", DisplayName = "Number of Frames Per Muscle")
	int32 NumFramesPerMuscle = 10;

	virtual void Evaluate(UE::Dataflow::FContext& Context) const override;
	virtual void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;

public:
	FCurveSamplingAnimationAssetTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
	: FDataflowTerminalNode(InParam, InGuid)
	{
		RegisterInputConnection(&SkeletalMeshAsset);
		RegisterInputConnection(&AnimationAsset);
	}
};
