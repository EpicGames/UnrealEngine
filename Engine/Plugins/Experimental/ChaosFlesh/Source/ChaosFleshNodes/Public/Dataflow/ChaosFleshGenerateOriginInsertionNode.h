// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "ChaosFleshGenerateOriginInsertionNode.generated.h"

//Given two sets of vertex indices, generate two sets of vertex indices for origins and insertions that are within X distance away.
USTRUCT(meta = (DataflowFlesh))
struct FGenerateOriginInsertionNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGenerateOriginInsertionNode, "GenerateOriginInsertion", "Flesh", "")

public:
	//typedef FManagedArrayCollection DataType;

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowInput, DisplayName = "OriginIndicesIn"))
	TArray<int32> OriginIndicesIn;

	UPROPERTY(meta = (DataflowInput, DisplayName = "InsertionIndicesIn"))
	TArray<int32> InsertionIndicesIn;

	UPROPERTY(meta = (DataflowOutput, DisplayName = "OriginIndicesOut"))
	TArray<int32> OriginIndicesOut;

	UPROPERTY(meta = (DataflowOutput, DisplayName = "InsertionIndicesOut"))
	TArray<int32> InsertionIndicesOut;
	
	UPROPERTY(EditAnywhere, Category = "Dataflow")
	float Radius = float(1);

	FGenerateOriginInsertionNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&OriginIndicesIn);
		RegisterInputConnection(&InsertionIndicesIn);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&OriginIndicesOut);
		RegisterOutputConnection(&InsertionIndicesOut);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
