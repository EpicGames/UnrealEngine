// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "ChaosFleshGenerateSkeletalBindingsNode.generated.h"

class UStaticMesh;
class USkeletalMesh;

DECLARE_LOG_CATEGORY_EXTERN(LogSkeletalBindings, Verbose, All);

// Generate barycentric bindings (used by the FleshDeformer deformer graph) of a render surface to a tetrahedral mesh.
USTRUCT(meta = (DataflowFlesh))
struct FGenerateSkeletalBindings : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGenerateSkeletalBindings, "GenerateSkeletalBindings", "Flesh", "")

public:
	typedef FManagedArrayCollection DataType;

	// Passthrough geometry collection. Bindings are stored as standalone groups in the \p Collection, keyed by the name of the input render mesh and all available LOD's.
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowInput, DisplayName = "BoneIndex"))
	int32 BoneIndexIn = 0;

	// The input mesh, whose render surface is used to generate bindings.
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowInput, DisplayName = "SkeletalMesh"))
	TObjectPtr<const USkeletalMesh> SkeletalMeshIn = nullptr;

	FGenerateSkeletalBindings(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterInputConnection(&BoneIndexIn);
		RegisterInputConnection(&SkeletalMeshIn);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};