// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#include "Dataflow/DataflowSelection.h"
#include "ChaosFleshGenerateSurfaceBindingsNode.generated.h"

class UStaticMesh;
class USkeletalMesh;
class UDynamicMesh;

DECLARE_LOG_CATEGORY_EXTERN(LogMeshBindings, Verbose, All);

/* Generate barycentric bindings (used by the FleshDeformer deformer graph and Geometry Cache generation) of a render surface to a tetrahedral mesh and its surface.
* If a point is outside of the tetrahedral mesh, find surface embedding within SurfaceProjectionSearchRadius.
Embeddings of LOD 0 are color coded in the render view:
	green: embedded on in a tetrahedron
	blue: embedded on a surface triangle
	red: orphan (cannot be embedded)
	yellow: orphan reparented to a tetrahedron from a node neighbor */
USTRUCT(meta = (DataflowFlesh))
struct FGenerateSurfaceBindings : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGenerateSurfaceBindings, "GenerateSurfaceBindings", "Flesh", "")

public:
	FGenerateSurfaceBindings(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterInputConnection(&StaticMeshIn);
		RegisterInputConnection(&SkeletalMeshIn);
		RegisterInputConnection(&TransformSelection)
			.SetCanHidePin(true)
			.SetPinIsHidden(true);
		RegisterInputConnection(&GeometryGroupGuidsIn)
			.SetCanHidePin(true)
			.SetPinIsHidden(true);
		RegisterOutputConnection(&SKMDynamicMesh)
			.SetCanHidePin(true)
			.SetPinIsHidden(true);
	}

private:
	/** Collection containing tetrahedral mesh and surface mesh. Bindings are stored as standalone groups in the \p Collection, keyed by the name of the input render mesh and all available LOD's. */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DisplayName = "Collection"))
	FManagedArrayCollection Collection;

	/** The input mesh, whose render surface is used to generate bindings. */
	UPROPERTY(EditAnywhere, Category = "Render Mesh", DisplayName = "Static Mesh", meta = (DataflowInput, EditCondition = "SkeletalMeshIn == nullptr"))
	TObjectPtr<const UStaticMesh> StaticMeshIn = nullptr;

	/** The input mesh, whose render surface is used to generate bindings. */
	UPROPERTY(EditAnywhere, Category = "Render Mesh", DisplayName = "Skeletal Mesh", meta = (DataflowInput, EditCondition = "StaticMeshIn == nullptr"))
	TObjectPtr<const USkeletalMesh> SkeletalMeshIn = nullptr;

	/** Render mesh will only bind to geometries whose transforms are in TransformSelection. */
	UPROPERTY(meta = (DataflowInput, DisplayName = "(Optional) TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	/** Render mesh will only bind to geometries whose GeometryGroupGuids match here. */
	UPROPERTY(meta = (DataflowInput, DisplayName = "(Optional) GeometryGroupGuids"))
	TArray<FString> GeometryGroupGuidsIn;

	/** Use the import geometry of the skeletal mesh. */
	bool bUseSkeletalMeshImportModel = false;

	/* Select skeletal mesh LODs to embed. Default empty list selects all LODs. */
	UPROPERTY(EditAnywhere, Category = "Render Mesh", meta = (DisplayName = "Skeletal Mesh LOD List", EditCondition = "SkeletalMeshIn != nullptr && !bUseSkeletalMeshImportModel"))
	TArray<int32> SkeletalMeshLODList;

	/** Enable binding to the exterior hull of the tetrahedron mesh. */
	UPROPERTY(EditAnywhere, Category = "Surface Projection", meta = (DisplayName = "DoSurfaceProjection"))
	bool bDoSurfaceProjection = true;

	/** The search radius when looking for surface triangles to bind to. */
	UPROPERTY(EditAnywhere, Category = "Surface Projection", meta = (DisplayName = "SurfaceProjectionSearchRadius", EditCondition = "bDoSurfaceProjection == true"))
	float SurfaceProjectionSearchRadius = 1.f;

	/** When nodes aren't contained in tetrahedra and surface projection fails, try to find suitable bindings by looking to neighboring parents. */
	UPROPERTY(EditAnywhere, Category = "Orphan Reparenting", meta = (DisplayName = "DoOrphanReparenting"))
	bool bDoOrphanReparenting = true;

	/** Converted from embedded skeletal/static mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> SKMDynamicMesh;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};