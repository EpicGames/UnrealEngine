// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "ChaosFlesh/FleshAsset.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "ChaosFleshGetFleshAssetNode.generated.h"

USTRUCT(meta = (DataflowFlesh))
struct FGetFleshAssetDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetFleshAssetDataflowNode, "GetFleshAsset", "Flesh", "")

public:

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DisplayName = "FleshAsset"))
	TObjectPtr<const UFleshAsset> FleshAsset = nullptr;

	UPROPERTY(meta = (DataflowOutput, DisplayName = "Collection"))
	FManagedArrayCollection Output;

	FGetFleshAssetDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterOutputConnection(&Output);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
