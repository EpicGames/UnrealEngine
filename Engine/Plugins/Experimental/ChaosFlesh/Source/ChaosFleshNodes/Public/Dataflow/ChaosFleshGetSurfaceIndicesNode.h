// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"
#include "Dataflow/DataflowSelection.h"
#include "ChaosFleshGetSurfaceIndicesNode.generated.h"

USTRUCT(meta = (DataflowFlesh))
struct FGetSurfaceIndicesNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetSurfaceIndicesNode, "GetSurfaceIndices", "Flesh", "")

public:
	UPROPERTY(meta = (DataflowInput, DisplayName = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowInput, DisplayName = "GeometryGroupGuidsIn"))
	TArray<FString> GeometryGroupGuidsIn;

	UPROPERTY(meta = (DataflowOutput, DisplayName = "SurfaceVertexSelection"))
	FDataflowVertexSelection SurfaceVertexSelection;

	FGetSurfaceIndicesNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&GeometryGroupGuidsIn);
		RegisterOutputConnection(&SurfaceVertexSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

namespace UE::Dataflow
{
	void RegisterChaosFleshEngineAssetNodes();
}

