// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "ChaosFleshIsolateComponentNode.generated.h"


USTRUCT(meta = (DataflowFlesh))
struct FIsolateComponentNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FIsolateComponentNode, "IsolateComponent", "Flesh", "")

public:
	//typedef FManagedArrayCollection DataType;

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	bool bDeleteHiddenFaces = false;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	FString TargetGeometryIndex = "";

	FIsolateComponentNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
