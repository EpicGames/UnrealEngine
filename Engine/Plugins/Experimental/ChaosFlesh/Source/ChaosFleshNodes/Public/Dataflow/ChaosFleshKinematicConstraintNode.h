// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"
#include "Dataflow/ChaosFleshNodesUtility.h"

#include "ChaosFleshKinematicConstraintNode.generated.h"

class USkeletalMesh;

// @todo(deprecate), rename to FKinematicConstraintDataflowNode
USTRUCT(meta = (DataflowFlesh))
struct FKinematicInitializationDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FKinematicInitializationDataflowNode, "KinematicInitialization", "Flesh", "")

public:
	typedef FManagedArrayCollection DataType;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	float Radius = 40.f;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	ESkeletalSeletionMode SkeletalSelectionMode = ESkeletalSeletionMode::Dataflow_SkeletalSelection_Single;

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowInput, DisplayName = "SkeletalMesh"))
	TObjectPtr<USkeletalMesh> SkeletalMeshIn;
		
	UPROPERTY(meta = (DataflowInput, DisplayName = "VertexIndices"))
	TArray<int32> VertexIndicesIn;

	UPROPERTY(meta = (DataflowInput, DisplayName = "BoneIndex"))
	int32 BoneIndexIn = INDEX_NONE;
	

	FKinematicInitializationDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterInputConnection(&SkeletalMeshIn);
		RegisterInputConnection(&VertexIndicesIn);
		RegisterInputConnection(&BoneIndexIn);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
