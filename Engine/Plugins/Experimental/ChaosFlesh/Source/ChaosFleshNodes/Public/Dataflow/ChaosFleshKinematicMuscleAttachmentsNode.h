// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "ChaosFleshKinematicMuscleAttachmentsNode.generated.h"

class USkeletalMesh;

USTRUCT(meta = (DataflowFlesh))
struct FKinematicMuscleAttachmentsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FKinematicMuscleAttachmentsDataflowNode, "KinematicMuscleAttachments", "Flesh", "")

public:
	typedef FManagedArrayCollection DataType;

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
		FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowInput, DisplayName = "OriginVertexIndices"))
		TArray<int32> OriginVertexIndicesIn;

	UPROPERTY(meta = (DataflowInput, DisplayName = "InsertionVertexIndices"))
		TArray<int32> InsertionVertexIndicesIn;

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowInput, DisplayName = "SkeletalMesh"))
		TObjectPtr<USkeletalMesh> SkeletalMeshIn;


	FKinematicMuscleAttachmentsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterInputConnection(&OriginVertexIndicesIn);
		RegisterInputConnection(&InsertionVertexIndicesIn);
		RegisterInputConnection(&SkeletalMeshIn);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

