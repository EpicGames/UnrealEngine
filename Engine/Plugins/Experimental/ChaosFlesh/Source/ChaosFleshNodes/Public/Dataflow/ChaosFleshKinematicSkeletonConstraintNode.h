// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "ChaosFleshKinematicSkeletonConstraintNode.generated.h"

class USkeleton;

/* Bind a animation driven skeleton hierarchy into the tetrahedron on the collection. */
USTRUCT(meta = (DataflowFlesh))
struct FKinematicSkeletonConstraintDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FKinematicSkeletonConstraintDataflowNode, "KinematicSkeletonConstraint", "Flesh", "")

public:
	typedef FManagedArrayCollection DataType;

	/* Pass through collection to place constraints in to. */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/* Skeleton to constraint to the tetrahedron (Must be co-located with the tetrahedron). */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Skeleton"))
	TObjectPtr<USkeleton> SkeletonIn = nullptr;

	/* Skeleton bones to exclude from the constraint. */
	UPROPERTY(EditAnywhere, Category = "Dataflow")
	FString ExclusionList = "";

	FKinematicSkeletonConstraintDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterInputConnection(&SkeletonIn);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

