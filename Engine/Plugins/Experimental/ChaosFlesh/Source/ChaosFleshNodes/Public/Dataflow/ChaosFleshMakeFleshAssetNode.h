// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/DataflowNode.h"
#include "ChaosFlesh/FleshAsset.h"

#include "ChaosFleshMakeFleshAssetNode.generated.h"

/** Converts managed array collection to a specified asset. */
USTRUCT(Meta = (DataflowFlesh))
struct FMakeFleshAssetNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeFleshAssetNode, "MakeFleshAsset", "Flesh", "Asset")

public:
	FMakeFleshAssetNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput*) const override;
	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override { return true; }
	virtual bool CanRemovePin() const override { return CollectionLods.Num() > NumInitialCollectionLods; }
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	virtual void PostSerialize(const FArchive& Ar) override;
	//~ End FDataflowNode interface

	TArray<TUniquePtr<FFleshCollection>> GetCleanedCollectionLodValues(UE::Dataflow::FContext& Context) const;
	UE::Dataflow::TConnectionReference<FManagedArrayCollection> GetConnectionReference(int32 Index) const;

	/** Input collection for this LOD. */
	UPROPERTY()
	TArray<FManagedArrayCollection> CollectionLods;


	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UFleshAsset> Asset;

	// This is for runtime only--used to determine if only properties need to be updated.
	static constexpr int32 NumRequiredInputs = 0;
	static constexpr int32 NumInitialCollectionLods = 1;
};