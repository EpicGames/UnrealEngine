// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ChaosFleshNodesUtility.generated.h"

struct FManagedArrayCollection;

UENUM()
enum TetMeshingMethod : int
{
	IsoStuffing		UMETA(DisplayName = "IsoStuffing"),
	TetWild			UMETA(DisplayName = "TetWild"),
};

UENUM(BlueprintType)
enum class ESkeletalSeletionMode : uint8
{
	Dataflow_SkeletalSelection_Single UMETA(DisplayName = "Single"),
	Dataflow_SkeletalSelection_Branch UMETA(DisplayName = "Sub-Branch"),
	//
	Chaos_Max UMETA(Hidden)
};

namespace UE::Dataflow
{
	TArray<FIntVector3> CHAOSFLESHNODES_API GetSurfaceTriangles(const TArray<FIntVector4>& Tets, const bool bKeepInterior);

	/**
	*  GetMatchingMeshIndices
	*	Find matching geometry names in the collection. If MeshNames is empty return all the geometry indices. 
	*	@param MeshNames	: List of name to match, or empty for all indices
	*	@param Collection	: Collection to search
	*/
	TArray<int32> CHAOSFLESHNODES_API GetMatchingMeshIndices(const TArray<FString>& MeshNames, const FManagedArrayCollection* InCollection);

}
