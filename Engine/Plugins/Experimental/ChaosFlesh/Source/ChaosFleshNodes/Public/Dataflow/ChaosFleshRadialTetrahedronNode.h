// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "ChaosFleshRadialTetrahedronNode.generated.h"

class UStaticMesh;
class FFleshCollection;

// @todo(deprecate), rename to FRadialTetrahedronDataflowNode
USTRUCT(meta = (DataflowFlesh))
struct FRadialTetrahedronDataflowNodes : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FRadialTetrahedronDataflowNodes, "RadialTetrahedron", "Flesh", "")

public:
	typedef FManagedArrayCollection DataType;

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "0.0"))
	double InnerRadius = double(1.0);

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "0.0"))
	double OuterRadius = double(2.0);

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "0.0"))
	double Height = double(1.0);

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "2"))
	int32 RadialSample = 2;

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "3"))
	int32 AngularSample = 4;

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "2"))
	int32 VerticalSample = 2;

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "0.0", ClampMax = "1.0"))
	double BulgeRatio = double(0.0);

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	bool bDiscardInteriorTriangles = true;

	FRadialTetrahedronDataflowNodes(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};





