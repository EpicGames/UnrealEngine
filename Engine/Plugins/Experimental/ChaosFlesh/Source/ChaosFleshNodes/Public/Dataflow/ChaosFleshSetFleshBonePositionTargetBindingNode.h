// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "ChaosFleshSetFleshBonePositionTargetBindingNode.generated.h"

class USkeletalMesh;

UENUM(BlueprintType)
enum class ESkeletalBindingMode : uint8
{
	Dataflow_SkeletalBinding_Kinematic UMETA(DisplayName = "Kinematic"),
	Dataflow_SkeletalBinding_PositionTarget UMETA(DisplayName = "Position Target"),
	//
	Chaos_Max UMETA(Hidden)
};

/**
* Deprecated in 5.6 to support fixed distance search radius and improve binding method.
*/
USTRUCT(meta = (Deprecated = "5.6"))
struct FSetFleshBonePositionTargetBindingDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSetFleshBonePositionTargetBindingDataflowNode, "SetFleshBonePositionTargetBinding", "Flesh", "")

public:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	ESkeletalBindingMode SkeletalBindingMode = ESkeletalBindingMode::Dataflow_SkeletalBinding_PositionTarget;

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "0.0", EditCondition = "SkeletalBindingMode == ESkeletalBindingMode::Dataflow_SkeletalBinding_PositionTarget", EditConditionHides))
	float PositionTargetStiffness = 10000.f;

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowInput, DisplayName = "SkeletalMesh"))
	TObjectPtr<USkeletalMesh> SkeletalMeshIn = nullptr;

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "0.0", ClampMax = "2.0", UIMin = "0.0", UIMax = "2.0"))
	float VertexRadiusRatio = .001f;


	FSetFleshBonePositionTargetBindingDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&SkeletalMeshIn);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
* Binds vertices from Collection to bone skeletal mesh surface via kinematic or weak constraints.
*/
USTRUCT(meta = (DataflowFlesh))
struct FSetFleshBonePositionTargetBindingDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSetFleshBonePositionTargetBindingDataflowNode_v2, "SetFleshBonePositionTargetBinding", "Flesh", "")

public:
	FSetFleshBonePositionTargetBindingDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&SkeletalMeshIn);
		RegisterInputConnection(&VertexSelection);
		RegisterOutputConnection(&Collection, &Collection);
	}

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	ESkeletalBindingMode SkeletalBindingMode = ESkeletalBindingMode::Dataflow_SkeletalBinding_PositionTarget;

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "0.0", EditCondition = "SkeletalBindingMode == ESkeletalBindingMode::Dataflow_SkeletalBinding_PositionTarget", EditConditionHides))
	float PositionTargetStiffness = 10000.f;

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowInput, DisplayName = "SkeletalMesh"))
	TObjectPtr<USkeletalMesh> SkeletalMeshIn = nullptr;

	/** (optional) only create kinematic/weak constraints on vertices in the VertexSelection */
	UPROPERTY(meta = (DataflowInput, DisplayName = "(Optional) VertexSelection"))
	FDataflowVertexSelection VertexSelection;

	/**
	* Collection vertices are bound to their closest skeletal mesh vertices within the search radius
	*/
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "0.0"))
	float SearchRadius = 0.f;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

namespace UE::Dataflow
{
	void RegisterChaosFleshPositionTargetInitializationNodes();
}
