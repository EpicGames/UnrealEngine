// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/ChaosFleshNodesUtility.h"

#include "ChaosFleshSetFleshDefaultPropertiesNode.generated.h"


USTRUCT(meta = (DataflowFlesh))
struct FSetFleshDefaultPropertiesNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSetFleshDefaultPropertiesNode, "SetFleshDefaultProperties", "Flesh", "")

public:

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	float Density = 1.f;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	float VertexStiffness = 1e6;

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float VertexDamping = 0.f;

	/*Sets incompressibility on vertex basis. 0.6 is default behavior. 
	1 means absolutely incompressible. 0 means no incompressibility constraint on the material */
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "0.00001", ClampMax = "1.0", UIMin = "0.00001", UIMax = "1.0"))
	float VertexIncompressibility = 0.6f;

	/*Sets inflation on vertex basis. 0.5 means no inflation/deflation.
	1 means inflation to 2X volume on each dimension. 0 means the material is deflated to 0 volume.*/
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "0.00001", UIMin = "0.00001"))
	float VertexInflation = 0.5f;

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection"))
	FManagedArrayCollection Collection;

	FSetFleshDefaultPropertiesNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
