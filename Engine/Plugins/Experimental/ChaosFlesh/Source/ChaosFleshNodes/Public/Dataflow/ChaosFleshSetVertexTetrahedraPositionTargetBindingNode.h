// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "ChaosFleshSetVertexTetrahedraPositionTargetBindingNode.generated.h"

USTRUCT(meta = (DataflowFlesh))
struct FSetVertexTetrahedraPositionTargetBindingDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
		DATAFLOW_NODE_DEFINE_INTERNAL(FSetVertexTetrahedraPositionTargetBindingDataflowNode, "SetVertexTetrahedraPositionTargetBinding", "Flesh", "")

public:
	typedef FManagedArrayCollection DataType;


	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
		FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowInput, DisplayName = "TargetIndicesIn"))
		TArray<int32> TargetIndicesIn;

	UPROPERTY(meta = (DataflowInput, DisplayName = "GeometryGroupGuidsIn"))
		TArray<FString> GeometryGroupGuidsIn;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
		float PositionTargetStiffness = 10000.f;



	FSetVertexTetrahedraPositionTargetBindingDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterInputConnection(&TargetIndicesIn);
		RegisterInputConnection(&GeometryGroupGuidsIn);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
