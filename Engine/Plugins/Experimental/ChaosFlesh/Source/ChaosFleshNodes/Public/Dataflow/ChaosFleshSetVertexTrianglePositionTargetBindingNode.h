// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"
#include "Dataflow/DataflowSelection.h"

#include "ChaosFleshSetVertexTrianglePositionTargetBindingNode.generated.h"

class UDynamicMesh;

/**
 * Create point-triangle weak constraints (springs) between surface meshes of different geometries based on search radius.
 */
USTRUCT(meta = (DataflowFlesh))
struct FSetVertexTrianglePositionTargetBindingDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSetVertexTrianglePositionTargetBindingDataflowNode, "SetVertexTrianglePositionTargetBinding", "Flesh", "")

public:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	float PositionTargetStiffness = 1.f;

	/** (optional) only create weak constraints from surface vertices in VertexSelection to triangles in other geometries */
	UPROPERTY(meta = (DataflowInput, DisplayName = "(Optional) VertexSelection"))
	FDataflowVertexSelection VertexSelection;
	
	/** Search radius for point-triangle pairs between geometry surfaces. */
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "0.0"))
	float SearchRadius = 0.f;

	/** if point-triangle weak constraints created are anisotropic and allow sliding along the triangle plane */
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DisplayName = "Allow Sliding"))
	bool bAllowSliding = false;

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (InlineEditConditionToggle))
	bool bZeroRestLengthEditable = false;

	/** if point-triangle weak constraints created are zero rest-length.
	* if true, this will cause point triangle pair to stick together, as opposed to separated by their rest state distance.*/
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DisplayName = "Use zero rest length springs", EditCondition = "bZeroRestLengthEditable"))
	bool bUseZeroRestLengthSprings = false;

	FSetVertexTrianglePositionTargetBindingDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&VertexSelection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Delete vertex-triangle weak constraints (zero rest length springs) between VertexSelection1 and VertexSelection2.
 */
USTRUCT(meta = (DataflowFlesh))
struct FDeleteVertexTrianglePositionTargetBindingDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDeleteVertexTrianglePositionTargetBindingDataflowNode, "DeleteVertexTrianglePositionTargetBinding", "Flesh", "")

public:
	FDeleteVertexTrianglePositionTargetBindingDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&VertexSelection1);
		RegisterInputConnection(&VertexSelection2);
		RegisterOutputConnection(&Collection, &Collection);
	}

private:
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** This node deletes springs between VertexSelection1 and VertexSelection2. */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic, DisplayName = "VertexSelection1"))
	FDataflowVertexSelection VertexSelection1;

	/** This node deletes springs between VertexSelection1 and VertexSelection2. */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic, DisplayName = "VertexSelection2"))
	FDataflowVertexSelection VertexSelection2;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Set custom vertices so that only these vertices can collide with other surfaces. 
 * Unselected vertices will not collide with unselected vertices.
 */
USTRUCT(meta = (DataflowFlesh))
struct FSetCollidableVerticesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSetCollidableVerticesDataflowNode, "SetCollidableVertices", "Flesh", "")

public:
	FSetCollidableVerticesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&VertexSelection);
		RegisterOutputConnection(&Collection, &Collection);
	}

private:
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** Vertices selected to be able to collide with others. Unselected vertices will not collide with unselected vertices*/
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic, DisplayName = "Collision Vertex Selection"))
	FDataflowVertexSelection VertexSelection;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Create air tetrahedral constraint between point-triangle pair from surface meshes of different geometries based on search radius. 
 * The added tetrahedra help to maintain distance between geometries.
 * This node renders the boundary of the added tetrahedral mesh.
 */
USTRUCT(meta = (DataflowFlesh))
struct FCreateAirTetrahedralConstraintDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCreateAirTetrahedralConstraintDataflowNode, "CreateAirTetrahedralConstraint", "Flesh", "")

public:
	FCreateAirTetrahedralConstraintDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&VertexSelection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&DynamicMesh)
			.SetCanHidePin(true)
			.SetPinIsHidden(true);
	}

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** (optional) only create tetrahedral constraints from surface vertices in VertexSelection to triangles in other geometries.
	* For example, if the VertexSelection contains only one geometry, only this geometry will bind to other geometries.
	* No constraints will be created between two geometries that are not in the VertexSelection.
	*/
	UPROPERTY(meta = (DataflowInput, DisplayName = "(Optional) VertexSelection"))
	FDataflowVertexSelection VertexSelection;

	/** tetrahedral constraint search radius*/
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "0.0"))
	float SearchRadius = 0.f;

	/** Render dynamic mesh of the boundary mesh of added tetrahedra*/
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> DynamicMesh;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Creates volume constraint (defined by point-triangle tetrahedron volume) between surface meshes of different geometries.
 * This constraint allow sliding of the point along the triangle plane.
 */
USTRUCT(meta = (Experimental, DataflowFlesh))
struct FCreateAirVolumeConstraintDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCreateAirVolumeConstraintDataflowNode, "CreateAirVolumeConstraint", "Flesh", "")

public:
	FCreateAirVolumeConstraintDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&VertexSelection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&DynamicMesh)
			.SetCanHidePin(true)
			.SetPinIsHidden(true);
	}

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** (optional) only create volume constraints from surface vertices in VertexSelection to triangles in other geometries.
	* For example, if the VertexSelection contains only one geometry, only this geometry will bind to other geometries.
	* No constraints will be created between two geometries that are not in the VertexSelection. */
	UPROPERTY(meta = (DataflowInput, DisplayName = "(Optional) VertexSelection"))
	FDataflowVertexSelection VertexSelection;

	/** search radius for point-triangle pairs */
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "0.0"))
	float SearchRadius = 0.f;

	/** Stiffness of the volume constraint. This should be around the same magnitude as Young's modulus. */
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "0.0"))
	float Stiffness = 1.f;

	/** Render dynamic mesh of the boundary mesh of added volume */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> DynamicMesh;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};