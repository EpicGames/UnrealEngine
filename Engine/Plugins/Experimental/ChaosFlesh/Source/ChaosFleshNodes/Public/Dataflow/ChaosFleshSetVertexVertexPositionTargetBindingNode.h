// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "ChaosFleshSetVertexVertexPositionTargetBindingNode.generated.h"


USTRUCT(meta = (DataflowFlesh))
struct FSetVertexVertexPositionTargetBindingDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSetVertexVertexPositionTargetBindingDataflowNode, "SetVertexVertexPositionTargetBinding", "Flesh", "")

public:
	typedef FManagedArrayCollection DataType;

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
		float RadiusRatio = .1f;

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
		FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowInput, DisplayName = "TargetIndicesIn"))
		TArray<int32> TargetIndicesIn;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
		float PositionTargetStiffness = 10000.f;



	FSetVertexVertexPositionTargetBindingDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterInputConnection(&TargetIndicesIn);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
