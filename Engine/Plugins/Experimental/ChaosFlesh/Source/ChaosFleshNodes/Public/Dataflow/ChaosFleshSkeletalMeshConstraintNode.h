// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "ChaosFleshSkeletalMeshConstraintNode.generated.h"

class USkeletalMesh;

// @todo(deprecate), rename to FSkeletalMeshConstraintDataflowNode
USTRUCT(meta = (DataflowFlesh))
struct FKinematicSkeletalMeshInitializationDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FKinematicSkeletalMeshInitializationDataflowNode, "KinematicSkeletalMeshInitialization", "Flesh", "")

public:
	typedef FManagedArrayCollection DataType;

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection"))
		FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowInput, DisplayName = "SkeletalMesh"))
		TObjectPtr<USkeletalMesh> SkeletalMeshIn;
	
	UPROPERTY(meta = (DataflowOutput, DisplayName = "SelectionSet"))
		TArray<int32> IndicesOut;

	FKinematicSkeletalMeshInitializationDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&SkeletalMeshIn);
		RegisterOutputConnection(&Collection);
		RegisterOutputConnection(&IndicesOut);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

