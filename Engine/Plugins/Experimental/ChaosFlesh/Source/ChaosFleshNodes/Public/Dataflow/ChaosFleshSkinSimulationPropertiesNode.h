// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/ChaosFleshNodesUtility.h"

#include "ChaosFleshSkinSimulationPropertiesNode.generated.h"


/*
Set triangle mesh to simulate using skin constraints 
*/
USTRUCT(meta = (DataflowFlesh))
struct FSkinSimulationPropertiesDataflowNodes : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSkinSimulationPropertiesDataflowNodes, "SkinSimulationProperties", "Flesh", "")

public:
	typedef FManagedArrayCollection DataType;

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DisplayName = "UseSkinConstraints"))
	bool bSkinConstraints = false;

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection"))
	FManagedArrayCollection Collection;


	FSkinSimulationPropertiesDataflowNodes(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};
