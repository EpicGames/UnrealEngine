// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/ChaosFleshNodesUtility.h"

#include "ChaosFleshTriangleMeshSimulationPropertiesNode.generated.h"



/*
Convert tetmesh to simulate using surface traingle mesh only
*/
USTRUCT(meta = (DataflowFlesh))
struct FTriangleMeshSimulationPropertiesDataflowNodes : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FTriangleMeshSimulationPropertiesDataflowNodes, "TriangleMeshSimulationProperties", "Flesh", "")

public:
	typedef FManagedArrayCollection DataType;

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DisplayName = "MeshNames"))
	TArray<FString> MeshNames;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	float TriangleMeshDensity = 1.f;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	float VertexTriangleMeshStiffness = 1e6;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	float VertexTriangleMeshDamping = 0.f;

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection"))
	FManagedArrayCollection Collection;


	FTriangleMeshSimulationPropertiesDataflowNodes(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;


};
