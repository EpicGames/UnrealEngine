// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"
#include "Field/FieldSystemTypes.h"

#include "ChaosFleshVisualizeFiberFieldNode.generated.h"

/**
* Visualizes a muscle fiber direction per tetrahedron from a GeometryCollection containing tetrahedra.
*/
USTRUCT(meta = (DataflowFlesh))
struct FVisualizeFiberFieldNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FVisualizeFiberFieldNode, "VisualizeFiberField", "Flesh", "")

public:
	UPROPERTY(meta = (DataflowInput, DisplayName = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	float VectorScale = 1.0;

	UPROPERTY(meta = (DataflowOutput, DisplayName = "VectorField"))
	FFieldCollection VectorField;
	
	FVisualizeFiberFieldNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&VectorField);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
* Visualizes position target vectors from GeometryCollection.
*/
USTRUCT(meta = (DataflowFlesh))
struct FVisualizePositionTargetsNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FVisualizePositionTargetsNode, "VisualizePositionTargets", "Flesh", "")

public:
	UPROPERTY(meta = (DataflowInput, DisplayName = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowOutput, DisplayName = "VectorField"))
	FFieldCollection VectorField;

	FVisualizePositionTargetsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&VectorField);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
* Visualizes kinematic faces from GeometryCollection.
*/
USTRUCT(meta = (DataflowFlesh))
struct FVisualizeKinematicFacesNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FVisualizeKinematicFacesNode, "VisualizeKinematicFaces", "Flesh", "")

public:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	FVisualizeKinematicFacesNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};