// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "GeometryCollectionAppendCollectionTransformNode.generated.h"

//@todo(deprecate), move to GeometryCollection as AppendCollectionTransformDataflowNode
USTRUCT(meta = (DataflowFlesh))
struct FAppendToCollectionTransformAttributeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FAppendToCollectionTransformAttributeDataflowNode, "AppendToCollectionTransformAttribute", "Flesh", "")

public:
	typedef FManagedArrayCollection DataType;


	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
		FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowInput, DisplayName = "Transform"))
		FTransform TransformIn = FTransform::Identity;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
		FString AttributeName = FString("ComponentTransform");

	UPROPERTY(EditAnywhere, Category = "Dataflow")
		FString GroupName = FString("ComponentTransformGroup");

	FAppendToCollectionTransformAttributeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&TransformIn);
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

namespace UE::Dataflow
{
	void RegisterChaosFleshKinematicInitializationNodes();
}

